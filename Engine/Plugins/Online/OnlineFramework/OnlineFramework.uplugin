{
	"FileVersion" : 3,
	
	"FriendlyName" : "Online Framework Plugin",
	"Version" : 2,
	"VersionName" : "1.0",
	"Description" : "Shared code for interacting with online gameplay services.",
	"Category" : "Online Platform",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"EnabledByDefault" : false,

	"Modules": [
		{
			"Name": "Qos",
			"Type": "Runtime"
		},
		{
			"Name": "Party",
			"Type": "Runtime"
		},
		{
			"Name": "Lobby",
			"Type": "Runtime"
		},
		{
			"Name": "PatchCheck",
			"Type": "Runtime"
		},
		{
			"Name": "Hotfix",
			"Type": "Runtime"
		},
		{
			"Name": "Rejoin",
			"Type": "Runtime"
		},
		{
			"Name": "PlayTimeLimit",
			"Type": "Runtime"
		},
		{
			"Name": "LoginFlow",
			"Type": "ClientOnly",
			"PlatformDenyList": [
				"Mac"
			]
		}
	],
	"LocalizationTargets": [
		{
			"Name": "OnlineFramework",
			"LoadingPolicy": "Always"
		}
	],
	"Plugins": [
		{
			"Name": "OnlineFrameworkCommon",
			"Enabled": true
		},
		{
			"Name": "OnlineSubsystem",
			"Enabled": true
		},
		{
			"Name": "OnlineSubsystemUtils",
			"Enabled": true
		},
		{
			"Name": "OnlineServicesOSSAdapter",
			"Enabled": true
		},
		{
			"Name": "OnlineSubsystemGDK",
			"PlatformAllowList": [ "Win64" ],
			"Enabled": false,
			"Optional": true
		}
	]
}
