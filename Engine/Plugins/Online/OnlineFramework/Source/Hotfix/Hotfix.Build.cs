// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using UnrealBuildTool;

public class Hotfix : ModuleRules
{
	public Hotfix(ReadOnlyTargetRules Target) : base(Target)
    {
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		
        PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"Engine",
				"AssetRegistry",
				"MoviePlayerProxy",
				"OnlineSubsystemUtils",
				"Json",
			}
			);

		PublicDependencyModuleNames.AddRange(
			new string[] {
				"PatchCheck",
				"InstallBundleManager",
				"OnlineSubsystem",
			}
			);
	}
}
