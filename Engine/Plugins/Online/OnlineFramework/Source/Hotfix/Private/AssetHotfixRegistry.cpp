// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetHotfixRegistry.h"

#include "Misc/ConfigCacheIni.h"
#include "OnlineHotfixManager.h"
#include "UObject/Class.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetHotfixRegistry)

void FAssetHotfixRegistry::Reset()
{
	PendingAssetHotfixes.Empty();
}

TArray<FName> FAssetHotfixRegistry::ParseAndStoreHotfixEntries(FName SourceTag, const FConfigSectionMap& AssetHotfixSection)
{
	// The contents here are adapted from UOnlineHotfixManager::PatchAssetsFromIniFiles()
	// The main difference being that we store the hotfixes instead of loading the asset and hotfixing immediately

	TArray<FName> AffectedPaths;
	int32 TotalPatchableAssets = 0;
	const UEnum* EnumPtr = StaticEnum<EHotfixOperation>();
	for (FConfigSectionMap::TConstIterator It(AssetHotfixSection); It; ++It)
	{
		++TotalPatchableAssets;
		FString DataLine(*It.Value().GetValue());
		if (!DataLine.IsEmpty())
		{
			TArray<FString> Tokens;
			DataLine.ParseIntoArray(Tokens, TEXT(";"));
			if (Tokens.Num() == 3 || Tokens.Num() == 5)
			{
				const int64 HotfixTypeInt = EnumPtr->GetValueByNameString(Tokens[1]);
				if (HotfixTypeInt != INDEX_NONE)
				{
					// Package path, used as lookup key in TryApplyHotfixes.
					const FName AssetPath = FName(Tokens[0]);
					FPendingAssetHotfix PendingHotfix;
					PendingHotfix.SourceTag = SourceTag;
					PendingHotfix.AssetClassName = It.Key();
					PendingHotfix.Operation = (EHotfixOperation)HotfixTypeInt;
					if (PendingHotfix.Operation == EHotfixOperation::RowUpdate)
					{
						if (Tokens.Num() == 5)
						{
							// The hotfix line should be
							//	+DataTable=<data table path>;RowUpdate;<row name>;<column name>;<new value>
							//	+CurveTable=<curve table path>;RowUpdate;<row name>;<column name>;<new value>
							//	+CurveFloat=<curve float path>;RowUpdate;None;<column name>;<new value>
							PendingHotfix.RowName = FName(Tokens[2]);
							PendingHotfix.ColumnOrData = MoveTemp(Tokens[3]);
							PendingHotfix.Value = MoveTemp(Tokens[4]);
							PendingAssetHotfixes.FindOrAdd(AssetPath).Entries.Emplace(MoveTemp(PendingHotfix));
							AffectedPaths.AddUnique(AssetPath);
						}
						else
						{
							UE_LOGF(LogHotfixManager, Error, "[Item: %d] Expected a hotfix type RowUpdate with 5 tokens but got %d", TotalPatchableAssets, Tokens.Num());
						}
					}
					else if (PendingHotfix.Operation == EHotfixOperation::AddRow ||
							 PendingHotfix.Operation == EHotfixOperation::TableUpdate ||
							 PendingHotfix.Operation == EHotfixOperation::CurveUpdate)
					{
						if (Tokens.Num() == 3)
						{
							// The hotfix line should be
							//	+DataTable=<data table path>;AddRow;"<json data>"
							//	+DataTable=<data table path>;TableUpdate;"<json data>"
							//	+CurveTable=<curve table path>;TableUpdate;"<json data>"
							//	+CurveFloat=<curve float path>;CurveUpdate;"<json data>"
							//	+CurveVector=<curve vector path>;CurveUpdate;"<json data>"
							//	+CurveLinearColor=<curve linear color path>;CurveUpdate;"<json data>"
							PendingHotfix.ColumnOrData = MoveTemp(Tokens[2]);
							PendingAssetHotfixes.FindOrAdd(AssetPath).Entries.Emplace(MoveTemp(PendingHotfix));
							AffectedPaths.AddUnique(AssetPath);
						}
						else
						{
							UE_LOGF(LogHotfixManager, Error, "[Item: %d] Expected a hotfix type %ls with 3 tokens but got %d", TotalPatchableAssets, *Tokens[1], Tokens.Num());
						}
					}
					else
					{
						UE_LOGF(LogHotfixManager, Error, "[Item: %d] Unhandled hotfix operation type %ls", TotalPatchableAssets, *Tokens[1]);
					}
				}
				else
				{
					UE_LOGF(LogHotfixManager, Error, "[Item: %d] Unknown hotfix type %ls", TotalPatchableAssets, *Tokens[1]);
				}
			}
			else
			{
				UE_LOGF(LogHotfixManager, Error, "[Item: %d] Wasn't able to parse the data with semicolon separated values. Expecting 3 or 5 arguments but parsed %d.", TotalPatchableAssets, Tokens.Num());
			}
		}
		else
		{
			UE_LOGF(LogHotfixManager, Warning, "[Item: %d] Empty value given for '%ls' entry!", TotalPatchableAssets, *It.Key().ToString());
		}
	}

	if (TotalPatchableAssets == 0)
	{
		UE_LOGF(LogHotfixManager, Display, "No assets were found in the 'AssetHotfix' section in the Game .ini file. No patching needed.");
	}
	else
	{
		UE_LOGF(LogHotfixManager, Display, "Parsed %i entries from 'AssetHotfix' section in the Game .ini file.", TotalPatchableAssets);
	}

	return AffectedPaths;
}

TArray<FName> FAssetHotfixRegistry::ParseAndStoreFromRawIniData(FName SourceTag, const FString& IniData)
{
	FConfigFile TempConfig;
	TempConfig.ProcessInputFileContents(IniData, TEXT("Unknown, see FAssetHotfixRegistry::ParseAndStoreFromRawIniData"));

	if (const FConfigSection* Section = TempConfig.FindSection(TEXT("AssetHotfix")))
	{
		return ParseAndStoreHotfixEntries(SourceTag, *Section);
	}

	return TArray<FName>();
}

void FAssetHotfixRegistry::StoreHotfixEntry(FName AssetPath, FPendingAssetHotfix Hotfix)
{
	TArray<FPendingAssetHotfix>& Entries = PendingAssetHotfixes.FindOrAdd(AssetPath).Entries;

	if (FPendingAssetHotfix* Existing = Entries.FindByKey(Hotfix))
	{
		UE_LOGF(LogHotfixManager, Warning, "Duplicate hotfix entry for asset '%ls', updating value (Source='%ls', Op=%d, Row='%ls')",
			*AssetPath.ToString(), *Hotfix.SourceTag.ToString(), (int32)Hotfix.Operation, *Hotfix.RowName.ToString());
		Existing->Value = MoveTemp(Hotfix.Value);
	}
	else
	{
		Entries.Emplace(MoveTemp(Hotfix));
	}
}

const FPendingAssetHotfixEntries* FAssetHotfixRegistry::FindHotfixesForAsset(FName AssetPath) const
{
	return PendingAssetHotfixes.Find(AssetPath);
}

bool FAssetHotfixRegistry::RemoveEntriesFromSource(FName SourceTag)
{
	bool bRemovedAny = false;
	for (auto It = PendingAssetHotfixes.CreateIterator(); It; ++It)
	{
		FPendingAssetHotfixEntries& HotfixEntries = It->Value;

		const uint32 NumRemoved = HotfixEntries.Entries.RemoveAll([SourceTag](const FPendingAssetHotfix& Entry){ return Entry.SourceTag == SourceTag; });
		if (NumRemoved > 0)
		{
			bRemovedAny = true;
			if (HotfixEntries.Entries.IsEmpty())
			{
				It.RemoveCurrent();
			}
		}
	}
	return bRemovedAny;
}

TArray<FName> FAssetHotfixRegistry::GetAllPendingPaths() const
{
	TArray<FName> PendingPaths;
	PendingAssetHotfixes.GenerateKeyArray(PendingPaths);
	return PendingPaths;
}

bool FAssetHotfixRegistry::TryApplyHotfixes(
	UObject* Asset,
	UOnlineHotfixManager* HotfixManager,
	bool& bOutEncounteredErrors,
	TSet<class UDataTable*>* ChangedDataTables,
	TSet<class UCurveTable*>* ChangedCurveTables) const
{
	const FName AssetPathName = Asset->GetOutermost()->GetFName();
	const FPendingAssetHotfixEntries* PendingHotfixesEntries = FindHotfixesForAsset(AssetPathName);
	if (PendingHotfixesEntries == nullptr)
	{
		return false;
	}

	const FString AssetPath = AssetPathName.ToString();
	if (!HotfixManager->ShouldHotfixAsset(AssetPath))
	{
		return false;
	}

	TArray<FString> ProblemStrings;
	bool bAnyModified = false;
	for (const FPendingAssetHotfix& Hotfix : PendingHotfixesEntries->Entries)
	{
		const int32 ProblemsBefore = ProblemStrings.Num();
		switch (Hotfix.Operation)
		{
		case EHotfixOperation::RowUpdate:
			HotfixManager->HotfixRowUpdate(Asset, AssetPath, Hotfix.RowName.ToString(), Hotfix.ColumnOrData, Hotfix.Value, ProblemStrings, ChangedDataTables, ChangedCurveTables, Hotfix.SourceTag);
			break;
		case EHotfixOperation::AddRow:
		case EHotfixOperation::TableUpdate:
		case EHotfixOperation::CurveUpdate:
			{
				FString JsonData;
				int32 ReadLen = 0;
				int32 InputLen = Hotfix.ColumnOrData.Len();
				if (FParse::QuotedString(*Hotfix.ColumnOrData, JsonData, &ReadLen) && ReadLen == InputLen)
				{
					if (Hotfix.Operation == EHotfixOperation::AddRow)
					{
						HotfixManager->HotfixAddRow(Asset, AssetPath, JsonData, ProblemStrings, ChangedDataTables);
					}
					else
					{
						HotfixManager->HotfixTableUpdate(Asset, AssetPath, JsonData, ProblemStrings);
					}
				}
				else
				{
					ProblemStrings.Add(TEXT("Json data wasn't able to be parsed as a quoted string. Check that we have opening and closing quotes around the json data."));
				}
			}
			break;
		default:
			UE_LOGF(LogHotfixManager, Error, "%ls: Unhandled hotfix operation", *AssetPath);
			break;
		}
		
		const bool bHadErrors = (ProblemStrings.Num() != ProblemsBefore);
		bAnyModified |= !bHadErrors;
	}

	for (const FString& ProblemString : ProblemStrings)
	{
		UE_LOGF(LogHotfixManager, Error, "%ls: %ls", *GetPathNameSafe(Asset), *ProblemString);
	}

	bOutEncounteredErrors = !ProblemStrings.IsEmpty();

	return bAnyModified;
}
