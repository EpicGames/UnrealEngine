// Copyright Epic Games, Inc. All Rights Reserved.

#include "HotfixModule.h"
#include "Modules/ModuleManager.h"

IMPLEMENT_MODULE(FHotfixModule, Hotfix);

void FHotfixModule::StartupModule()
{
}

void FHotfixModule::ShutdownModule()
{
}
