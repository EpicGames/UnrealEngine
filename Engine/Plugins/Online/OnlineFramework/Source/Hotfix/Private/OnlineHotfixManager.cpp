// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineHotfixManager.h"
#include "Online.h"
#include "OnlineSubsystemUtils.h"
#include "UObject/UObjectIterator.h"
#include "UObject/Package.h"

#include "HAL/ConsoleManager.h"
#include "HAL/ThreadHeartBeat.h"

#include "Misc/PackageName.h"
#include "Misc/EngineVersion.h"
#include "Misc/FileHelper.h"
#include "Misc/CoreDelegates.h"
#include "Misc/App.h"
#include "Misc/ConfigUtilities.h"
#include "Misc/ConfigCacheIni.h"

#include "MoviePlayerProxy.h"

#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "Engine/CurveTable.h"
#include "Engine/DataTable.h"
#include "Curves/CurveFloat.h"
#include "Curves/CurveVector.h"
#include "Curves/CurveLinearColor.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Engine/GameInstance.h"
#include "Engine/GameViewportClient.h"

#include "AutoRTFM.h"
#include "Serialization/AsyncLoadingFlushContext.h"
#include "UObject/PropertyAccessUtil.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OnlineHotfixManager)

DEFINE_LOG_CATEGORY(LogHotfixManager);

/** This character must be between important pieces of file information (platform, initype, version) */
#define HOTFIX_SEPARATOR TEXT("_")
/** The prefix for any hotfix file that expects to indicate version information */
#define HOTFIX_VERSION_TAG TEXT("Ver-")
/** The prefix for any hotfix file that expects to indicate branch version information */
#define HOTFIX_BRANCH_VERSION_TAG TEXT("Branch-")
/** The prefix used when there no custom config so we don't try to match empty strings */
#define HOTFIX_NO_CUSTOM_CONFIG TEXT("##NOCONFIG##")

// do we support loading local HF files?
#define UE_TITLE_FILE_DEV_LOCAL_FILES_ENABLED !UE_BUILD_SHIPPING


FName NAME_HotfixManager(TEXT("HotfixManager"));

static TAutoConsoleVariable<int32> CVarUseNewDynamicLayersForHotfix(
	TEXT("ini.UseNewDynamicLayersForHotfix"),
	1,
	TEXT("If true, use the new dynamic layers that load/unload configs, specifically for Hotfixes"),
	ECVF_Default);

static TAutoConsoleVariable<bool> CVarEmptyDynamicHotfixContentsOnNewHotfix(
	TEXT("hotfix.EmptyDynamicHotfixContentsOnNewHotfix"),
	false,
	TEXT("If true clear out the existing values for DynamicHotfixContents when a new hotfix is processed. When set to false only remove the files which were removed since the last hotfix."),
	ECVF_Default);

#if !UE_BUILD_SHIPPING
/* static */ bool UOnlineHotfixManager::bAlwaysReapplyHotfixes = false;
static FAutoConsoleVariableRef CVarAlwaysReapplyHotfixes(
	TEXT("hotfix.AlwaysReapply"),
	UOnlineHotfixManager::bAlwaysReapplyHotfixes,
	TEXT("Whether hotfixes should be downloaded and reapplied on every check, for debugging"));
#endif // !UE_BUILD_SHIPPING

class FPakFileVisitor : public IPlatformFile::FDirectoryVisitor
{
public:
	virtual bool Visit(const TCHAR* FilenameOrDirectory, bool bIsDirectory) override
	{
		if (!bIsDirectory)
		{
			Files.Add(FilenameOrDirectory);
		}
		return true;
	}

	TArray<FString> Files;
};

namespace OnlineHotfixManagerCVars
{
	static bool bDeferBroadcastCurveTableModified = true;
	static FAutoConsoleVariableRef DeferBroadcastCurveTableModifiedCVar(
		TEXT("hotfix.DeferBroadcastCurveTableModified"),
		bDeferBroadcastCurveTableModified,
		TEXT("Whether to wait until all asset hotfixes have been applied before broadcasting OnCurveTableChanged delegates, as opposed to broadcasting after each individual modification"),
		ECVF_Default);

	static bool bUseHotfixOnLoad = false;
	static FAutoConsoleVariableRef UseHotfixOnLoadCVar(
		TEXT("hotfix.UseHotfixOnLoad"),
		bUseHotfixOnLoad,
		TEXT("If true, hotfix assets as they are loaded instead of force-loading all hotfixable assets and applying hotfixes eagerly"),
		ECVF_Default);

	static bool bHotfixOnLoadIncludesOnDiskPluginInis = true;
	static FAutoConsoleVariableRef HotfixOnLoadIncludesOnDiskPluginInisCVar(
		TEXT("hotfix.HotfixOnLoadIncludesOnDiskPluginInis"),
		bHotfixOnLoadIncludesOnDiskPluginInis,
		TEXT("If true, when the hotfix on load system is active it will also load and apply syntax from on-disk plugin inis"),
		ECVF_Default);

	static bool bHotfixOnLoadUnconditionallyApplyChanges = true;
	static FAutoConsoleVariableRef HotfixOnLoadUnconditionallyApplyChanges(
		TEXT("hotfix.HotfixOnLoadUnconditionallyApplyChanges"),
		bHotfixOnLoadUnconditionallyApplyChanges,
		TEXT("If true, when the hotfix on load system is active it will apply changes even when no changes are reported. Necessary to accomodate instances when Default inis are re-applied for non-plugin assets."),
		ECVF_Default);
}

namespace
{
	/** @return the expected network version for hotfix files determined at compile time */
	FString GetNetworkVersion()
	{
		static FString NetVerStr;
		if (NetVerStr.IsEmpty())
		{
			uint32 NetVer = FNetworkVersion::GetNetworkCompatibleChangelist();
			NetVerStr = FString::Printf(TEXT("%s%d"), HOTFIX_VERSION_TAG, NetVer);
		}
		return NetVerStr;
	}

	/** @return the expected branch version for hotfix files determined at compile time */
	FString GetBranchVersion()
	{
		static FString BranchVersion;
		if (BranchVersion.IsEmpty())
		{
			BranchVersion = HOTFIX_BRANCH_VERSION_TAG + FPaths::GetCleanFilename(FEngineVersion::Current().GetBranch());
		}
		return BranchVersion;
	}

	/**
	 * Given a hotfix file name, return the file name with version stripped out and exposed separately
	 *
	 * @param InFilename name of file to search for version information
	 * @param OutFilename name with version information removed
	 * @param OutNetVersion version of the hotfix file it present in the name
	 * @param OutBranchVersion version of the hotfix file it present in the name
	 */
	void GetFilenameAndVersion(const FString& InFilename, FString& OutFilename, FString& OutNetVersion, FString& OutBranchVersion)
	{
		TArray<FString> FileParts;
		int32 NumTokens = InFilename.ParseIntoArray(FileParts, HOTFIX_SEPARATOR);
		if (NumTokens > 0)
		{
			for (int i = 0; i < FileParts.Num(); i++)
			{
				if (FileParts[i].StartsWith(HOTFIX_VERSION_TAG))
				{
					OutNetVersion = FileParts[i];
				}
				else if (FileParts[i].StartsWith(HOTFIX_BRANCH_VERSION_TAG))
				{
					OutBranchVersion = FileParts[i];
				}
				else
				{
					OutFilename += FileParts[i];
					if (i < FileParts.Num() - 1)
					{
						OutFilename += HOTFIX_SEPARATOR;
					}
				}
			}
		}
	}

}

void UOnlineHotfixManager::BroadcastTableChanges(const TSet<UDataTable*>& ChangedDataTables, const TSet<UCurveTable*>& ChangedCurveTables)
{
	for (UDataTable* DataTable : ChangedDataTables)
	{
		if (DataTable != nullptr)
		{
			DataTable->HandleDataTableChanged();
		}
	}
	for (UCurveTable* CurveTable : ChangedCurveTables)
	{
		if (CurveTable != nullptr)
		{
			CurveTable->OnCurveTableChanged().Broadcast();
		}
	}
}

bool UOnlineHotfixManager::IsCompatibleHotfixFile(const FString& InFilename, FString& OutFilename)
{
	bool bHasNetVersion = false;
	bool bCompatibleNetHotfix = false;
	bool bHasBranchVersion = false;
	bool bCompatibleBranchHotfix = false;
	FString OutNetVersion;
	FString OutBranchVersion;
	GetFilenameAndVersion(InFilename, OutFilename, OutNetVersion, OutBranchVersion);

	if (!OutNetVersion.IsEmpty())
	{
		bHasNetVersion = true;
		const FString NetworkVersion = GetNetworkVersion();
		if (OutNetVersion == NetworkVersion)
		{
			bCompatibleNetHotfix = true;
		}
	}

	if (!OutBranchVersion.IsEmpty())
	{
		bHasBranchVersion = true;
		const FString BranchVersion = GetBranchVersion();
		if (OutBranchVersion == BranchVersion)
		{
			bCompatibleBranchHotfix = true;
		}
	}

	return (bCompatibleNetHotfix || !bHasNetVersion) && (bCompatibleBranchHotfix || !bHasBranchVersion);
}

UOnlineHotfixManager::UOnlineHotfixManager() :
	Super(),
	TotalFiles(0),
	NumDownloaded(0),
	TotalBytes(0),
	NumBytes(0),
	bHotfixingInProgress(false),
	bHotfixNeedsMapReload(false),
	ChangedOrRemovedPakCount(0)
{
	OnEnumerateFilesCompleteDelegate = FOnEnumerateFilesCompleteDelegate::CreateUObject(this, &UOnlineHotfixManager::OnEnumerateFilesComplete);
	OnReadFileProgressDelegate = FOnReadFileProgressDelegate::CreateUObject(this, &UOnlineHotfixManager::OnReadFileProgress);
	OnReadFileCompleteDelegate = FOnReadFileCompleteDelegate::CreateUObject(this, &UOnlineHotfixManager::OnReadFileComplete);
#if !UE_BUILD_SHIPPING
	bLogMountedPakContents = FParse::Param(FCommandLine::Get(), TEXT("LogHotfixPakContents"));
#endif
	GameContentPath = FString() / FApp::GetProjectName() / TEXT("Content");

	if (this != GetClass()->GetDefaultObject())
	{
		if (!UObject::IsGarbageEliminationEnabled())
		{
			FCoreUObjectDelegates::GetPreGarbageCollectDelegate().AddUObject(this, &UOnlineHotfixManager::StopTrackingInvalidHotfixedAssets);
		}

		UE::DynamicConfig::HotfixPluginForBranch.AddUObject(this, &UOnlineHotfixManager::HotfixDynamicBranch);
		UE::DynamicConfig::RemovePluginHotfixesFromDynamicBranch.AddUObject(this, &UOnlineHotfixManager::RemoveHotfixesFromDynamicBranch);
	}
}

UOnlineHotfixManager::UOnlineHotfixManager(FVTableHelper& Helper)
	: Super(Helper)
{
}

UOnlineHotfixManager::~UOnlineHotfixManager()
{
	if (!UObject::IsGarbageEliminationEnabled())
	{
		FCoreUObjectDelegates::GetPreGarbageCollectDelegate().RemoveAll(this);
	}
}

UClass* UOnlineHotfixManager::GetConfiguredHotfixManagerClass()
{
	UOnlineHotfixManager* DefaultObject = UOnlineHotfixManager::StaticClass()->GetDefaultObject<UOnlineHotfixManager>();
	FString HotfixManagerClassName = DefaultObject->HotfixManagerClassName;
	UClass* HotfixManagerClass = LoadClass<UOnlineHotfixManager>(nullptr, *HotfixManagerClassName, nullptr, LOAD_None, nullptr);
	if (HotfixManagerClass == nullptr)
	{
		// Retrieve the static value, before any plugins, hotfixes, or any other dynamic configs have been applied, as a fallback
		// This is not a standard method for retrieving config values, and should not be used as a general solution in other code
		FString FallbackClassName;
		GConfig->FindBranch(TEXT("Engine"), TEXT("DefaultEngine"))->CombinedStaticLayers.GetString(TEXT("/Script/Hotfix.OnlineHotfixManager"), TEXT("HotfixManagerClassName"), FallbackClassName);
		if (FallbackClassName.Len())
		{
			UE_LOGF(LogHotfixManager, Display, "Unable to locate hotfix manager class '%ls'. Trying '%ls' instead.", *HotfixManagerClassName, *FallbackClassName)
				HotfixManagerClassName = FallbackClassName;
			HotfixManagerClass = LoadClass<UOnlineHotfixManager>(nullptr, *HotfixManagerClassName, nullptr, LOAD_None, nullptr);
		}
		if (HotfixManagerClass == nullptr)
		{
			UE_LOGF(LogHotfixManager, Warning, "Failed to find requested hotfix manager class '%ls'. Falling back to default.", *HotfixManagerClassName);
			// Just use the default class if it couldn't load what was specified
			HotfixManagerClass = UOnlineHotfixManager::StaticClass();
		}
	}

	return HotfixManagerClass;
}

UOnlineHotfixManager* UOnlineHotfixManager::Get(UWorld* World)
{
	UOnlineHotfixManager* DefaultObject = UOnlineHotfixManager::StaticClass()->GetDefaultObject<UOnlineHotfixManager>();
	IOnlineSubsystem* OnlineSub = Online::GetSubsystem(World, DefaultObject->OSSName.Len() > 0 ? FName(*DefaultObject->OSSName) : NAME_None);
	if (OnlineSub != nullptr)
	{
		UOnlineHotfixManager* HotfixManager = Cast<UOnlineHotfixManager>(OnlineSub->GetNamedInterface(NAME_HotfixManager));
		if (HotfixManager == nullptr)
		{
			UClass* HotfixManagerClass = GetConfiguredHotfixManagerClass();
			// Create it and store it
			HotfixManager = NewObject<UOnlineHotfixManager>(GetTransientPackage(), HotfixManagerClass);
			OnlineSub->SetNamedInterface(NAME_HotfixManager, HotfixManager);
		}

		if (World)
		{
			HotfixManager->OwnerWorld = World;
		}
		return HotfixManager;
	}
	return nullptr;
}

void UOnlineHotfixManager::PostInitProperties()
{
#if !UE_BUILD_SHIPPING
	FParse::Value(FCommandLine::Get(), TEXT("HOTFIXPREFIX="), DebugPrefix);
	if (!DebugPrefix.IsEmpty() && !DebugPrefix.EndsWith(HOTFIX_SEPARATOR))
	{
		DebugPrefix += HOTFIX_SEPARATOR;
	}
#endif
	// So we only try to apply files for this platform
	PlatformPrefix = DebugPrefix + ANSI_TO_TCHAR(FPlatformProperties::PlatformName());
	PlatformPrefix += HOTFIX_SEPARATOR;
	// Server prefix
	ServerPrefix = DebugPrefix + GetDedicatedServerPrefix();
	// Custom Config prefix
	CustomConfigPrefix = FConfigCacheIni::GetCustomConfigString().IsEmpty() ? HOTFIX_NO_CUSTOM_CONFIG : (DebugPrefix + FConfigCacheIni::GetCustomConfigString());
	CustomConfigPrefix += HOTFIX_SEPARATOR;

	// Build the default prefix too
	DefaultPrefix = DebugPrefix + TEXT("Default");

#if UE_TITLE_FILE_DEV_LOCAL_FILES_ENABLED
	// if we are going to load from local hotfix files, use the OnlineTitleFileNull class, which only loads from local files
	FString TitleFileLocalPath, HotfixLocalPath;
	if (FParse::Value(FCommandLine::Get(), TEXT("TitleFileLocalPath="), TitleFileLocalPath) || FParse::Value(FCommandLine::Get(), TEXT("HotfixLocalPath="), HotfixLocalPath))
	{
		// if HotfixLocalPath is specified, check that it either matches an existing TitleFileLocalPath, or if it doesn't exist, then alias it to 
		// TitleFileLocalPath so that the OnlineTitleFileNull code can use the proper option
		if (!HotfixLocalPath.IsEmpty())
		{
			if (TitleFileLocalPath.IsEmpty())
			{
				FString LocalCommandLine = FString::Printf(TEXT(" TitleFileLocalPath=\"%s\""), *HotfixLocalPath);
				FCommandLine::Append(*LocalCommandLine);
			}
			else
			{
				ensure(HotfixLocalPath == TitleFileLocalPath);
			}
		}

		OSSName = "Null";
	}
#endif

	Super::PostInitProperties();
}

void UOnlineHotfixManager::BeginDestroy()
{
	TeardownHotfixOnLoad();
	Super::BeginDestroy();
}

void UOnlineHotfixManager::ConfigureHotfixOnLoad()
{
	// Evaluate the hotfix-on-load CVar once. This cannot run in PostInitProperties
	// because config files have not been downloaded yet. Latched so that mid-session
	// CVar changes do not arm the delegate against an empty asset registry.
	if (bConfiguredHotfixOnLoad)
	{
		return;
	}
	bConfiguredHotfixOnLoad = true;

	if (!OnlineHotfixManagerCVars::bUseHotfixOnLoad)
	{
		return;
	}

	for (const FSoftClassPath& HotfixableClass : HotfixableAssetClasses)
	{
		if (UClass* ResolvedClass = HotfixableClass.TryLoadClass<UObject>())
		{
			ResolvedHotfixableAssetClasses.Add(ResolvedClass);
			FCoreUObjectDelegates::RegisterHotfixableAssetClass(ResolvedClass);
		}
		else
		{
			UE_LOGF(LogHotfixManager, Warning, "Unable to locate hotfixable asset class '%ls'", *HotfixableClass.ToString());
		}
	}
	OnHotfixableAssetLoadedHandle = FCoreUObjectDelegates::OnHotfixableAssetLoaded.AddUObject(this, &UOnlineHotfixManager::OnHotfixableAssetLoaded);
	bHotfixOnLoadEnabled = true;
}

void UOnlineHotfixManager::OnHotfixProcessSkipped()
{
	ConfigureHotfixOnLoad();
}

void UOnlineHotfixManager::TeardownHotfixOnLoad()
{
	if (!bHotfixOnLoadEnabled)
	{
		return;
	}

	FCoreUObjectDelegates::OnHotfixableAssetLoaded.Remove(OnHotfixableAssetLoadedHandle);
	for (UClass* ResolvedClass : ResolvedHotfixableAssetClasses)
	{
		FCoreUObjectDelegates::UnregisterHotfixableAssetClass(ResolvedClass);
	}
	ResolvedHotfixableAssetClasses.Empty();
	bHotfixOnLoadEnabled = false;
}

void UOnlineHotfixManager::Init()
{
	bHotfixingInProgress = true;
	bHotfixNeedsMapReload = false;
	TotalFiles = 0;
	NumDownloaded = 0;
	TotalBytes = 0;
	NumBytes = 0;
	ChangedOrRemovedPakCount = 0;
	OnlineTitleFile = Online::GetTitleFileInterface(OSSName.Len() ? FName(*OSSName, FNAME_Find) : NAME_None);
	if (OnlineTitleFile.IsValid())
	{
		OnEnumerateFilesCompleteDelegateHandle = OnlineTitleFile->AddOnEnumerateFilesCompleteDelegate_Handle(OnEnumerateFilesCompleteDelegate);
		OnReadFileProgressDelegateHandle = OnlineTitleFile->AddOnReadFileProgressDelegate_Handle(OnReadFileProgressDelegate);
		OnReadFileCompleteDelegateHandle = OnlineTitleFile->AddOnReadFileCompleteDelegate_Handle(OnReadFileCompleteDelegate);
	}
}

void UOnlineHotfixManager::Cleanup()
{
	PendingHotfixFiles.Empty();
	if (OnlineTitleFile.IsValid())
	{
		// Make sure to give back the memory used when reading the hotfix files
		OnlineTitleFile->ClearFiles();
		OnlineTitleFile->ClearOnEnumerateFilesCompleteDelegate_Handle(OnEnumerateFilesCompleteDelegateHandle);
		OnlineTitleFile->ClearOnReadFileProgressDelegate_Handle(OnReadFileProgressDelegateHandle);
		OnlineTitleFile->ClearOnReadFileCompleteDelegate_Handle(OnReadFileCompleteDelegateHandle);
	}
	OnlineTitleFile = nullptr;
	bHotfixingInProgress = false;
	AsyncFlushContext = nullptr;
}

void UOnlineHotfixManager::OnHotfixableAssetLoaded(UObject* Asset)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_UOnlineHotfixManager_OnHotfixableAssetLoaded);

	ensure(IsInGameThread());

	// Early out if we don't have any hotfixes
	if (!AssetHotfixRegistry.HasPendingHotfixes())
	{
		return;
	}

	TSet<UDataTable*> ChangedDataTables;
	TSet<UCurveTable*> ChangedCurveTables;

	// Apply all hotfixes to this asset
	bool bEncounteredErrors = false;
	if (AssetHotfixRegistry.TryApplyHotfixes(Asset, this, bEncounteredErrors, &ChangedDataTables, &ChangedCurveTables))
	{
		UE_LOGF(LogHotfixManager, Log, "Hotfixed '%ls'", *Asset->GetFullName());
		BroadcastTableChanges(ChangedDataTables, ChangedCurveTables);
	}
}

void UOnlineHotfixManager::StartHotfixProcess()
{
	UE_LOGF(LogHotfixManager, Log, "Starting Hotfix Process");

	// Patching the editor this way seems like a bad idea
	const bool bShouldHotfix = ShouldPerformHotfix();
	if (!bShouldHotfix)
	{
		UE_LOGF(LogHotfixManager, Warning, "Hotfixing skipped when not running game/server");
		FinalizeHotfixProcess(EHotfixResult::SuccessNoChange);
		return;
	}

	if (bHotfixingInProgress)
	{
		UE_LOGF(LogHotfixManager, Warning, "Hotfixing already in progress");
		return;
	}

	Init();
	// Kick off an enumeration of the files that are available to download
	if (OnlineTitleFile.IsValid())
	{
		OnlineTitleFile->EnumerateFiles();
	}
	else
	{
		UE_LOGF(LogHotfixManager, Error, "Failed to start the hotfixing process due to no OnlineTitleInterface present for OSS(%ls)", *OSSName);
		FinalizeHotfixProcess(EHotfixResult::Failed);
	}
}

struct FHotfixFileSortPredicate
{
	struct FHotfixFileNameSortPredicate
	{
		const FString PlatformPrefix;
		const FString ServerPrefix;
		const FString DefaultPrefix;
		const FString CustomConfigPrefix;

		FHotfixFileNameSortPredicate(const FString& InPlatformPrefix, const FString& InServerPrefix, const FString& InDefaultPrefix, const FString& InCustomConfigPrefix) :
			PlatformPrefix(InPlatformPrefix),
			ServerPrefix(InServerPrefix),
			DefaultPrefix(InDefaultPrefix),
			CustomConfigPrefix(InCustomConfigPrefix)
		{
		}

		int32 GetPriorityForCompare(const FString& InHotfixName) const
		{
			// Non-ini files are applied last
			int32 Priority = 50;

			if (InHotfixName.EndsWith(TEXT("INI")))
			{
				FString HotfixName, NetworkVersion, BranchVersion;
				GetFilenameAndVersion(InHotfixName, HotfixName, NetworkVersion, BranchVersion);

				// Defaults are applied first
				if (HotfixName.StartsWith(DefaultPrefix))
				{
					Priority = 10;
				}
				// Server trumps default
				else if (HotfixName.StartsWith(ServerPrefix))
				{
					Priority = 20;
				}
				// Platform trumps server
				else if (HotfixName.StartsWith(PlatformPrefix))
				{
					Priority = 30;
				}
				// CustomConfig trumps platform
				else if (HotfixName.StartsWith(CustomConfigPrefix))
				{
					Priority = 40;
				}
				// Other INIs listed in game override of WantsHotfixProcessing will trump all other INIs
				else
				{
					Priority = 50;
				}

				if (!BranchVersion.IsEmpty())
				{
					// Branch versioned hotfixes apply after all but net versioned hotfixes within their type
					Priority += 3;
				}

				if (!NetworkVersion.IsEmpty())
				{
					// Network versioned hotfixes apply last within their type
					Priority += 5;
				}
			}

			return Priority;
		}

		bool Compare(const FString& A, const FString& B) const
		{
			int32 APriority = GetPriorityForCompare(A);
			int32 BPriority = GetPriorityForCompare(B);
			if (APriority != BPriority)
			{
				return APriority < BPriority;
			}
			else
			{
				// Fall back to sort by the string order if both have same priority
				return A < B;
			}
		}
	};
	FHotfixFileNameSortPredicate FileNameSorter;

	FHotfixFileSortPredicate(const FString& InPlatformPrefix, const FString& InServerPrefix, const FString& InDefaultPrefix, const FString& InCustomConfigPrefix) :
		FileNameSorter(InPlatformPrefix, InServerPrefix, InDefaultPrefix, InCustomConfigPrefix)
	{
	}

	bool operator()(const FCloudFileHeader &A, const FCloudFileHeader &B) const
	{
		return FileNameSorter.Compare(A.FileName, B.FileName);
	}

	bool operator()(const FString& A, const FString& B) const
	{
		return FileNameSorter.Compare(FPaths::GetCleanFilename(A), FPaths::GetCleanFilename(B));
	}
};

void UOnlineHotfixManager::OnEnumerateFilesComplete(bool bWasSuccessful, const FString& ErrorStr)
{
	UE_LOGF(LogHotfixManager, Log, "EnumerateFiles Http Request Complete");

	if (bWasSuccessful)
	{
		check(OnlineTitleFile.IsValid());
		// Cache our current set so we can compare for differences
		LastHotfixFileList = HotfixFileList;
		HotfixFileList.Empty();
		// Get the new header data
		OnlineTitleFile->GetFileList(HotfixFileList);
		FilterHotfixFiles();
		// Reduce the set of work to just the files that changed since last run
		BuildHotfixFileListDeltas();
		// Sort after filtering so that the comparison below doesn't fail to different order from the server
		ChangedHotfixFileList.Sort<FHotfixFileSortPredicate>(FHotfixFileSortPredicate(PlatformPrefix, ServerPrefix, DefaultPrefix, CustomConfigPrefix));
		// Read any changed files
		if (ChangedHotfixFileList.Num() > 0)
		{
			// Update our totals for our progress delegates
			TotalFiles = ChangedHotfixFileList.Num();
			for (const FCloudFileHeader& FileHeader : ChangedHotfixFileList)
			{
				TotalBytes += FileHeader.FileSize;
			}
			ReadHotfixFiles();
		}
		else
		{
			if (RemovedHotfixFileList.Num() > 0)
			{
				UE_LOGF(LogHotfixManager, Display, "Files have been removed since last check. Reverting.");

				// Prevent async loading while reverting hotfixes.
				check(!AsyncFlushContext);
				AsyncFlushContext = MakeUnique<FAsyncLoadingFlushContext>(TEXT("RevertHotfix"));
				AsyncFlushContext->Flush(
					FOnAsyncLoadingFlushComplete::CreateWeakLambda(
						this,
						[this]()
						{
							// No changes, just reverts
							// Perform any undo operations needed
							RestoreBackupIniFiles();
							UnmountHotfixFiles();

							for (const FCloudFileHeader& FileHeader : RemovedHotfixFileList)
							{
								TriggerOnHotfixRemovedFileDelegates(FileHeader.FileName);
							}
							
							FinalizeHotfixProcess(EHotfixResult::SuccessNoChange);
						}));
			}
			else
			{
				UE_LOGF(LogHotfixManager, Display, "Returned hotfix data is the same as last application, skipping the apply phase");
				FinalizeHotfixProcess(EHotfixResult::SuccessNoChange);
			}
		}
	}
	else
	{
		UE_LOGF(LogHotfixManager, Warning, "Enumeration of hotfix files failed");
		FinalizeHotfixProcess(EHotfixResult::Failed);
	}
}

void UOnlineHotfixManager::CheckAvailability(FOnHotfixAvailableComplete& InCompletionDelegate)
{
	// Checking for hotfixes in editor is not supported
	const bool bShouldHotfix = ShouldPerformHotfix();
	if (!bShouldHotfix)
	{
		UE_LOGF(LogHotfixManager, Warning, "Hotfixing availability skipped when not running game/server");
		InCompletionDelegate.ExecuteIfBound(EHotfixResult::SuccessNoChange);
		return;
	}

	if (bHotfixingInProgress)
	{
		UE_LOGF(LogHotfixManager, Warning, "Hotfixing availability skipped because hotfix in progress");
		InCompletionDelegate.ExecuteIfBound(EHotfixResult::Failed);
		return;
	}

	OnlineTitleFile = Online::GetTitleFileInterface(OSSName.Len() ? FName(*OSSName, FNAME_Find) : NAME_None);

	bHotfixingInProgress = true;

	// Kick off an enumeration of the files that are available to download
	if (OnlineTitleFile.IsValid())
	{
		FOnEnumerateFilesCompleteDelegate OnEnumerateFilesForAvailabilityCompleteDelegate;
		OnEnumerateFilesForAvailabilityCompleteDelegate.BindUObject(this, &UOnlineHotfixManager::OnEnumerateFilesForAvailabilityComplete, InCompletionDelegate);
		OnEnumerateFilesForAvailabilityCompleteDelegateHandle = OnlineTitleFile->AddOnEnumerateFilesCompleteDelegate_Handle(OnEnumerateFilesForAvailabilityCompleteDelegate);

		// The hotfix process isn't purely reversible as it kicks off HTTP requests, so we defer the work until transaction commit time.
		UE_AUTORTFM_ONCOMMIT(this)
		{
			OnlineTitleFile->EnumerateFiles();
		};
	}
	else
	{
		UE_LOGF(LogHotfixManager, Error, "Failed to start the hotfix check process due to no OnlineTitleInterface present for OSS(%ls)", *OSSName);
		FinalizeHotfixProcess(EHotfixResult::Failed);
	}
}

void UOnlineHotfixManager::OnHotfixAvailablityCheck(const TArray<FCloudFileHeader>& PendingChangedFiles, const TArray<FCloudFileHeader>& PendingRemoveFiles)
{
	// empty in base class
}

void UOnlineHotfixManager::OnEnumerateFilesForAvailabilityComplete(bool bWasSuccessful, const FString& ErrorStr, FOnHotfixAvailableComplete InCompletionDelegate)
{
	if (OnlineTitleFile.IsValid())
	{
		OnlineTitleFile->ClearOnEnumerateFilesCompleteDelegate_Handle(OnEnumerateFilesForAvailabilityCompleteDelegateHandle);
	}

	EHotfixResult Result = EHotfixResult::Failed;
	if (bWasSuccessful)
	{
		TArray<FCloudFileHeader> TmpHotfixFileList;
		TArray<FCloudFileHeader> TmpLastHotfixFileList;

		TmpHotfixFileList = HotfixFileList;
		TmpLastHotfixFileList = LastHotfixFileList;

		// Cache our current set so we can compare for differences
		LastHotfixFileList = HotfixFileList;
		HotfixFileList.Empty();
		// Get the new header data
		OnlineTitleFile->GetFileList(HotfixFileList);
		FilterHotfixFiles();
		// Reduce the set of work to just the files that changed since last run
		BuildHotfixFileListDeltas();

		// Read any changed files
		if (ChangedHotfixFileList.Num() > 0 || RemovedHotfixFileList.Num() > 0)
		{
			UE_LOGF(LogHotfixManager, Display, "Hotfix files available");
			Result = EHotfixResult::Success;
		}
		else
		{
			UE_LOGF(LogHotfixManager, Display, "Returned hotfix data is the same as last application, returning nothing to do");
			Result = EHotfixResult::SuccessNoChange;
		}

		OnHotfixAvailablityCheck(ChangedHotfixFileList, RemovedHotfixFileList);

		// Restore state to before the check
		RemovedHotfixFileList.Empty();
		ChangedHotfixFileList.Empty();
		HotfixFileList = TmpHotfixFileList;
		LastHotfixFileList = TmpLastHotfixFileList;
	}
	else
	{
		UE_LOGF(LogHotfixManager, Warning, "Enumeration of hotfix files failed");
	}

	OnlineTitleFile = nullptr;
	bHotfixingInProgress = false;
	InCompletionDelegate.ExecuteIfBound(Result);
}

void UOnlineHotfixManager::BuildHotfixFileListDeltas()
{
	RemovedHotfixFileList.Empty();
	ChangedHotfixFileList.Empty();
	// Go through the current list and see if it's changed from the previous attempt
	TSet<FString> DirtyIniCategories;
	for (const FCloudFileHeader& CurrentHeader : HotfixFileList)
	{
		bool bFoundMatch = LastHotfixFileList.Contains(CurrentHeader);
		if (!bFoundMatch || bAlwaysReapplyHotfixes)
		{
			// All NEW or CHANGED ini files will be added to the process list
			ChangedHotfixFileList.Add(CurrentHeader);

			if (CurrentHeader.FileName.EndsWith(TEXT(".INI"), ESearchCase::IgnoreCase))
			{
				// Make sure that ALL INIs of this "category" get marked for inclusion below
				DirtyIniCategories.Add(GetStrippedConfigFileName(CurrentHeader.FileName));
			}
		}
	}
	// Find any files that have been removed from the set of hotfix files
	for (const FCloudFileHeader& LastHeader : LastHotfixFileList)
	{
		bool bFoundMatch = HotfixFileList.ContainsByPredicate(
			[&LastHeader](const FCloudFileHeader& CurrentHeader)
		{
			return LastHeader.FileName == CurrentHeader.FileName;
		});
		if (!bFoundMatch)
		{
			// We've been removed so add to the removed list
			RemovedHotfixFileList.Add(LastHeader);

			if (LastHeader.FileName.EndsWith(TEXT(".INI"), ESearchCase::IgnoreCase))
			{
				// Make sure that ALL INIs of this "category" get marked for inclusion below
				DirtyIniCategories.Add(GetStrippedConfigFileName(LastHeader.FileName));
			}
		}
	}

	// Apply all hotfix files for each ini file if the category has been marked dirty
	// For example, if DefaultGame.ini has changed, also consider XboxOne_Game.ini changed
	// This is necessary because we revert the ini file to the pre-hotfix state
	if (DirtyIniCategories.Num() > 0)
	{
		for (const FCloudFileHeader& CurrentHeader : HotfixFileList)
		{
			if (CurrentHeader.FileName.EndsWith(TEXT(".INI"), ESearchCase::IgnoreCase))
			{
				for (const FString& StrippedIniName : DirtyIniCategories)
				{
					if (CurrentHeader.FileName.EndsWith(StrippedIniName, ESearchCase::IgnoreCase))
					{
						// Be sure to include any ini in a "dirty" category that remains in the latest HotfixFileList
						ChangedHotfixFileList.AddUnique(CurrentHeader);
					}
				}
			}
		}
	}

	UE_LOGF(LogHotfixManager, Log, "Hotfix deltas: updated:[%ls], removed:[%ls]",
		*FString::JoinBy(ChangedHotfixFileList, TEXT(", "), &FCloudFileHeader::FileName),
		*FString::JoinBy(RemovedHotfixFileList, TEXT(", "), &FCloudFileHeader::FileName));
}

void UOnlineHotfixManager::FilterHotfixFiles()
{
	for (int32 Idx = 0; Idx < HotfixFileList.Num(); Idx++)
	{
		if (!WantsHotfixProcessing(HotfixFileList[Idx]))
		{
			HotfixFileList.RemoveAt(Idx, EAllowShrinking::No);
			Idx--;
		}
	}
}

void UOnlineHotfixManager::ReadHotfixFiles()
{
	if (ChangedHotfixFileList.Num())
	{
		check(OnlineTitleFile.IsValid());
		// Kick off a read for each file
		// Do this in two passes so already cached files don't trigger completion
		for (const FCloudFileHeader& FileHeader : ChangedHotfixFileList)
		{
			UE_LOGF(LogHotfixManager, VeryVerbose, "HF: %ls %ls %d ", *FileHeader.DLName, *FileHeader.FileName, FileHeader.FileSize);
			PendingHotfixFiles.Add(FileHeader.DLName, FPendingFileDLProgress());
		}
		for (const FCloudFileHeader& FileHeader : ChangedHotfixFileList)
		{
			OnlineTitleFile->ReadFile(FileHeader.DLName);
		}
	}
	else
	{
		UE_LOGF(LogHotfixManager, Display, "No hotfix files need to be downloaded");
		FinalizeHotfixProcess(EHotfixResult::Success);
	}
}

void UOnlineHotfixManager::OnReadFileComplete(bool bWasSuccessful, const FString& FileName)
{
	if (PendingHotfixFiles.Contains(FileName))
	{
		if (bWasSuccessful)
		{
			FCloudFileHeader* Header = GetFileHeaderFromDLName(FileName);
			check(Header != nullptr);
			UE_LOGF(LogHotfixManager, Log, "Hotfix file (%ls) downloaded. Size was (%d)", *GetFriendlyNameFromDLName(FileName), Header->FileSize);
			// Completion updates the file count and progress updates the byte count
			UpdateProgress(1, 0);
			PendingHotfixFiles.Remove(FileName);
			if (PendingHotfixFiles.Num() == 0)
			{
				// Prevent async loading while applying hotfixes.
				check(!AsyncFlushContext);
				AsyncFlushContext = MakeUnique<FAsyncLoadingFlushContext>(TEXT("ApplyHotfix"));
				AsyncFlushContext->Flush(
					FOnAsyncLoadingFlushComplete::CreateWeakLambda(
						this,
						[this]()
						{
							const EHotfixResult Result = ApplyHotfix();
							FinalizeHotfixProcess(Result);
						}));
			}
		}
		else
		{
			UE_LOGF(LogHotfixManager, Error, "Hotfix file (%ls) failed to download", *GetFriendlyNameFromDLName(FileName));
			FinalizeHotfixProcess(EHotfixResult::Failed);
		}
	}
}

void UOnlineHotfixManager::UpdateProgress(uint32 FileCount, uint64 UpdateSize)
{
	NumDownloaded += FileCount;
	NumBytes += UpdateSize;
	// Update our progress
	TriggerOnHotfixProgressDelegates(NumDownloaded, TotalFiles, NumBytes, TotalBytes);
}

EHotfixResult UOnlineHotfixManager::ApplyHotfix()
{
	// Check the CVar value before unmounting any previous hotfixes in case the value was changed by the previous hotfix.
	const bool bEmptyDynamicHotfixContentsOnNewHotfix = CVarEmptyDynamicHotfixContentsOnNewHotfix->GetBool();
	if (bEmptyDynamicHotfixContentsOnNewHotfix)
	{
		UE_LOGF(LogHotfixManager, Log, "[DynamicHotfixContents] Removing ini data for all pending hotfixes");
		DynamicHotfixContents.Empty();
	}

	// Perform any undo operations needed
	// This occurs same frame as the application of new hotfixes
	RestoreBackupIniFiles();
	UnmountHotfixFiles();

	for (const FCloudFileHeader& FileHeader : RemovedHotfixFileList)
	{
		TriggerOnHotfixRemovedFileDelegates(FileHeader.FileName);

		if (!bEmptyDynamicHotfixContentsOnNewHotfix)
		{
			const FString HotfixTag = GetStrippedConfigFileName(FileHeader.FileName).Replace(TEXT(".ini"), TEXT(""));
			UE_LOGF(LogHotfixManager, Log, "[DynamicHotfixContents] Removing ini data for pending hotfix %ls (%ls)", *FileHeader.FileName, *HotfixTag);
			DynamicHotfixContents.Remove(FName(HotfixTag));
		}
	}

	for (const FCloudFileHeader& FileHeader : ChangedHotfixFileList)
	{
		if (!ApplyHotfixProcessing(FileHeader))
		{
			UE_LOGF(LogHotfixManager, Error, "Couldn't apply hotfix file (%ls)", *FileHeader.FileName);
			return EHotfixResult::Failed;
		}
		// Let anyone listening know we just processed this file
		TriggerOnHotfixProcessedFileDelegates(FileHeader.FileName, GetCachedDirectory() / FileHeader.DLName);
	}

	UE_LOGF(LogHotfixManager, Display, "Hotfix data has been successfully applied");
	EHotfixResult Result = EHotfixResult::Success;
	if (ChangedOrRemovedPakCount > 0)
	{
		UE_LOGF(LogHotfixManager, Display, "Hotfix has changed or removed PAK files so a relaunch of the app is needed");
		Result = EHotfixResult::SuccessNeedsRelaunch;
	}
	else if (bHotfixNeedsMapReload)
	{
		UE_LOGF(LogHotfixManager, Display, "Hotfix has detected PAK files containing currently loaded maps, so a level load is needed");
		Result = EHotfixResult::SuccessNeedsReload;
	}

	return Result;
}

#if !UE_BUILD_SHIPPING
void UOnlineHotfixManager::ApplyLocalTestHotfix(FString Filename)
{
	const FString CleanName = FPaths::GetCleanFilename(Filename);
	FString IniData;
	if (FFileHelper::LoadFileToString(IniData, *Filename))
	{
		if (HotfixIniFile(CleanName, IniData))
		{
			UE_LOGF(LogHotfixManager, Display, "Successfully applied test hotfix file %ls", *Filename);
		}
		else
		{
			UE_LOGF(LogHotfixManager, Display, "Failed to apply test hotfix file %ls", *Filename);
		}
	}
	else
	{
		UE_LOGF(LogHotfixManager, Warning, "Unable to read test hotfix file '%ls'", *Filename);
	}
}
#endif

void UOnlineHotfixManager::FinalizeHotfixProcess(EHotfixResult HotfixResult)
{
#if !UE_BUILD_SHIPPING
	// Apply this here so it overwrites any downloaded hotfix changes
	FString IniFilename;
	if (FParse::Value(FCommandLine::Get(), TEXT("-TestHotfixIniFile="), IniFilename))
	{
		ApplyLocalTestHotfix(IniFilename);
	}
#endif

	ConfigureHotfixOnLoad();

	if (HotfixResult != EHotfixResult::Failed)
	{
		if (IsHotfixOnLoadEnabled())
		{
			// Always parse asset hotfixes, even if we received no actual hotfix data, so that baked and command line
			// hotfixes for plugin-based assets get registered and applied via the on-load delegate when those assets load.
			ParseAssetHotfixes();
#if !UE_BUILD_SHIPPING
			ValidateHotfixableAssetClasses(AssetHotfixRegistry.GetAllPendingPaths());
#endif
			// Unconditonally apply hotfixes the first time we get a non-failure response
			// Afterwards, only apply hotfixes if the data actually changed
			if (OnlineHotfixManagerCVars::bHotfixOnLoadUnconditionallyApplyChanges || !bHasAppliedHotfixesToLoadedAssets || HotfixResult != EHotfixResult::SuccessNoChange)
			{
				ApplyHotfixesToLoadedAssets();
				bHasAppliedHotfixesToLoadedAssets = true;
			}
		}
		else if (HotfixResult != EHotfixResult::SuccessNoChange)
		{
			PatchAssetsFromIniFiles();
		}
	}

	FThreadHeartBeat::Get().InitSettings();
	FGameThreadHitchHeartBeat::Get().InitSettings();
	FGenericPlatformMisc::TickHotfixables();

	TriggerOnHotfixCompleteDelegates(HotfixResult);
	if (HotfixResult == EHotfixResult::Failed)
	{
		HotfixFileList.Empty();
		UnmountHotfixFiles();
	}
	Cleanup();
}

bool UOnlineHotfixManager::WantsHotfixProcessing(const FCloudFileHeader& FileHeader)
{
	const FString Extension = FPaths::GetExtension(FileHeader.FileName);
	if (Extension == TEXT("INI"))
	{
		FString CloudFilename;
		if (IsCompatibleHotfixFile(FileHeader.FileName, CloudFilename))
		{
			bool bIsServerHotfix = CloudFilename.StartsWith(ServerPrefix);
			bool bWantsServerHotfix = IsRunningDedicatedServer() && bIsServerHotfix;
			bool bWantsDefaultHotfix = CloudFilename.StartsWith(DefaultPrefix);
			bool bWantsPlatformHotfix = CloudFilename.StartsWith(PlatformPrefix);
			bool bWantsCustomConfigPrefix = CloudFilename.StartsWith(CustomConfigPrefix);

			if (bWantsPlatformHotfix)
			{
				UE_LOGF(LogHotfixManager, Log, "Using platform hotfix %ls", *FileHeader.FileName);
			}
			else if (bWantsServerHotfix)
			{
				UE_LOGF(LogHotfixManager, Log, "Using server hotfix %ls", *FileHeader.FileName);
			}
			else if (bWantsDefaultHotfix)
			{
				UE_LOGF(LogHotfixManager, Log, "Using default hotfix %ls", *FileHeader.FileName);
			}
			else if (bWantsCustomConfigPrefix)
			{
				UE_LOGF(LogHotfixManager, Log, "Using customconfig hotfix %ls", *FileHeader.FileName);
			}

			return bWantsPlatformHotfix || bWantsServerHotfix || bWantsDefaultHotfix || bWantsCustomConfigPrefix;
		}
		else
		{
			UE_LOGF(LogHotfixManager, Verbose, "File not compatible %ls, skipping.", *FileHeader.FileName);
			return false;
		}
	}
	else if (Extension == TEXT("PAK"))
	{
		return FileHeader.FileName.Find(PlatformPrefix) != -1;
	}
	return false;
}

bool UOnlineHotfixManager::ApplyHotfixProcessing(const FCloudFileHeader& FileHeader)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_UOnlineHotfixManager_ApplyHotfixProcessing);

	bool bSuccess = false;
	const FString Extension = FPaths::GetExtension(FileHeader.FileName);
	if (Extension == TEXT("PAK"))
	{
		bSuccess = HotfixPakFile(FileHeader);
	}
	else
	{
		TArray<uint8> FileData;
		if (OnlineTitleFile->GetFileContents(FileHeader.DLName, FileData))
		{
			if (PreProcessDownloadedFileData(FileHeader, FileData))
			{
				TriggerOnHotfixUpdatedFileDelegates(FileHeader.FileName, FileData);

				if (Extension == TEXT("INI"))
				{
					UE_LOGF(LogHotfixManager, Log, "Applying hotfix %ls", *FileHeader.FileName);

					// Convert to a FString
					FileData.Add(0);
					FString HotfixStr;
					FFileHelper::BufferToString(HotfixStr, FileData.GetData(), FileData.Num());
					bSuccess = HotfixIniFile(FileHeader.FileName, HotfixStr);
				}
			}
			else
			{
				UE_LOGF(LogHotfixManager, Warning, "Failed to process contents of %ls", *FileHeader.FileName);
			}
		}
		else
		{
			UE_LOGF(LogHotfixManager, Warning, "Failed to get contents of %ls", *FileHeader.FileName);
		}
	}
	OnlineTitleFile->ClearFile(FileHeader.FileName);
	return bSuccess;
}

FString UOnlineHotfixManager::GetStrippedConfigFileName(const FString& IniName)
{
	FString StrippedIniName;
	FString NetworkVersion;
	FString BranchVersion;
	GetFilenameAndVersion(IniName, StrippedIniName, NetworkVersion, BranchVersion);

	if (StrippedIniName.StartsWith(PlatformPrefix))
	{
		StrippedIniName = IniName.Right(StrippedIniName.Len() - PlatformPrefix.Len());
	}
	else if (StrippedIniName.StartsWith(ServerPrefix))
	{
		StrippedIniName = IniName.Right(StrippedIniName.Len() - ServerPrefix.Len());
	}
	else if (StrippedIniName.StartsWith(DefaultPrefix))
	{
		StrippedIniName = IniName.Right(StrippedIniName.Len() - DefaultPrefix.Len());
	}
	else if (StrippedIniName.StartsWith(CustomConfigPrefix))
	{
		StrippedIniName = IniName.Right(StrippedIniName.Len() - CustomConfigPrefix.Len());
	}
	else if (StrippedIniName.StartsWith(DebugPrefix))
	{
		StrippedIniName = IniName.Right(StrippedIniName.Len() - DebugPrefix.Len());
	}
	return StrippedIniName;
}

FString UOnlineHotfixManager::BuildConfigCacheKey(const FString& IniName)
{
	const FString IniNameNoExtension = FPaths::GetBaseFilename(IniName);

	return GConfig->GetConfigFilename(*IniNameNoExtension);
}

FConfigBranch* UOnlineHotfixManager::GetBranch(const FString& IniName)
{
	const FString StrippedIniName(GetStrippedConfigFileName(IniName));
	const FString StrippedIniNameNoExtension = FPaths::GetBaseFilename(StrippedIniName);

	// Find the branch by basename or full filename - simple case
	FConfigBranch* Branch = GConfig->FindBranch(*StrippedIniNameNoExtension, StrippedIniNameNoExtension);

	// If we can't find anything, then we'll look for a branch matching the suffix that has an active plugin tag matching the name.
	if (!Branch)
	{
		const auto FindActivePluginBranch = [&StrippedIniNameNoExtension, &Branch](const FString& BranchName)
		{
			if (!Branch && StrippedIniNameNoExtension.EndsWith(BranchName))
			{
				const FString ActivePluginName = StrippedIniNameNoExtension.LeftChop(BranchName.Len()); \
				Branch = GConfig->FindBranchWithDynamicLayerTag(FName(*BranchName), FName(*ActivePluginName));
			}
		};

#define FIND_BRANCH(BranchName) FindActivePluginBranch(#BranchName);
		ENUMERATE_KNOWN_INI_FILES(FIND_BRANCH);
#undef FIND_BRANCH
	}

	return Branch;
}

FConfigFile* UOnlineHotfixManager::GetConfigFile(const FString& IniName)
{
	const FString StrippedIniName(GetStrippedConfigFileName(IniName));
	const FString StrippedIniNameNoExtension = FPaths::GetBaseFilename(StrippedIniName);
	
	FConfigFile* ConfigFile = nullptr;

	// Start by searching for a known config name.
	if (GConfig->IsKnownConfigName(FName(*StrippedIniNameNoExtension, FNAME_Find)))
	{
		ConfigFile = GConfig->FindConfigFile(StrippedIniNameNoExtension);
	}

	// Fall back to a partial path search.
	if (ConfigFile == nullptr)
	{
		// Look for the first matching INI file entry
		for (const FString& IniFilename : GConfig->GetFilenames())
		{
			if (IniFilename.EndsWith(StrippedIniName) || IniFilename == StrippedIniNameNoExtension)
			{
				ConfigFile = GConfig->FindConfigFile(IniFilename);
				break;
			}
		}
	}

	// If not found, add this file to the config cache.
	if (ConfigFile == nullptr)
	{
		const FString ProcessedName(BuildConfigCacheKey(StrippedIniName));
		FConfigFile Empty;
		GConfig->SetFile(ProcessedName, &Empty);
		ConfigFile = GConfig->Find(ProcessedName);
	}
	check(ConfigFile);
	// We never want to save these merged files
	ConfigFile->NoSave = true;
	return ConfigFile;
}

bool UOnlineHotfixManager::HotfixIniFile(const FString& FileName, const FString& IniData)
{
	// Flush async loading before modifying GConfig.
	FlushAsyncLoading();
	
	static bool bUseNewDynamicLayers = CVarUseNewDynamicLayersForHotfix->GetInt() != 0;
	if (bUseNewDynamicLayers)
	{
		FName Tag = *BuildConfigCacheKey(FileName);
		UE::DynamicConfig::PerformDynamicConfig(Tag, [this, Tag, FileName, IniData](FConfigModificationTracker* ChangeTracker)
		{
			// If this file is a known type of ini type, such as DefaultEngine.ini, then we'll find a branch here such as Engine.
			// This will also work for an active GFP that has applied its config.
			FConfigBranch* Branch = GetBranch(FileName);
			if (Branch)
			{
				ChangeTracker->CVars.Add(TEXT("ConsoleVariables")).CVarPriority = (int)ECVF_SetByHotfix;
				Branch->AddDynamicLayerStringToHierarchy(FileName, IniData, Tag, DynamicLayerPriority::Hotfix, ChangeTracker);
			}

			// If not, then it's probably a GFP hotfix that will be applied whenever the plugin activates.
			else
			{
				const FString HotfixTag = GetStrippedConfigFileName(FileName).Replace(TEXT(".ini"), TEXT(""));

				UE_LOGF(LogHotfixManager, Log, "[DynamicHotfixContents] Storing ini data for pending hotfix %ls (%ls)", *FileName, *HotfixTag);

				TArray<TPair<FString, FString>>& HotfixesForTag = DynamicHotfixContents.FindOrAdd(*HotfixTag);
				HotfixesForTag.Add(TPair<FString, FString>(FileName, IniData));
			}
		});
		
		return true;
	}

	FConfigFile* ConfigFile = GetConfigFile(FileName);
	// Store the original file so we can undo this later
	FConfigFileBackup& BackupFile = BackupIniFile(FileName, ConfigFile);
	// Merge the string into the config file
	ConfigFile->CombineFromBuffer(IniData, FileName);

	ReloadObjectsAffectedByConfigFile(FileName, IniData, ConfigFile->Name.ToString(), BackupFile.ClassesReloaded, false);

	return true;
}

bool UOnlineHotfixManager::HotfixPakFile(const FCloudFileHeader& FileHeader)
{
	if (!FCoreDelegates::MountPak.IsBound())
	{
		UE_LOGF(LogHotfixManager, Error, "PAK file (%ls) could not be mounted because MountPak is not bound", *FileHeader.FileName);
		return false;
	}
	FString PakLocation = FString::Printf(TEXT("%s/%s"), *GetCachedDirectory(), *FileHeader.DLName);
	if (IPakFile* PakFile = FCoreDelegates::MountPak.Execute(PakLocation, 0))
	{
		MountedPakFiles.Add(FileHeader.DLName);
		UE_LOGF(LogHotfixManager, Log, "Hotfix mounted PAK file (%ls)", *FileHeader.FileName);
		int32 NumInisReloaded = 0;
		const double StartTime = FPlatformTime::Seconds();

		// Iterate through the the pak file's contents for INI and asset reloading.
		TArray<FString> IniList;
		FPakFileVisitor Visitor;
		PakFile->PakVisitPrunedFilenames(Visitor);
		for (const FString& InternalPakFileName : Visitor.Files)
		{
			if (InternalPakFileName.EndsWith(TEXT(".ini")))
			{
				IniList.Add(InternalPakFileName);
			}
		}

		// Iterate through all loaded maps and see if they have patches in the pak file and therefore this hotfix needs to reload a map
		for (TObjectIterator<UPackage> it; !bHotfixNeedsMapReload && it; ++it)
		{
			UPackage* Package = *it;
			if (Package && Package->ContainsMap())
			{
				const FString FileName = FPackageName::LongPackageNameToFilename(Package->GetLoadedPath().GetPackageName(), FPackageName::GetMapPackageExtension());
				if (PakFile->PakContains(FileName))
				{
					bHotfixNeedsMapReload = true;
				}
			}
		}

		// Sort the INIs so they are processed consistently
		IniList.Sort<FHotfixFileSortPredicate>(FHotfixFileSortPredicate(PlatformPrefix, ServerPrefix, DefaultPrefix, CustomConfigPrefix));
		// Now process the INIs in sorted order
		for (const FString& IniName : IniList)
		{
			HotfixPakIniFile(IniName);
			NumInisReloaded++;
		}
		UE_LOGF(LogHotfixManager, Log, "Processing pak file (%ls) took %f seconds and resulted in (%d) INIs being reloaded",
			*FileHeader.FileName, FPlatformTime::Seconds() - StartTime, NumInisReloaded);
#if !UE_BUILD_SHIPPING
		if (bLogMountedPakContents)
		{
			UE_LOGF(LogHotfixManager, Log, "Files in pak file (%ls):", *FileHeader.FileName);
			for (const FString& FileName : Visitor.Files)
			{
				UE_LOGF(LogHotfixManager, Log, "\t\t%ls", *FileName);
			}
		}
#endif
		return true;
	}
	return false;
}

bool UOnlineHotfixManager::IsMapLoaded(const FString& MapName)
{
	FString MapPackageName(MapName.Left(MapName.Len() - 5));
	MapPackageName = MapPackageName.Replace(*GameContentPath, TEXT("/Game"));
	// If this map's UPackage exists, it is currently loaded
	UPackage* MapPackage = FindObject<UPackage>(nullptr, *MapPackageName, EFindObjectFlags::ExactClass);
	return MapPackage != nullptr;
}

bool UOnlineHotfixManager::HotfixPakIniFile(const FString& FileName)
{
	// Flush async loading before modifying GConfig.
	FlushAsyncLoading();

	FString StrippedName;
	const double StartTime = FPlatformTime::Seconds();
	// Need to strip off the PAK path
	FileName.Split(TEXT("/"), nullptr, &StrippedName, ESearchCase::IgnoreCase, ESearchDir::FromEnd);
	FConfigFile* ConfigFile = GetConfigFile(StrippedName);
	if (!ConfigFile->Combine(FString(TEXT("../../../")) + FileName.Replace(*GameContentPath, TEXT("/Game"))))
	{
		UE_LOGF(LogHotfixManager, Log, "Hotfix failed to merge INI (%ls) found in a PAK file", *FileName);
		return false;
	}
	UE_LOGF(LogHotfixManager, Log, "Hotfix merged INI (%ls) found in a PAK file", *FileName);
	int32 NumObjectsReloaded = 0;
	// Now that we have a list of classes to update, we can iterate objects and
	// reload if they match the INI file that was changed
	TArray<UObject*> Classes;
	GetObjectsOfClass(UClass::StaticClass(), Classes, true, RF_NoFlags);
	TArray<UClass*> ClassesToReload;
	for (UObject* ClassObject : Classes)
	{
		if (UClass* const Class = Cast<UClass>(ClassObject))
		{
			if (Class->HasAnyClassFlags(CLASS_Config) &&
				Class->ClassConfigName == ConfigFile->Name)
			{
				TArray<UObject*> Objects;
				GetObjectsOfClass(Class, Objects, true, RF_NoFlags);
				for (UObject* Object : Objects)
				{
					if (IsValid(Object))
					{
						// Force a reload of the config vars
						Object->ReloadConfig();
						NumObjectsReloaded++;
					}
				}
			}
		}
	}
	UE_LOGF(LogHotfixManager, Log, "Updating config from %ls took %f seconds reloading %d objects",
		*FileName, FPlatformTime::Seconds() - StartTime, NumObjectsReloaded);
	return true;
}

const FString UOnlineHotfixManager::GetFriendlyNameFromDLName(const FString& DLName) const
{
	for (const FCloudFileHeader& Header : HotfixFileList)
	{
		if (Header.DLName == DLName)
		{
			return Header.FileName;
		}
	}
	return FString();
}

void UOnlineHotfixManager::UnmountHotfixFiles()
{
	if (MountedPakFiles.Num() == 0)
	{
		return;
	}
	// Unmount any changed hotfix files since we need to download them again
	for (const FCloudFileHeader& FileHeader : ChangedHotfixFileList)
	{
		for (int32 Index = 0; Index < MountedPakFiles.Num(); Index++)
		{
			if (MountedPakFiles[Index] == FileHeader.DLName)
			{
				FCoreDelegates::OnUnmountPak.Execute(MountedPakFiles[Index]);
				MountedPakFiles.RemoveAt(Index);
				ChangedOrRemovedPakCount++;
				UE_LOGF(LogHotfixManager, Log, "Hotfix unmounted PAK file (%ls) so it can be redownloaded", *FileHeader.FileName);
				break;
			}
		}
	}
	// Unmount any removed hotfix files
	for (const FCloudFileHeader& FileHeader : RemovedHotfixFileList)
	{
		for (int32 Index = 0; Index < MountedPakFiles.Num(); Index++)
		{
			if (MountedPakFiles[Index] == FileHeader.DLName)
			{
				FCoreDelegates::OnUnmountPak.Execute(MountedPakFiles[Index]);
				MountedPakFiles.RemoveAt(Index);
				ChangedOrRemovedPakCount++;
				UE_LOGF(LogHotfixManager, Log, "Hotfix unmounted PAK file (%ls) since it was removed from the hotfix set", *FileHeader.FileName);
				break;
			}
		}
	}
}

FCloudFileHeader* UOnlineHotfixManager::GetFileHeaderFromDLName(const FString& FileName)
{
	for (int32 Index = 0; Index < HotfixFileList.Num(); Index++)
	{
		if (HotfixFileList[Index].DLName == FileName)
		{
			return &HotfixFileList[Index];
		}
	}
	return nullptr;
}

void UOnlineHotfixManager::OnReadFileProgress(const FString& FileName, uint64 BytesRead)
{
	if (PendingHotfixFiles.Contains(FileName))
	{
		// Since the title file is reporting absolute numbers subtract out the last update so we can add a delta
		uint64 Delta = BytesRead - PendingHotfixFiles[FileName].Progress;
		PendingHotfixFiles[FileName].Progress = BytesRead;
		// Completion updates the file count and progress updates the byte count
		UpdateProgress(0, Delta);
	}
}

UOnlineHotfixManager::FConfigFileBackup& UOnlineHotfixManager::BackupIniFile(const FString& IniName, const FConfigFile* ConfigFile)
{
	FString BackupIniName = BuildConfigCacheKey(GetStrippedConfigFileName(IniName));
	if (FConfigFileBackup* Backup = IniBackups.FindByPredicate([&BackupIniName](const FConfigFileBackup& Entry) { return Entry.IniName == BackupIniName; }))
	{
		// Only store one copy of each ini file, consisting of the original state
		return *Backup;
	}

	int32 AddAt = IniBackups.AddDefaulted();
	FConfigFileBackup& NewBackup = IniBackups[AddAt];
	NewBackup.IniName = BackupIniName;
	NewBackup.ConfigData = *ConfigFile;
	return NewBackup;
}

void UOnlineHotfixManager::RestoreBackupIniFiles()
{
	static bool bUseNewDynamicLayers = CVarUseNewDynamicLayersForHotfix->GetInt() != 0;
	if (bUseNewDynamicLayers)
	{
		// @todo branch - would be nice to have a way to know nothing was backed up yet, with Branch mode, so we can skip the FlushAsyncLoading call when there's nothing to do
		// Flush async loading before modifying GConfig.
		FlushAsyncLoading();

		// when just unloading, we don't need an actual tag
		UE::DynamicConfig::PerformDynamicConfig(NAME_None, [this](FConfigModificationTracker* ChangeTracker)
		{
			for (const FCloudFileHeader& FileHeader : ChangedHotfixFileList)
			{
				if (FileHeader.FileName.EndsWith(TEXT(".INI")))
				{
					FName Tag = *BuildConfigCacheKey(FileHeader.FileName);
					FConfigCacheIni::RemoveTagFromAllBranches(Tag, ChangeTracker);
				}
			}
			for (const FCloudFileHeader& FileHeader : RemovedHotfixFileList)
			{
				if (FileHeader.FileName.EndsWith(TEXT(".INI")))
				{
					FName Tag = *BuildConfigCacheKey(FileHeader.FileName);
					FConfigCacheIni::RemoveTagFromAllBranches(Tag, ChangeTracker);
				}
			}
		});
		
		return;
	}

	if (IniBackups.Num() == 0)
	{
		return;
	}

	// Flush async loading before modifying GConfig.
	FlushAsyncLoading();

	const double StartTime = FPlatformTime::Seconds();
	TArray<FString> ClassesToRestore;

	// Restore any changed INI files and build a list of which ones changed for UObject reloading below
	for (const FCloudFileHeader& FileHeader : ChangedHotfixFileList)
	{
		if (FileHeader.FileName.EndsWith(TEXT(".INI")))
		{
			const FString ProcessedName = BuildConfigCacheKey(GetStrippedConfigFileName(FileHeader.FileName));
			for (int32 Index = 0; Index < IniBackups.Num(); Index++)
			{
				const FConfigFileBackup& BackupFile = IniBackups[Index];
				if (IniBackups[Index].IniName == ProcessedName)
				{
					ClassesToRestore.Append(BackupFile.ClassesReloaded);

					GConfig->SetFile(BackupFile.IniName, &BackupFile.ConfigData);
					IniBackups.RemoveAt(Index);
					break;
				}
			}
		}
	}

	// Also restore any files that were previously part of the hotfix and now are not
	for (const FCloudFileHeader& FileHeader : RemovedHotfixFileList)
	{
		if (FileHeader.FileName.EndsWith(TEXT(".INI")))
		{
			const FString ProcessedName = BuildConfigCacheKey(GetStrippedConfigFileName(FileHeader.FileName));
			for (int32 Index = 0; Index < IniBackups.Num(); Index++)
			{
				const FConfigFileBackup& BackupFile = IniBackups[Index];
				if (BackupFile.IniName == ProcessedName)
				{
					ClassesToRestore.Append(BackupFile.ClassesReloaded);

					GConfig->SetFile(BackupFile.IniName, &BackupFile.ConfigData);
					IniBackups.RemoveAt(Index);
					break;
				}
			}
		}
	}

	uint32 NumObjectsReloaded = 0;
	if (ClassesToRestore.Num() > 0)
	{
		TArray<UClass*> RestoredClasses;
		RestoredClasses.Reserve(ClassesToRestore.Num());
		for (int32 Index = 0; Index < ClassesToRestore.Num(); Index++)
		{
			UClass* Class = FindObject<UClass>(nullptr, *ClassesToRestore[Index], EFindObjectFlags::ExactClass);
			if (Class != nullptr)
			{
				// Add this to the list to check against
				RestoredClasses.Add(Class);
			}
		}

		for (UClass* Class : RestoredClasses)
		{
			if (Class->HasAnyClassFlags(CLASS_Config))
			{
				TArray<UObject*> Objects;
				GetObjectsOfClass(Class, Objects, true, RF_NoFlags);
				for (UObject* Object : Objects)
				{
					if (IsValid(Object))
					{
						UE_LOGF(LogHotfixManager, Verbose, "Restoring %ls", *Object->GetPathName());
						Object->ReloadConfig();
						NumObjectsReloaded++;
					}
				}
			}
		}
	}
	UE_LOGF(LogHotfixManager, Log, "Restoring config for %d changed classes took %f seconds reloading %d objects",
		ClassesToRestore.Num(), FPlatformTime::Seconds() - StartTime, NumObjectsReloaded);
}

void UOnlineHotfixManager::PatchAssetsFromIniFiles()
{
	UE_LOGF(LogHotfixManager, Display, "Checking for assets to be patched using data from 'AssetHotfix' section in the Game .ini file");

	int32 TotalPatchableAssets = 0;
	AssetsHotfixedFromIniFiles.Reset();

	// Everything should be under the 'AssetHotfix' section in Game.ini
	const FConfigSection* AssetHotfixConfigSection = GConfig->GetSection(TEXT("AssetHotfix"), false, GGameIni);
	if (AssetHotfixConfigSection != nullptr)
	{
		// These are the asset types we support patching right now
		UClass* const PatchableAssetClasses[] = 
		{ 
			UCurveTable::StaticClass(), 
			UDataTable::StaticClass(), 
			UCurveFloat::StaticClass(),
            UCurveVector::StaticClass(),
			UCurveLinearColor::StaticClass(),
		};

		TSet<UDataTable*> ChangedDataTables;
		TSet<UCurveTable*> ChangedCurveTables;
		TSet<UCurveTable*>* ChangedCurveTablesPointer = OnlineHotfixManagerCVars::bDeferBroadcastCurveTableModified ? &ChangedCurveTables : nullptr;

		for (FConfigSection::TConstIterator It(*AssetHotfixConfigSection); It; ++It)
		{
			FMoviePlayerProxy::BlockingTick();
			++TotalPatchableAssets;

			// Make sure the entry has a valid class name that we support
			UClass* AssetClass = nullptr;
			FString PatchableAssetClassesStr;
			for (UClass* PatchableAssetClass : PatchableAssetClasses)
			{
				if (PatchableAssetClass)
				{
					PatchableAssetClassesStr += PatchableAssetClass->GetFName().ToString() + TEXT(" ");

					if (PatchableAssetClass->GetFName().IsEqual(It.Key()))
					{
						AssetClass = PatchableAssetClass;
					}
				}
			}

			if (AssetClass != nullptr)
			{
				TArray<FString> ProblemStrings;

				FString DataLine(*It.Value().GetValue());

				if (!DataLine.IsEmpty())
				{
					TArray<FString> Tokens;
					DataLine.ParseIntoArray(Tokens, TEXT(";"));
					if (Tokens.Num() == 3 || Tokens.Num() == 5)
					{
						const FString& AssetPath(Tokens[0]);
						const FString& HotfixType(Tokens[1]);

						if (!ShouldHotfixAsset(AssetPath))
						{
							//Child class says not to hotfix this asset so it shouldn't be included in the total
							--TotalPatchableAssets;
							continue;
						}

						bool bAddAssetToHotfixedList = false;

						// Find or load the asset
						UObject* Asset = FPackageName::IsValidLongPackageName(AssetPath, true) ? StaticLoadObject(AssetClass, nullptr, *AssetPath) : nullptr;
						if (Asset != nullptr)
						{
							const FString RowUpdate(TEXT("RowUpdate"));
							const FString AddRow(TEXT("AddRow"));
							const FString TableUpdate(TEXT("TableUpdate"));
							const FString CurveUpdate(TEXT("CurveUpdate"));

							if (HotfixType == RowUpdate)
							{
								if (Tokens.Num() == 5)
								{
									// The hotfix line should be
									//	+DataTable=<data table path>;RowUpdate;<row name>;<column name>;<new value>
									//	+CurveTable=<curve table path>;RowUpdate;<row name>;<column name>;<new value>
									//	+CurveFloat=<curve float path>;RowUpdate;None;<column name>;<new value>
									HotfixRowUpdate(Asset, AssetPath, Tokens[2], Tokens[3], Tokens[4], ProblemStrings, &ChangedDataTables, ChangedCurveTablesPointer);
									bAddAssetToHotfixedList = ProblemStrings.Num() == 0;
								}
								else
								{
									ProblemStrings.Add(FString::Printf(TEXT("Expected a hotfix type RowUpdate with 5 tokens but got %d"), Tokens.Num()));
								}
							}
							else if (HotfixType == AddRow)
							{
								if (Tokens.Num() == 3)
								{
									// The hotfix line should be
									//	+DataTable=<data table path>;AddRow;"<json data>"

									// We have to read json data as quoted string because tokenizing it creates extra unwanted characters.
									FString JsonData;

									// Json should be read in its entirety, if the whole buffer wasn't read the string is malformed. 
									int32 ReadLen = 0;
									int32 InputLen = Tokens[2].Len();
									if (FParse::QuotedString(*Tokens[2], JsonData, &ReadLen) && ReadLen == InputLen)
									{
										HotfixAddRow(Asset, AssetPath, JsonData, ProblemStrings);
										bAddAssetToHotfixedList = ProblemStrings.Num() == 0;
									}
									else
									{
										ProblemStrings.Add(TEXT("Json data wasn't able to be parsed as a quoted string. Check that we have opening and closing quotes around the json data."));
									}
								}
								else
								{
									ProblemStrings.Add(FString::Printf(TEXT("Expected a hotfix type AddRow with 3 tokens but got %d"), Tokens.Num()));
								}
							}
							else if (HotfixType == TableUpdate || HotfixType == CurveUpdate)
							{
								if (Tokens.Num() == 3)
								{
									// The hotfix line should be
									//	+DataTable=<data table path>;TableUpdate;"<json data>"
									//	+CurveTable=<curve table path>;TableUpdate;"<json data>"
									//	+CurveFloat=<curve float path>;CurveUpdate;"<json data>"
									//	+CurveVector=<curve vector path>;CurveUpdate;"<json data>"
									//	+CurveLinearColor=<curve linear color path>;CurveUpdate;"<json data>"

									// We have to read json data as quoted string because tokenizing it creates extra unwanted characters.
									FString JsonData;

									// Json should be read in its entirety, if the whole buffer wasn't read the string is malformed. 
									int32 ReadLen = 0;
									int32 InputLen = Tokens[2].Len();
									if (FParse::QuotedString(*Tokens[2], JsonData, &ReadLen) && ReadLen == InputLen)
									{
										HotfixTableUpdate(Asset, AssetPath, JsonData, ProblemStrings);
										bAddAssetToHotfixedList = ProblemStrings.Num() == 0;
									}
									else
									{
										ProblemStrings.Add(TEXT("Json data wasn't able to be parsed as a quoted string. Check that we have opening and closing quotes around the json data."));
									}
								}
								else
								{
									ProblemStrings.Add(FString::Printf(TEXT("Expected a hotfix type %s with 3 tokens but got %d"), *HotfixType, Tokens.Num()));
								}
							}
							else
							{
								ProblemStrings.Add(FString::Printf(TEXT("Unknown hotfix type %s"), *HotfixType));
							}
						}
						else
						{
							if (ShouldWarnAboutMissingWhenPatchingFromIni(AssetPath))
							{
								const FString Problem(FString::Printf(TEXT("Couldn't find or load asset '%s' (class '%s').  This asset will not be patched.  Double check that your asset type and path string is correct."), *AssetPath, *AssetClass->GetPathName()));
								ProblemStrings.Add(Problem);
							}
						}

						if (!bAddAssetToHotfixedList)
						{
							for (const FString& ProblemString : ProblemStrings)
							{
								UE_LOGF(LogHotfixManager, Error, "[Item: %d] %ls: %ls", TotalPatchableAssets, *GetPathNameSafe(Asset), *ProblemString);
							}
						}
						else
						{
							// We'll keep a reference to the successfully patched asset.  We want to make sure our changes survive throughout
							// this session, so we reference it to prevent it from being evicted from memory.  It's OK if we end up re-patching
							// the same asset multiple times per session.
							AssetsHotfixedFromIniFiles.Add(Asset);
						}
					}
					else
					{
						UE_LOGF(LogHotfixManager, Error, "[Item: %d] Wasn't able to parse the data with semicolon separated values. Expecting 3 or 5 arguments but parsed %d.", TotalPatchableAssets, Tokens.Num());
					}
				}
				else
				{
					UE_LOGF(LogHotfixManager, Warning, "[Item: %d] Empty value given for '%ls' entry!", TotalPatchableAssets, *It.Key().ToString());
				}
			}
			else
			{
				UE_LOGF(LogHotfixManager, Error, "[Item: %d] Invalid patchable asset type '%ls' - supported types: %ls", TotalPatchableAssets, *It.Key().ToString(), *PatchableAssetClassesStr);
			}
		}

		BroadcastTableChanges(ChangedDataTables, ChangedCurveTables);
	}

	if (TotalPatchableAssets == 0)
	{
		UE_LOGF(LogHotfixManager, Display, "No assets were found in the 'AssetHotfix' section in the Game .ini file. No patching needed.");
	}
	else if (TotalPatchableAssets == AssetsHotfixedFromIniFiles.Num())
	{
		UE_LOGF(LogHotfixManager, Display, "Successfully patched all %i assets from the 'AssetHotfix' section in the Game .ini file. These assets will be forced to remain loaded.", AssetsHotfixedFromIniFiles.Num());
	}
	else
	{
		// Some assets will fail to patch due to their plugins not being installed or mounted when PatchAssetsFromIniFiles is called.
		// This behavior is a limitation of having a hotfix for the overall game and will require plugin specific hotfixes to address.
		// PatchAssetsFromIniFiles must be called again when plugins change to allow patching assets when their plugin is mounted.
		UE_LOGF(LogHotfixManager, Warning, "Only %i of %i assets were successfully patched from 'AssetHotfix' section in the Game .ini file. The patched assets will be forced to remain loaded. Any assets that failed to patch may be left in an invalid state! Failure may occur due to the plugin not being installed and mounted.", AssetsHotfixedFromIniFiles.Num(), TotalPatchableAssets);
	}
}

void UOnlineHotfixManager::RegisterAssetHotfixEntry(FName AssetPath, FPendingAssetHotfix Hotfix)
{
	AssetHotfixRegistry.StoreHotfixEntry(AssetPath, MoveTemp(Hotfix));
}

void UOnlineHotfixManager::ApplyRegisteredHotfixesForAsset(UObject* Asset)
{
	TSet<UDataTable*> ChangedDataTables;
	TSet<UCurveTable*> ChangedCurveTables;
	bool bEncounteredErrors = false;
	if (AssetHotfixRegistry.TryApplyHotfixes(Asset, this, bEncounteredErrors, &ChangedDataTables, &ChangedCurveTables))
	{
		BroadcastTableChanges(ChangedDataTables, ChangedCurveTables);
	}
}

void UOnlineHotfixManager::ParseAssetHotfixes()
{
	// Clear base-ini entries (tagged NAME_None) from a previous hotfix cycle before
	// re-parsing. GFP-sourced entries (named tags) are managed separately via
	// RemoveHotfixesFromDynamicBranch and are intentionally left intact here.
	AssetHotfixRegistry.RemoveEntriesFromSource(NAME_None);

	// Everything should be under the 'AssetHotfix' section in Game.ini
	const FConfigSection* AssetHotfixConfigSection = GConfig->GetSection(TEXT("AssetHotfix"), false, GGameIni);
	if (AssetHotfixConfigSection != nullptr)
	{
		AssetHotfixRegistry.ParseAndStoreHotfixEntries(NAME_None, *AssetHotfixConfigSection);
	}
}

void UOnlineHotfixManager::ApplyHotfixesToLoadedAssets()
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_UOnlineHotfixManager_ApplyHotfixesToLoadedAssets);
	TArray<FName> AllPaths = AssetHotfixRegistry.GetAllPendingPaths();
	ApplyHotfixesToLoadedAssets(AllPaths);
}

void UOnlineHotfixManager::ApplyHotfixesToLoadedAssets(TConstArrayView<FName> AssetPaths)
{
	TSet<UDataTable*> ChangedDataTables;
	TSet<UCurveTable*> ChangedCurveTables;

	FString PathString;
	int32 NumHotfixesApplied = 0;
	int32 NumEncounteredError = 0;
	for (FName Path : AssetPaths)
	{
		Path.ToString(PathString);
		UObject* AssetInMemory = FindObject<UObject>(nullptr, PathString);

		// If we found a package then get the primary asset in the package
		// The [AssetHotfix] section uses package paths and not object paths
		if (AssetInMemory && AssetInMemory->IsA<UPackage>())
		{
			FString AssetName = FPackageName::GetShortName(PathString);
			AssetInMemory = FindObject<UObject>(AssetInMemory, *AssetName);
		}

		if (AssetInMemory)
		{
			bool bEncounteredErrors = false;
			if (AssetHotfixRegistry.TryApplyHotfixes(AssetInMemory, this, bEncounteredErrors, &ChangedDataTables, &ChangedCurveTables))
			{
				NumHotfixesApplied++;
				UE_LOGF(LogHotfixManager, Log, "Hotfixed '%ls'", *AssetInMemory->GetFullName());
			}

			if (bEncounteredErrors)
			{
				NumEncounteredError++;
			}
		}
	}

	if (AssetPaths.IsEmpty())
	{
		UE_LOGF(LogHotfixManager, Display, "No assets were found in the 'AssetHotfix' section in the Game .ini file. No patching needed.");
	}
	else if (NumHotfixesApplied == AssetPaths.Num())
	{
		if (NumEncounteredError > 0)
		{
			UE_LOGF(LogHotfixManager, Warning, "Patched all %i of %i assets from 'AssetHotfix' section in the Game .ini file but encountered %i partial errors applying patches to asset. Possible culprits include bad syntax in plugin Game .ini files. Inspect log for patch errors for more details.", NumHotfixesApplied, AssetPaths.Num(), NumEncounteredError);
		}
		else
		{
			UE_LOGF(LogHotfixManager, Display, "Successfully patched all %i assets from the 'AssetHotfix' section in the Game .ini file.", AssetPaths.Num());
		}
	}
	else
	{
		UE_LOGF(LogHotfixManager, Warning, "Only %i of %i assets were successfully patched from 'AssetHotfix' section in the Game .ini file. Inspect log for patch errors for more details.", NumHotfixesApplied, AssetPaths.Num());
	}

	BroadcastTableChanges(ChangedDataTables, ChangedCurveTables);
}

#if !UE_BUILD_SHIPPING
void UOnlineHotfixManager::ValidateHotfixableAssetClasses(TConstArrayView<FName> AssetPaths)
{
	TMap<FName, FAssetData> PackageToAsset;
	UE::AssetRegistry::GetAssetForPackages(AssetPaths, PackageToAsset);

	for (FName AssetPath : AssetPaths)
	{
		const FAssetData* AssetData = PackageToAsset.Find(AssetPath);
		if (!AssetData)
		{
			UE_LOGF(LogHotfixManager, Warning,
				"Hotfix entry targets asset '%ls' which was not found in the asset registry.",
				*AssetPath.ToString());
			continue;
		}

		UClass* AssetClass = AssetData->GetClass();
		if (AssetClass && !FCoreUObjectDelegates::IsHotfixableAssetClass(AssetClass))
		{
			UE_LOGF(LogHotfixManager, Warning,
				"Hotfix entry targets '%ls' (class '%ls') which is not registered as hotfixable. "
				"Add it to HotfixableAssetClasses in [/Script/Hotfix.OnlineHotfixManager].",
				*AssetPath.ToString(), *AssetClass->GetName());
		}
	}
}
#endif

void UOnlineHotfixManager::ReloadConfigsFromIniFiles()
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_UOnlineHotfixManager_ReloadConfigsFromIniFiles);

	if (HotfixFileList.IsEmpty())
	{
		return;
	}

	TArray<FString> ClassesToReload;

	for (const FCloudFileHeader& FileHeader : HotfixFileList)
	{
		if (FileHeader.FileName.EndsWith(TEXT(".INI")))
		{
			const FString ProcessedName = BuildConfigCacheKey(GetStrippedConfigFileName(FileHeader.FileName));
			for (int32 Index = 0; Index < IniBackups.Num(); Index++)
			{
				const FConfigFileBackup& BackupFile = IniBackups[Index];
				if (IniBackups[Index].IniName == ProcessedName)
				{
					ClassesToReload.Append(BackupFile.ClassesReloaded);
				}
			}
		}
	}

	int32 NumObjectsReloaded = 0;
	const double StartTime = FPlatformTime::Seconds();

	if (ClassesToReload.Num() > 0)
	{
		FlushAsyncLoading();
		
		TArray<UClass*> RestoredClasses;
		RestoredClasses.Reserve(ClassesToReload.Num());
		for (int32 Index = 0; Index < ClassesToReload.Num(); Index++)
		{
			UClass* Class = FindObject<UClass>(nullptr, *ClassesToReload[Index], EFindObjectFlags::ExactClass);
			if (Class != nullptr)
			{
				// Add this to the list to check against
				RestoredClasses.Add(Class);
			}
		}

		for (UClass* Class : RestoredClasses)
		{
			if (Class->HasAnyClassFlags(CLASS_Config))
			{
				TArray<UObject*> Objects;
				GetObjectsOfClass(Class, Objects, true, RF_NoFlags);
				for (UObject* Object : Objects)
				{
					if (IsValid(Object))
					{
						UE_LOGF(LogHotfixManager, Verbose, "Reloading %ls", *Object->GetPathName());
						Object->ReloadConfig();
						NumObjectsReloaded++;
					}
				}
			}
		}
	}

	UE_LOGF(LogHotfixManager, Log, "Reloading config for %d changed classes took %f seconds reloading %d objects",
		ClassesToReload.Num(), FPlatformTime::Seconds() - StartTime, NumObjectsReloaded);
}

void UOnlineHotfixManager::HotfixRowUpdate(
	UObject* Asset,
	const FString& AssetPath,
	const FString& RowName,
	const FString& ColumnName,
	const FString& NewValue,
	TArray<FString>& ProblemStrings,
	TSet<UDataTable*>* ChangedDataTables,
	TSet<UCurveTable*>* ChangedCurveTables,
	FName SourceTag)
{
	if (AssetPath.IsEmpty())
	{
		ProblemStrings.Add(TEXT("The table's path is empty. We cannot continue the hotfix."));
		return;
	}
	if (RowName.IsEmpty())
	{
		ProblemStrings.Add(TEXT("The row name is empty. We cannot continue the hotfix."));
		return;
	}
	if (ColumnName.IsEmpty())
	{
		ProblemStrings.Add(TEXT("The column name is empty. We cannot continue the hotfix."));
		return;
	}
	if (NewValue.IsEmpty())
	{
		ProblemStrings.Add(TEXT("The new value is empty. We cannot continue the hotfix."));
		return;
	}

	UDataTable* DataTable = Cast<UDataTable>(Asset);
	UCurveTable* CurveTable = Cast<UCurveTable>(Asset);
	UCurveFloat* CurveFloat = Cast<UCurveFloat>(Asset);
	if (DataTable != nullptr)
	{
		// Edit the row with the new value.
		bool bWasDataTableChanged = false;

		FProperty* DataTableRowProperty = PropertyAccessUtil::FindPropertyByName(FName(*ColumnName), DataTable->GetRowStruct());
		if (DataTableRowProperty)
		{
			// See what type of property this is.
			FNumericProperty* NumProp = CastField<FNumericProperty>(DataTableRowProperty);
			FStrProperty* StrProp = CastField<FStrProperty>(DataTableRowProperty);
			FNameProperty* NameProp = CastField<FNameProperty>(DataTableRowProperty);
			FSoftObjectProperty* SoftObjProp = CastField<FSoftObjectProperty>(DataTableRowProperty);

			// Get the row data by name.
			static const FString Context = FString(TEXT("UOnlineHotfixManager::PatchAssetsFromIniFiles"));
			uint8* DataTableRow = DataTable->FindRowUnchecked(FName(*RowName));
			if (DataTableRow)
			{
				uint8* RowData = DataTableRowProperty->ContainerPtrToValuePtr<uint8>(DataTableRow, 0);
				if (RowData)
				{
					// Numeric property
					if (NumProp)
					{
						if (NewValue.IsNumeric())
						{
							// Integer
							if (NumProp->IsInteger())
							{
								const int64 OldPropertyValue = NumProp->GetSignedIntPropertyValue(RowData);
								const int64 NewPropertyValue = FCString::Atoi(*NewValue);
								NumProp->SetIntPropertyValue(RowData, NewPropertyValue);
								OnHotfixTableValueInt64WithSource(*Asset, RowName, ColumnName, OldPropertyValue, NewPropertyValue, SourceTag);
								bWasDataTableChanged = true;
								UE_LOG(LogHotfixManager, VeryVerbose, TEXT("Data table %s row %s updated column %s from %" INT64_FMT " to %" INT64_FMT "."), *AssetPath, *RowName, *ColumnName, OldPropertyValue, NewPropertyValue);
							}
							// Float
							else
							{
								const double OldPropertyValue = NumProp->GetFloatingPointPropertyValue(RowData);
								const double NewPropertyValue = FCString::Atod(*NewValue);
								NumProp->SetFloatingPointPropertyValue(RowData, NewPropertyValue);
								OnHotfixTableValueDoubleWithSource(*Asset, RowName, ColumnName, OldPropertyValue, NewPropertyValue, SourceTag);
								bWasDataTableChanged = true;
								UE_LOGF(LogHotfixManager, VeryVerbose, "Data table %ls row %ls updated column %ls from %.2f to %.2f.", *AssetPath, *RowName, *ColumnName, OldPropertyValue, NewPropertyValue);
							}
						}
						// Not a number.
						else
						{
							const FString Problem(FString::Printf(TEXT("The new value %s is not a number when it should be."), *NewValue));
							ProblemStrings.Add(Problem);
						}
					}
					// String property
					else if (StrProp)
					{
						const FString OldPropertyValue = StrProp->GetPropertyValue(RowData);
						const FString NewPropertyValue = NewValue;
						StrProp->SetPropertyValue(RowData, NewPropertyValue);
						OnHotfixTableValueStringWithSource(*Asset, RowName, ColumnName, OldPropertyValue, NewPropertyValue, SourceTag);
						bWasDataTableChanged = true;
						UE_LOGF(LogHotfixManager, VeryVerbose, "Data table %ls row %ls updated column %ls from %ls to %ls.", *AssetPath, *RowName, *ColumnName, *OldPropertyValue, *NewPropertyValue);
					}
					// FName property
					else if (NameProp)
					{
						const FName OldPropertyValue = NameProp->GetPropertyValue(RowData);
						const FName NewPropertyValue = FName(*NewValue);
						NameProp->SetPropertyValue(RowData, NewPropertyValue);
						OnHotfixTableValueNameWithSource(*Asset, RowName, ColumnName, OldPropertyValue, NewPropertyValue, SourceTag);
						bWasDataTableChanged = true;
						UE_LOGF(LogHotfixManager, VeryVerbose, "Data table %ls row %ls updated column %ls from %ls to %ls.", *AssetPath, *RowName, *ColumnName, *OldPropertyValue.ToString(), *NewPropertyValue.ToString());
					}
					// Soft Object property
					else if (SoftObjProp)
					{
						FSoftObjectPtr OldPropertyValue = SoftObjProp->GetPropertyValue(RowData);
						FSoftObjectPtr NewPropertyValue(NewValue);
						SoftObjProp->SetPropertyValue(RowData, NewPropertyValue);
						OnHotfixTableValueSoftObjectWithSource(*Asset, RowName, ColumnName, OldPropertyValue, NewPropertyValue, SourceTag);
						bWasDataTableChanged = true;
						UE_LOGF(LogHotfixManager, VeryVerbose, "Data table %ls row %ls updated column %ls from %ls to %ls.", *AssetPath, *RowName, *ColumnName, *OldPropertyValue.ToString(), *NewPropertyValue.ToString());
					}
					else
					{
						// Evaluate supported property types, i.e. FBoolProperty, and attempt to assign the value
						const FString OldPropertyValue = DataTableUtils::GetPropertyValueAsString(DataTableRowProperty, (uint8*)DataTableRow, EDataTableExportFlags::UseSimpleText);
						FString Error = DataTableUtils::AssignStringToProperty(NewValue, DataTableRowProperty, (uint8*)DataTableRow);

						if (Error.IsEmpty())
						{
							bWasDataTableChanged = true;
							UE_LOGF(LogHotfixManager, VeryVerbose, "Data table %ls row %ls updated column %ls from %ls to %ls.", *AssetPath, *RowName, *ColumnName, *OldPropertyValue, *NewValue);
						}
						else
						{
							FString Problem(FString::Printf(TEXT("Failed to update data table %s row %s column %s from %s to %s."), *AssetPath, *RowName, *ColumnName, *OldPropertyValue, *NewValue));
							ProblemStrings.Add(MoveTemp(Problem));
							ProblemStrings.Add(MoveTemp(Error));
						}
					}
				}
				// Row data wasn't found.
				else
				{
					const FString Problem(FString::Printf(TEXT("The data table row data for row %s was not found."), *RowName));
					ProblemStrings.Add(Problem);
				}
			}
			// Row wasn't found.
			else
			{
				const FString Problem(FString::Printf(TEXT("The data table row %s was not found."), *RowName));
				ProblemStrings.Add(Problem);
			}
		}
		// Property wasn't found.
		else
		{
			const FString Problem(FString::Printf(TEXT("Couldn't find the data table property named %s. Check the spelling."), *ColumnName));
			ProblemStrings.Add(Problem);
		}

		if (bWasDataTableChanged)
		{
			if (ChangedDataTables == nullptr)
			{
				DataTable->HandleDataTableChanged();
			}
			else
			{
				ChangedDataTables->Add(DataTable);
			}
		}
	}
	else if (CurveTable)
	{
		bool bWasCurveTableChanged = false;

		if (ColumnName.IsNumeric())
		{
			// Get the row data by name.
			static const FString Context = FString(TEXT("UOnlineHotfixManager::PatchAssetsFromIniFiles"));
			FRealCurve* CurveTableRow = CurveTable->FindCurve(FName(*RowName), Context);

			if (CurveTableRow)
			{
				// Edit the row with the new value.
				const float KeyTime = FCString::Atof(*ColumnName);
				FKeyHandle Key = CurveTableRow->FindKey(KeyTime);

				bool bWasExistingKey = CurveTableRow->IsKeyHandleValid(Key);

				if (NewValue.IsNumeric())
				{
					const float OldPropertyValue = CurveTableRow->GetKeyValue(Key);
					const float NewPropertyValue = FCString::Atof(*NewValue);
					Key = CurveTableRow->UpdateOrAddKey(KeyTime, NewPropertyValue);
					if (CurveTableRow->IsKeyHandleValid(Key))
					{
						OnHotfixTableValueFloatWithSource(*Asset, RowName, ColumnName, OldPropertyValue, NewPropertyValue, SourceTag);
						bWasCurveTableChanged = true;

						if (bWasExistingKey)
						{
							UE_LOGF(LogHotfixManager, VeryVerbose, "Curve table %ls row %ls updated column %ls from %.2f to %.2f.", *AssetPath, *RowName, *ColumnName, OldPropertyValue, NewPropertyValue);
						}
						else
						{
							UE_LOGF(LogHotfixManager, VeryVerbose, "Curve table %ls row %ls added column %ls with value %.2f.", *AssetPath, *RowName, *ColumnName, NewPropertyValue);
						}
					}
					else
					{
						const FString Problem(FString::Printf(TEXT("Unable to update Curve table %s row %s column %s with value %.2f."), *AssetPath, *RowName, *ColumnName, NewPropertyValue));
						ProblemStrings.Add(Problem);
					}
				}
				else
				{
					const FString Problem(FString::Printf(TEXT("The new value %s at key %f is not a number when it should be."), *NewValue, KeyTime));
					ProblemStrings.Add(Problem);
				}
			}
			else
			{
				const FString Problem(FString::Printf(TEXT("The curve table row for row name %s was not found."), *RowName));
				ProblemStrings.Add(Problem);
			}
		}
		else
		{
			const FString Problem(FString::Printf(TEXT("The column name %s is not a number when it should be."), *ColumnName));
			ProblemStrings.Add(Problem);
		}

		if (bWasCurveTableChanged)
		{
			if (ChangedCurveTables == nullptr)
			{
				CurveTable->OnCurveTableChanged().Broadcast();
			}
			else
			{
				ChangedCurveTables->Add(CurveTable);
			}
		}
	}
	else if (CurveFloat)
	{
		if (ColumnName.IsNumeric())
		{
			// Edit the curve with the new value.
			const float KeyTime = FCString::Atof(*ColumnName);
			FKeyHandle Key = CurveFloat->FloatCurve.FindKey(KeyTime);
			if (CurveFloat->FloatCurve.IsKeyHandleValid(Key))
			{
				if (NewValue.IsNumeric())
				{
					const float OldPropertyValue = CurveFloat->FloatCurve.GetKeyValue(Key);
					const float NewPropertyValue = FCString::Atof(*NewValue);
					CurveFloat->FloatCurve.SetKeyValue(Key, NewPropertyValue);
					OnHotfixTableValueFloatWithSource(*Asset, RowName, ColumnName, OldPropertyValue, NewPropertyValue, SourceTag);

					UE_LOGF(LogHotfixManager, VeryVerbose, "Curve float %ls updated column %ls from %.2f to %.2f.", *AssetPath, *ColumnName, OldPropertyValue, NewPropertyValue);
				}
				else
				{
					const FString Problem(FString::Printf(TEXT("The new value %s is not a number when it should be."), *NewValue));
					ProblemStrings.Add(Problem);
				}
			}
			else
			{
				const FString Problem(FString::Printf(TEXT("The column name %s isn't a valid key into the curve float."), *ColumnName));
				ProblemStrings.Add(Problem);
			}
		}
		else
		{
			const FString Problem(FString::Printf(TEXT("The column name %s is not a number when it should be."), *ColumnName));
			ProblemStrings.Add(Problem);
		}
	}
	else
	{
		ProblemStrings.Add(TEXT("The Asset isn't a Data Table, Curve Table, or Curve Float."));
	}
}

void UOnlineHotfixManager::HotfixAddRow(
	UObject* Asset,
	const FString& AssetPath,
	const FString& JsonData,
	TArray<FString>& ProblemStrings,
	TSet<UDataTable*>* ChangedDataTables)
{
	if (AssetPath.IsEmpty())
	{
		ProblemStrings.Add(TEXT("The table's path is empty. We cannot continue the hotfix."));
		return;
	}
	if (JsonData.IsEmpty())
	{
		ProblemStrings.Add(TEXT("The JSON data is empty. We cannot continue the hotfix."));
		return;
	}

	if (UDataTable* DataTable = Cast<UDataTable>(Asset))
	{
		FName RowName = NAME_None;
		// Allow hotfix to be applied multiple times
		const bool bRemoveDuplicate = true;
		if (!DataTableUtils::AddRowJSON(*DataTable, JsonData, ProblemStrings, bRemoveDuplicate, &RowName))
		{
			return;
		}

		OnHotfixTableAddRow(*DataTable, RowName);

		if (ChangedDataTables == nullptr)
		{
			DataTable->HandleDataTableChanged();
		}
		else
		{
			ChangedDataTables->Add(DataTable);
		}
	}
}

void UOnlineHotfixManager::HotfixTableUpdate(UObject* Asset, const FString& AssetPath, const FString& JsonData, TArray<FString>& ProblemStrings)
{
	if (AssetPath.IsEmpty())
	{
		ProblemStrings.Add(TEXT("The table's path is empty. We cannot continue the hotfix."));
		return;
	}
	if (JsonData.IsEmpty())
	{
		ProblemStrings.Add(TEXT("The JSON data is empty. We cannot continue the hotfix."));
		return;
	}

	// Let's import over the object in place.
	UCurveTable* CurveTable = Cast<UCurveTable>(Asset);
	UDataTable* DataTable = Cast<UDataTable>(Asset);
	UCurveFloat* CurveFloat = Cast<UCurveFloat>(Asset);
	UCurveVector* CurveVector = Cast<UCurveVector>(Asset);
	UCurveLinearColor* CurveLinearColor = Cast<UCurveLinearColor>(Asset);
	if (CurveTable != nullptr)
	{
		ProblemStrings.Append(CurveTable->CreateTableFromJSONString(JsonData));
		UE_LOGF(LogHotfixManager, Log, "Curve table %ls updated.", *AssetPath);
	}
	else if (DataTable != nullptr)
	{
		ProblemStrings.Append(DataTable->CreateTableFromJSONString(JsonData));
		UE_LOGF(LogHotfixManager, Log, "Data table %ls updated.", *AssetPath);
	}
	else if (CurveFloat != nullptr)
	{
		CurveFloat->ImportFromJSONString(JsonData, ProblemStrings);
		UE_LOGF(LogHotfixManager, Log, "Curve float %ls updated.", *AssetPath);
	}
	else if (CurveVector != nullptr)
	{
		CurveVector->ImportFromJSONString(JsonData, ProblemStrings);
		UE_LOGF(LogHotfixManager, Log, "Curve vector %ls updated.", *AssetPath);
	}
	else if (CurveLinearColor != nullptr)
	{
		CurveLinearColor->ImportFromJSONString(JsonData, ProblemStrings);
		UE_LOGF(LogHotfixManager, Log, "Curve linear color %ls updated.", *AssetPath);
	}
	else
	{
		ProblemStrings.Add(TEXT("Unable to hotfix this asset type. Only DataTables, CurveTables and Curve data types are supported."));
	}
}

bool UOnlineHotfixManager::ShouldPerformHotfix()
{
	return IsRunningGame() || IsRunningDedicatedServer() || IsRunningClientOnly();
}

FString UOnlineHotfixManager::GetDedicatedServerPrefix() const
{
	return TEXT("DedicatedServer");
}

bool UOnlineHotfixManager::ShouldHotfixAsset(const FString& AssetPath) const
{
	return true;
}

UWorld* UOnlineHotfixManager::GetWorld() const
{
	// OwnerWorld may be stale as it is set at the time that ::Get is called.
	if (OwnerWorld.IsValid())
	{
		return OwnerWorld.Get();
	}

	// Try to get the active world from the game instance.
	UGameViewportClient* ActiveViewport = GEngine ? GEngine->GameViewport.Get() : nullptr;
	UGameInstance* GameInstance = ActiveViewport ? ActiveViewport->GetGameInstance() : nullptr;
	return GameInstance ? GameInstance->GetWorld() : nullptr;
}

void UOnlineHotfixManager::StopTrackingInvalidHotfixedAssets()
{
	AssetsHotfixedFromIniFiles.RemoveAllSwap([](const UObject* Obj) { return !IsValid(Obj); });
}

void UOnlineHotfixManager::HotfixDynamicBranch(const FName& Tag, const FName& Branch, class FConfigModificationTracker* ModificationTracker)
{
	const FString HotfixBaseName = Tag.ToString() + Branch.ToString();
	const TArray<TPair<FString, FString>>* HotfixesForTag = DynamicHotfixContents.Find(*HotfixBaseName);

	UE_LOGF(LogHotfixManager, VeryVerbose, "[DynamicHotfixContents] HotfixDynamicBranch: Searching for hotfix for base name: %ls. Result: %ls", *HotfixBaseName, HotfixesForTag ? TEXT("found") : TEXT("not found"));

	TArray<FName> AffectedPaths;
	const bool bCheckForNewHotfixes = IsHotfixOnLoadEnabled() && Branch == NAME_Game;

	if (OnlineHotfixManagerCVars::bHotfixOnLoadIncludesOnDiskPluginInis && bCheckForNewHotfixes)
	{
		// Asset hotfixes in on-disk plugin ini files are not applied until the plugin is activated, which means that they are not picked up within ParseAssetHotfixes().
		// Instead, identify the hotfixes as the plugin activates and track them now
		if (FConfigBranch* BranchToHotfix = GConfig->FindBranch(Branch, FString()))
		{
			for (const FConfigCommandStream* Layer : BranchToHotfix->DynamicLayers)
			{
				if (Layer && Layer->Tag == Tag && !Layer->IsEmpty())
				{
					UE_LOGF(LogHotfixManager, VeryVerbose, "HotfixDynamicBranch: searching for [AssetHotfix] entries in local config for plugin %ls", *Tag.ToString());

					const static FString AssetHotfixString = FString(TEXT("AssetHotfix"));
					if (const FConfigCommandStreamSection* Section = Layer->Find(AssetHotfixString))
					{
						TArray<FName> LocalPaths = AssetHotfixRegistry.ParseAndStoreHotfixEntries(Tag, *Section);
						if (LocalPaths.Num() > 0)
						{
							UE_LOGF(LogHotfixManager, Log, "HotfixDynamicBranch: parsed %d [AssetHotfix] entries from local config for plugin %ls", LocalPaths.Num(), *Tag.ToString());
							AffectedPaths.Append(MoveTemp(LocalPaths));
						}
					}
				}
			}
		}
	}

	if (HotfixesForTag)
	{
		FConfigBranch* BranchToHotfix = GConfig->FindBranch(Branch, FString());
		for (const TPair<FString, FString>& HotfixFilenameAndContents : *HotfixesForTag)
		{
			UE_LOGF(LogHotfixManager, Log, "[DynamicHotfixContents] HotfixDynamicBranch: applying hotfix %ls for tag %ls on branch %ls", *HotfixFilenameAndContents.Key, *Tag.ToString(), *Branch.ToString());

			if (BranchToHotfix && !BranchToHotfix->AddDynamicLayerStringToHierarchy(HotfixFilenameAndContents.Key, HotfixFilenameAndContents.Value, Tag, DynamicLayerPriority::Hotfix, ModificationTracker))
			{
				UE_LOGF(LogHotfixManager, Warning, "[DynamicHotfixContents] HotfixDynamicBranch: failed to apply hotfix");
			}

			// This specifically checks for 'Game' branch as [AssetHotfix] must be in Game ini files.
			if (bCheckForNewHotfixes)
			{
				AffectedPaths.Append(AssetHotfixRegistry.ParseAndStoreFromRawIniData(Tag, HotfixFilenameAndContents.Value));
			}
		}
	}

	if (AffectedPaths.Num() > 0)
	{
#if !UE_BUILD_SHIPPING
		ValidateHotfixableAssetClasses(AffectedPaths);
#endif
		ApplyHotfixesToLoadedAssets(AffectedPaths);
	}
}

void UOnlineHotfixManager::RemoveHotfixesFromDynamicBranch(const FName& Tag)
{
	AssetHotfixRegistry.RemoveEntriesFromSource(Tag);
	OnDynamicHotfixesRemoved(Tag);
}

void UOnlineHotfixManager::ReloadObjectsAffectedByConfigFile(const FString& IniDataFileName, const FString& IniData, const FString& ConfigFilename, TArray<FString>& ReloadedClassesPathNames, bool bUseLoadConfig)
{
	TArray<UClass*> Classes;
	TArray<UObject*> PerObjectConfigObjects;
	int32 StartIndex = 0;
	int32 EndIndex = 0;
	TSet<FString> UpdatedSectionNames;
	// Find the set of object classes that were affected
	while (StartIndex >= 0 && StartIndex < IniData.Len() && EndIndex >= StartIndex)
	{
		// Find the next section header
		StartIndex = IniData.Find(TEXT("["), ESearchCase::IgnoreCase, ESearchDir::FromStart, StartIndex);
		if (StartIndex > -1)
		{
			// Find the ending section identifier
			EndIndex = IniData.Find(TEXT("]"), ESearchCase::IgnoreCase, ESearchDir::FromStart, StartIndex);
			if (EndIndex > StartIndex)
			{
				// Ignore square brackets in the middle of string
				// - per object section starts with new line
				// - there's no " character between opening bracket and line start
				const bool bStartsWithNewLine = (StartIndex == 0) || (IniData[StartIndex - 1] == TEXT('\n'));
				if (!bStartsWithNewLine)
				{
					bool bStartsInsideString = false;
					for (int32 CharIdx = StartIndex - 1; CharIdx >= 0; CharIdx--)
					{
						const bool bHasStringMarker = (IniData[CharIdx] == TEXT('"'));
						if (bHasStringMarker)
						{
							bStartsInsideString = true;
							break;
						}

						const bool bHasNewLineMarker = (IniData[CharIdx] == TEXT('\n'));
						if (bHasNewLineMarker)
						{
							break;
						}
					}

					if (bStartsInsideString)
					{
						StartIndex = EndIndex;
						continue;
					}
				}

				UpdatedSectionNames.Emplace(IniData.Mid(StartIndex + 1, EndIndex - StartIndex - 1));

				int32 PerObjectNameIndex = IniData.Find(TEXT(" "), ESearchCase::IgnoreCase, ESearchDir::FromStart, StartIndex);

				const TCHAR* AssetHotfixIniHACK = TEXT("[AssetHotfix]");
				if (FCString::Strnicmp(*IniData + StartIndex, AssetHotfixIniHACK, FCString::Strlen(AssetHotfixIniHACK)) == 0)
				{
					// HACK - Make AssetHotfix the last element in the ini file so that this parsing isn't affected by it for now
					break;
				}

				// Per object config entries will have a space in the name, but classes won't
				if (PerObjectNameIndex == -1 || PerObjectNameIndex > EndIndex)
				{
					const TCHAR* ScriptHeader = TEXT("[/Script/");
					const TCHAR* GameHeader = TEXT("[/Game/");
					if (FCString::Strnicmp(*IniData + StartIndex, ScriptHeader, FCString::Strlen(ScriptHeader)) == 0)
					{
						const int32 ScriptSectionTag = 9;
						// Snip the text out and try to find the class for that
						const FString PackageClassName = IniData.Mid(StartIndex + ScriptSectionTag, EndIndex - StartIndex - ScriptSectionTag);
						// Find the class for this so we know what to update
						UClass* Class = FindObject<UClass>(nullptr, *PackageClassName, EFindObjectFlags::ExactClass);
						if (Class)
						{
							// Add this to the list to check against
							Classes.Add(Class);
							ReloadedClassesPathNames.AddUnique(Class->GetPathName());
						}
					}
					else if (FCString::Strnicmp(*IniData + StartIndex, GameHeader, FCString::Strlen(GameHeader)) == 0)
					{
						const int32 GameSectionTag = 1;
						// Snip the text out and try to find the class for that
						const FString PackageClassName = IniData.Mid(StartIndex + GameSectionTag, EndIndex - StartIndex - GameSectionTag);
						UBlueprintGeneratedClass* BPGeneratedClass = LoadObject<UBlueprintGeneratedClass>(nullptr, *PackageClassName);
						if (BPGeneratedClass)
						{
							// Add this to the list to check against
							Classes.Add(BPGeneratedClass);
							ReloadedClassesPathNames.AddUnique(BPGeneratedClass->GetPathName());
						}
					}
				}
				// Handle the per object config case by finding the object for reload
				else
				{
					const int32 ClassNameStart = PerObjectNameIndex + 1;
					const FString ClassName = IniData.Mid(ClassNameStart, EndIndex - ClassNameStart);

					// Look up the class to search for
					UClass* ObjectClass = UClass::TryFindTypeSlow<UClass>(ClassName);

					if (ObjectClass)
					{
						const int32 Count = PerObjectNameIndex - StartIndex - 1;
						const FString PerObjectName = IniData.Mid(StartIndex + 1, Count);

						// Explicitly search the transient package (won't update non-transient objects)
						UObject* PerObject = StaticFindFirstObject(ObjectClass, *PerObjectName, EFindFirstObjectOptions::NativeFirst);
						if (PerObject != nullptr)
						{
							PerObjectConfigObjects.Add(PerObject);
							ReloadedClassesPathNames.AddUnique(ObjectClass->GetPathName());
						}
					}
					else
					{
						UE_LOGF(LogHotfixManager, Warning, "Specified per-object class %ls was not found", *ClassName);
					}
				}
				StartIndex = EndIndex;
			}
		}
	}

	int32 NumObjectsReloaded = 0;
	const double StartTime = FPlatformTime::Seconds();
	// Now that we have a list of classes to update, we can iterate objects and reload
	for (UClass* Class : Classes)
	{
		if (Class->HasAnyClassFlags(CLASS_Config))
		{
			TArray<UObject*> Objects;
			GetObjectsOfClass(Class, Objects, true, RF_NoFlags);
			for (UObject* Object : Objects)
			{
				if (IsValid(Object))
				{
					// Force a reload of the config vars
					UE_LOGF(LogHotfixManager, Verbose, "Reloading %ls", *Object->GetPathName());
					bUseLoadConfig ? Object->LoadConfig() : Object->ReloadConfig();
					NumObjectsReloaded++;
				}
			}
		}
	}

	// Reload any PerObjectConfig objects that were affected
	for (UObject* ReloadObject : PerObjectConfigObjects)
	{
		UE_LOGF(LogHotfixManager, Verbose, "Reloading %ls", *ReloadObject->GetPathName());
		bUseLoadConfig ? ReloadObject->LoadConfig() : ReloadObject->ReloadConfig();
		NumObjectsReloaded++;
	}

	FCoreDelegates::TSOnConfigSectionsChanged().Broadcast(ConfigFilename, UpdatedSectionNames);

	UE_LOGF(LogHotfixManager, Log, "Updating config from %ls took %f seconds and reloaded %d objects",
		*IniDataFileName, FPlatformTime::Seconds() - StartTime, NumObjectsReloaded);
}

#if UE_ALLOW_EXEC_COMMANDS
struct FHotfixManagerExec :
	public FSelfRegisteringExec
{
protected:
	virtual bool Exec_Runtime(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar) override
	{
		if (FParse::Command(&Cmd, TEXT("HOTFIX")))
		{
			UOnlineHotfixManager* HotfixManager = UOnlineHotfixManager::Get(InWorld);
			if (HotfixManager != nullptr)
			{
				HotfixManager->StartHotfixProcess();
			}
			return true;
		}
		else if (FParse::Command(&Cmd, TEXT("TESTHOTFIXSORT")))
		{
			TArray<FCloudFileHeader> TestList;
			FCloudFileHeader Header;
			Header.FileName = TEXT("SomeRandom.ini");
			TestList.Add(Header);
			Header.FileName = TEXT("DedicatedServerGame.ini");
			TestList.Add(Header);
			Header.FileName = TEXT("pakchunk1-PS4_P.pak");
			TestList.Add(Header);
			Header.FileName = TEXT("EN_Game.locres");
			TestList.Add(Header);
			Header.FileName = TEXT("DefaultGame.ini");
			TestList.Add(Header);
			Header.FileName = TEXT("Ver-1234_DefaultEngine.ini");
			TestList.Add(Header);
			Header.FileName = TEXT("PS4_DefaultEngine.ini");
			TestList.Add(Header);
			Header.FileName = TEXT("DefaultEngine.ini");
			TestList.Add(Header);
			Header.FileName = TEXT("pakchunk0-PS4_P.pak");
			TestList.Add(Header);
			Header.FileName = TEXT("PS4_DefaultGame.ini");
			TestList.Add(Header);
			Header.FileName = TEXT("Ver-1234_PS4_DefaultGame.ini");
			TestList.Add(Header);
			Header.FileName = TEXT("PS4_Ver-1234_DefaultGame.ini");
			TestList.Add(Header);
			Header.FileName = TEXT("AnotherRandom.ini");
			TestList.Add(Header);
			Header.FileName = TEXT("DedicatedServerEngine.ini");
			TestList.Add(Header);
			TestList.Sort<FHotfixFileSortPredicate>(FHotfixFileSortPredicate(TEXT("PS4_"), TEXT("DedicatedServer"), TEXT("Default"), HOTFIX_NO_CUSTOM_CONFIG));

			UE_LOGF(LogHotfixManager, Log, "Hotfixing sort is:");
			for (const FCloudFileHeader& FileHeader : TestList)
			{
				UE_LOGF(LogHotfixManager, Log, "\t%ls", *FileHeader.FileName);
			}

			TArray<FString> TestList2;
			TestList2.Add(TEXT("SomeRandom.ini"));
			TestList2.Add(TEXT("DefaultGame.ini"));
			TestList2.Add(TEXT("PS4_DefaultEngine.ini"));
			TestList2.Add(TEXT("DedicatedServerEngine.ini"));
			TestList2.Add(TEXT("DedicatedServerGame.ini"));
			TestList2.Add(TEXT("DefaultEngine.ini"));
			TestList2.Add(TEXT("PS4_DefaultGame.ini"));
			TestList2.Add(TEXT("AnotherRandom.ini"));
			TestList2.Sort<FHotfixFileSortPredicate>(FHotfixFileSortPredicate(TEXT("PS4_"), TEXT("DedicatedServer"), TEXT("Default"), HOTFIX_NO_CUSTOM_CONFIG));

			UE_LOGF(LogHotfixManager, Log, "Hotfixing PAK INI file sort is:");
			for (const FString& IniName : TestList2)
			{
				UE_LOGF(LogHotfixManager, Log, "\t%ls", *IniName);
			}
			return true;
		}
		else if (FParse::Command(&Cmd, TEXT("LOGHOTFIXASSETLIST")))
		{
			if (UOnlineHotfixManager* HotfixManager = UOnlineHotfixManager::Get(InWorld))
			{
				int32 CurrentAsset = 0;
				const int32 AssetCount = HotfixManager->AssetsHotfixedFromIniFiles.Num();
				UE_LOGF(LogHotfixManager, Log, "Logging contetents of AssetsHotfixedFromIniFiles. Asset count: %d", AssetCount);
				for (const TObjectPtr<UObject>& ObjPtr : HotfixManager->AssetsHotfixedFromIniFiles)
				{
					++CurrentAsset;
					UE_LOGF(LogHotfixManager, Log, "\t[%d/%d]: %ls", CurrentAsset, AssetCount, *GetPathNameSafe(ObjPtr.Get()));
				}
				UE_LOGF(LogHotfixManager, Log, "Logging complete.");
			}
			return true;
		}
		return false;
	}
};
static FHotfixManagerExec HotfixManagerExec;
#endif // UE_ALLOW_EXEC_COMMANDS
