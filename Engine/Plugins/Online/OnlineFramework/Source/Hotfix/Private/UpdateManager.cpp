// Copyright Epic Games, Inc. All Rights Reserved.

#include "UpdateManager.h"
#include "HAL/IConsoleManager.h"
#include "Misc/CommandLine.h"
#include "Engine/GameInstance.h"
#include "Misc/CoreDelegates.h"
#include "InstallBundleManagerInterface.h"
#include "InstallBundleUtils.h"

#include "OnlineError.h"
#include "OnlineHotfixManager.h"
#include "Engine/World.h"

#include "PatchCheck.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "ProfilingDebugging/LoadTimeTracker.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UpdateManager)

static TAutoConsoleVariable<int32> CVarDebugUpdateManager(
	TEXT("UI.DebugUpdateCheck"),
	-1,
	TEXT("Force switch between update states (-1 is off)"));

static TAutoConsoleVariable<float> CVarUpdateManagerCacheTimeoutSeconds(
	TEXT("UpdateManager.CacheTimeoutSeconds"),
	300,
	TEXT("Timeout in seconds to return a previous update manager response."),
	ECVF_SaveForNextBoot);

struct FLoadingScreenConfig
{
public:
	// Do we check for hotfixes in this build?
	static bool CheckForHotfixes() 
	{ 
#if UE_BUILD_SHIPPING
		return true;
#else
		static bool bCheckHotfixes = !FParse::Param(FCommandLine::Get(), TEXT("SkipHotfixCheck"));
		return bCheckHotfixes;
#endif
	}

	// Do we block waiting for pending async loads to complete during the initial loading screen state?
	static bool ShouldBlockOnInitialLoad() { return (FPlatformProperties::IsServerOnly() || true); }

	// Can we preload assets used in Agora during the initial loading screen?
	static bool CanPreloadMapAssets() { return true; }

private:
	FLoadingScreenConfig() {}
};

UUpdateManager::UUpdateManager()
	: UpdateCheckStartDelay(0.2f)
	, UpdateCheckCachedResponseDelay(0.1f)
	, HotfixCheckCompleteDelay(0.1f)
	, UpdateCheckCompleteDelay(0.5f)
	, HotfixAvailabilityCheckCompleteDelay(0.1f)
	, UpdateCheckAvailabilityCompleteDelay(0.1f)
	, AppSuspendedUpdateCheckTimeSeconds(600)
	, bInitialUpdateFinished(false)
	, bCheckHotfixAvailabilityOnly(false)
	, CurrentUpdateState(EUpdateState::UpdateIdle)
	, WorstNumFilesPendingLoadViewed(0)
	, LastPatchCheckResult(EInstallBundleManagerPatchCheckResult::PatchCheckFailure)
	, LastHotfixResult(EHotfixResult::Failed)
	, LoadStartTime(0.0)
	, UpdateStateEnum(nullptr)
	, UpdateCompletionEnum(nullptr)
{
	LastUpdateCheck[0] = LastUpdateCheck[1] = FDateTime(0);
	LastCompletionResult[0] = LastCompletionResult[1] = EUpdateCompletionStatus::UpdateUnknown;
	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		UpdateStateEnum = StaticEnum<EUpdateState>();
		UpdateCompletionEnum = StaticEnum<EUpdateCompletionStatus>();

		RegisterDelegates();
	}
}

UUpdateManager::~UUpdateManager()
{
	UnregisterDelegates();
}

void UUpdateManager::SetPending()
{ 
	if (ChecksEnabled())
	{
		CurrentUpdateState = EUpdateState::UpdatePending;
	}
}

void UUpdateManager::RegisterStatsCollector(const TSharedPtr<IUpdateManagerStatsCollector>& InStatsCollector)
{
	StatsCollector = InStatsCollector;
}

void UUpdateManager::Reset()
{
	LastUpdateCheck[0] = LastUpdateCheck[1] = FDateTime(0);
	SetUpdateState(EUpdateState::UpdatePending);
}

void UUpdateManager::StartCheck(bool bInCheckHotfixOnly)
{
	StartCheckInternal(bInCheckHotfixOnly);
}

void UUpdateManager::StartUpdateCheck(const FString& ContextName)
{
	StartUpdateCheckInternal(GetContextDefinition(ContextName));
}

UUpdateManager::EUpdateStartResult UUpdateManager::StartCheckInternal(bool bInCheckHotfixOnly)
{
	FUpdateContextDefinition AdhocDefinition;
	AdhocDefinition.Name = TEXT("Adhoc");
	AdhocDefinition.bCheckAvailabilityOnly = bInCheckHotfixOnly;
	return StartUpdateCheckInternal(AdhocDefinition);
}

UUpdateManager::EUpdateStartResult UUpdateManager::StartUpdateCheckInternal(const FUpdateContextDefinition& ContextDefinition)
{
	EUpdateStartResult Result = EUpdateStartResult::None;

	UE_LOGF(LogHotfixManager, Log, "Starting update check using context \"%ls\"", *ContextDefinition.Name);

	if (!ChecksEnabled() || !ContextDefinition.bEnabled)
	{
		UE_LOGF(LogHotfixManager, Display, "Update checks disabled!");
		bInitialUpdateFinished = true;

		if (TSharedPtr<IUpdateManagerStatsCollector> StrongStatsCollector = StatsCollector.Pin())
		{
			StrongStatsCollector->OnCheckStarted({
				.ContextName = ContextDefinition.Name,
				.bEnabled = false });
		}

		// Move to the pending state until the delayed response can fire, to more closely match non-editor behavior.
		SetUpdateState(EUpdateState::UpdatePending);

		auto StartDelegate = [this]()
		{
			CheckComplete(EUpdateCompletionStatus::UpdateSuccess_NoChange);
		};
		
		DelayResponse(StartDelegate, 0.1f);
		return Result;
	}

	if (!StartCheckInternalTimerHandle.IsValid() &&
		(CurrentUpdateState == EUpdateState::UpdateIdle ||
		CurrentUpdateState == EUpdateState::UpdatePending ||
		CurrentUpdateState == EUpdateState::UpdateComplete))
	{
		bCheckHotfixAvailabilityOnly = ContextDefinition.bCheckAvailabilityOnly;
		bCurrentUpdatePatchCheckEnabled = ContextDefinition.bPatchCheckEnabled;

		// Immediately move into a pending state so the UI state trigger fires
		SetUpdateState(EUpdateState::UpdatePending);

		const EUpdateCompletionStatus LastResult = LastCompletionResult[bCheckHotfixAvailabilityOnly];
		const FTimespan DeltaTime = FDateTime::UtcNow() - LastUpdateCheck[bCheckHotfixAvailabilityOnly];

		const bool bForceCheck = LastResult == EUpdateCompletionStatus::UpdateUnknown ||
								 LastResult == EUpdateCompletionStatus::UpdateFailure_PatchCheck ||
								 LastResult == EUpdateCompletionStatus::UpdateFailure_HotfixCheck ||
								 LastResult == EUpdateCompletionStatus::UpdateFailure_NotLoggedIn;

		const double CacheTimer = CVarUpdateManagerCacheTimeoutSeconds.GetValueOnGameThread();
		const double TimeSinceCheck = DeltaTime.GetTotalSeconds();
		if (bForceCheck || TimeSinceCheck >= CacheTimer || UOnlineHotfixManager::bAlwaysReapplyHotfixes)
		{
			if (TSharedPtr<IUpdateManagerStatsCollector> StrongStatsCollector = StatsCollector.Pin())
			{
				StrongStatsCollector->OnCheckStarted({
					.ContextName = ContextDefinition.Name,
					.bCheckAvailabilityOnly = ContextDefinition.bCheckAvailabilityOnly });
			}

			auto StartDelegate = [this]()
			{
				// Check for a patch first, then hotfix application
				StartPatchCheck();
				StartCheckInternalTimerHandle.Reset();
			};

			// Give the UI state widget a chance to start listening for delegates
			StartCheckInternalTimerHandle = DelayResponse(StartDelegate, UpdateCheckStartDelay);
			Result = EUpdateStartResult::UpdateStarted;
		}
		else
		{
			if (TSharedPtr<IUpdateManagerStatsCollector> StrongStatsCollector = StatsCollector.Pin())
			{
				StrongStatsCollector->OnCheckStarted({
					.ContextName = ContextDefinition.Name,
					.bCheckAvailabilityOnly = ContextDefinition.bCheckAvailabilityOnly,
					.bCachedResponse = true,
					.CacheExpiration = CacheTimer - TimeSinceCheck });
			}

			UE_LOGF(LogHotfixManager, Display, "Returning cached update result %d. Cache timeout in %.2f/%.2f seconds.", (int32)LastResult, CacheTimer - TimeSinceCheck, CacheTimer);
			auto StartDelegate = [this, LastResult]()
			{
				CheckComplete(LastResult, false);
				StartCheckInternalTimerHandle.Reset();
			};

			StartCheckInternalTimerHandle = DelayResponse(StartDelegate, UpdateCheckCachedResponseDelay);
			Result = EUpdateStartResult::UpdateCached;
		}
	}
	else
	{
		UE_LOGF(LogHotfixManager, Display, "Update already in progress");
	}

	return Result;
}

void UUpdateManager::CheckComplete(EUpdateCompletionStatus Result, bool bUpdateTimestamp)
{
	UE_LOGF(LogHotfixManager, Display, "CheckComplete %ls", UpdateCompletionEnum ? *UpdateCompletionEnum->GetNameStringByValue((int64)Result) : TEXT("Invalid"));

	UGameInstance* GameInstance = GetGameInstance();
	bool bIsServer = GameInstance ? GameInstance->IsDedicatedServerInstance() : false;

#if !UE_BUILD_SHIPPING
	int32 DbgVal = CVarDebugUpdateManager.GetValueOnGameThread();
	if (DbgVal >= 0 && DbgVal <= (int32)EUpdateCompletionStatus::UpdateFailure_NotLoggedIn)
	{
		Result = (EUpdateCompletionStatus)DbgVal;
		UE_LOGF(LogHotfixManager, Display, "CheckComplete OVERRIDE! %ls", UpdateCompletionEnum ? *UpdateCompletionEnum->GetNameStringByValue((int64)Result) : TEXT("Invalid"));
	}
#endif

	LastCompletionResult[bCheckHotfixAvailabilityOnly] = Result;

	const bool bSuccessResult = (Result == EUpdateCompletionStatus::UpdateSuccess ||	
								 Result == EUpdateCompletionStatus::UpdateSuccess_NoChange ||
								 Result == EUpdateCompletionStatus::UpdateSuccess_NeedsReload ||
								 Result == EUpdateCompletionStatus::UpdateSuccess_NeedsRelaunch);
	
	if (bUpdateTimestamp && bSuccessResult)
	{
		LastUpdateCheck[bCheckHotfixAvailabilityOnly] = FDateTime::UtcNow();
	}

	auto CompletionDelegate = [this, Result, bUpdateTimestamp]()
	{
		UE_LOGF(LogHotfixManager, Display, "External CheckComplete %ls", UpdateCompletionEnum ? *UpdateCompletionEnum->GetNameStringByValue((int64)Result) : TEXT("Invalid"));
		if (TSharedPtr<IUpdateManagerStatsCollector> StrongStatsCollector = StatsCollector.Pin())
		{
			StrongStatsCollector->OnCheckComplete(Result);
		}

		if (!bInitialUpdateFinished)
		{
			// Prime the state so that the first "after login" check will occur
			bInitialUpdateFinished = true;
			SetUpdateState(EUpdateState::UpdatePending);
		}
		else
		{
			SetUpdateState(EUpdateState::UpdateComplete);
		}

		EUpdateCompletionStatus FinalResult = Result;
		if (Result == EUpdateCompletionStatus::UpdateSuccess && !bCheckHotfixAvailabilityOnly && !bUpdateTimestamp)
		{
			// if this is a cached value, and we are not checking availability only, we should return NoChange,
			// As we have already applied this hotfix.
			FinalResult = EUpdateCompletionStatus::UpdateSuccess_NoChange;
		}

		bCheckHotfixAvailabilityOnly = false;
		bCurrentUpdatePatchCheckEnabled = true;
		
		OnUpdateCheckComplete().Broadcast(FinalResult);
	};

	// Delay completion delegate to give UI a chance to show the screen for a reasonable amount of time
	DelayResponse(CompletionDelegate, bCheckHotfixAvailabilityOnly ? UpdateCheckAvailabilityCompleteDelay : UpdateCheckCompleteDelay);
}

inline FName GetUniqueTag(UUpdateManager* UpdateManager)
{
	return FName(*FString::Printf(TEXT("Tag_%u_%s"), UpdateManager->GetUniqueID(), *UpdateManager->GetName()));
}

void UUpdateManager::StartPatchCheck()
{
	ensure(ChecksEnabled());
	if (TSharedPtr<IUpdateManagerStatsCollector> StrongStatsCollector = StatsCollector.Pin())
	{
		StrongStatsCollector->OnCheckStep_PatchCheckStarted();
	}

	SetUpdateState(EUpdateState::CheckingForPatch);
	UGameInstance* GameInstance = GetGameInstance();
	if ((GameInstance && GameInstance->IsDedicatedServerInstance()) || !bCurrentUpdatePatchCheckEnabled)
	{
		PatchCheckComplete(EPatchCheckResult::NoPatchRequired);
		return;
	}

	TSharedPtr<IInstallBundleManager> InstallBundleMan = IInstallBundleManager::GetPlatformInstallBundleManager();
	if (InstallBundleMan && !InstallBundleMan->IsNullInterface())
	{
		InstallBundleMan->PatchCheckCompleteDelegate.AddUObject(this, &UUpdateManager::InstallBundlePatchCheckComplete);
		InstallBundleMan->AddEnvironmentWantsPatchCheckBackCompatDelegate(
			GetUniqueTag(this),
			FEnvironmentWantsPatchCheck::CreateUObject(this, &UUpdateManager::EnvironmentWantsPatchCheck));
		InstallBundleMan->StartPatchCheck();
	}
	else
	{
		FPatchCheck::Get().GetOnComplete().AddUObject(this, &UUpdateManager::PatchCheckComplete);
		FPatchCheck::Get().AddEnvironmentWantsPatchCheckBackCompatDelegate(
			GetUniqueTag(this),
			FEnvironmentWantsPatchCheck::CreateUObject(this, &UUpdateManager::EnvironmentWantsPatchCheck));
		FPatchCheck::Get().StartPatchCheck();
	}
}

bool UUpdateManager::ChecksEnabled() const
{
	return !GIsEditor;
}

bool UUpdateManager::EnvironmentWantsPatchCheck() const
{
	return false;
}

EInstallBundleManagerPatchCheckResult ToInstallBundleManagerPatchCheckResult(EPatchCheckResult InResult)
{
	// EInstallBundleManagerPatchCheckResult should be a superset of EPatchCheckResult

	EInstallBundleManagerPatchCheckResult OutResult = EInstallBundleManagerPatchCheckResult::PatchCheckFailure;
	switch (InResult)
	{
	case EPatchCheckResult::NoPatchRequired:
		OutResult = EInstallBundleManagerPatchCheckResult::NoPatchRequired;
		break;
	case EPatchCheckResult::PatchRequired:
		OutResult = EInstallBundleManagerPatchCheckResult::ClientPatchRequired;
		break;
	case EPatchCheckResult::NoLoggedInUser:
		OutResult = EInstallBundleManagerPatchCheckResult::NoLoggedInUser;
		break;
	case EPatchCheckResult::PatchCheckFailure:
		OutResult = EInstallBundleManagerPatchCheckResult::PatchCheckFailure;
		break;
	default:
		ensureAlwaysMsgf(false, TEXT("Unknown EPatchCheckResult"));
		OutResult = EInstallBundleManagerPatchCheckResult::PatchCheckFailure;
		break;
	}

	// Make sure we don't miss a case
	static_assert(InstallBundleUtil::CastToUnderlying(EPatchCheckResult::Count) == 4, "");

	return OutResult;
}

void UUpdateManager::PatchCheckComplete(EPatchCheckResult PatchResult)
{
	FPatchCheck::Get().GetOnComplete().RemoveAll(this);
	FPatchCheck::Get().RemoveEnvironmentWantsPatchCheckBackCompatDelegate(GetUniqueTag(this));

	InstallBundlePatchCheckComplete(ToInstallBundleManagerPatchCheckResult(PatchResult));
}

void UUpdateManager::InstallBundlePatchCheckComplete(EInstallBundleManagerPatchCheckResult PatchResult)
{
	if (TSharedPtr<IUpdateManagerStatsCollector> StrongStatsCollector = StatsCollector.Pin())
	{
		StrongStatsCollector->OnCheckStep_PatchCheckComplete(PatchResult);
	}

	TSharedPtr<IInstallBundleManager> InstallBundleMan = IInstallBundleManager::GetPlatformInstallBundleManager();
	if (InstallBundleMan && !InstallBundleMan->IsNullInterface())
	{
		InstallBundleMan->RemoveEnvironmentWantsPatchCheckBackCompatDelegate(GetUniqueTag(this));
	}
	IInstallBundleManager::PatchCheckCompleteDelegate.RemoveAll(this);

	LastPatchCheckResult = PatchResult;

	if (PatchResult == EInstallBundleManagerPatchCheckResult::NoPatchRequired)
	{
		StartHotfixCheck();
	}
	else if (PatchResult == EInstallBundleManagerPatchCheckResult::NoLoggedInUser)
	{
		CheckComplete(EUpdateCompletionStatus::UpdateFailure_NotLoggedIn);
	}
	else
	{
		ensure(PatchResult == EInstallBundleManagerPatchCheckResult::PatchCheckFailure ||
			PatchResult == EInstallBundleManagerPatchCheckResult::ClientPatchRequired ||
			PatchResult == EInstallBundleManagerPatchCheckResult::ContentPatchRequired);

		// Skip hotfix check in error states, but still preload data as if there was nothing wrong
		StartInitialPreload();
	}
}

void UUpdateManager::StartHotfixCheck()
{
	if (TSharedPtr<IUpdateManagerStatsCollector> StrongStatsCollector = StatsCollector.Pin())
	{
		StrongStatsCollector->OnCheckStep_HotfixCheckStarted();
	}

	if (bCheckHotfixAvailabilityOnly)
	{
		// Just check for the presence of a hotfix
		StartHotfixAvailabilityCheck();
	}
	else
	{
		SetUpdateState(EUpdateState::CheckingForHotfix);

		UOnlineHotfixManager* HotfixManager = GetHotfixManager<UOnlineHotfixManager>();

		if (FLoadingScreenConfig::CheckForHotfixes())
		{
			HotfixProgressDelegateHandle = HotfixManager->AddOnHotfixProgressDelegate_Handle(FOnHotfixProgressDelegate::CreateUObject(this, &ThisClass::OnHotfixProgress));
			HotfixProcessedFileDelegateHandle = HotfixManager->AddOnHotfixProcessedFileDelegate_Handle(FOnHotfixProcessedFileDelegate::CreateUObject(this, &ThisClass::OnHotfixProcessedFile));
			HotfixCompleteDelegateHandle = HotfixManager->AddOnHotfixCompleteDelegate_Handle(FOnHotfixCompleteDelegate::CreateUObject(this, &ThisClass::OnHotfixCheckComplete));

			HotfixManager->StartHotfixProcess();
		}
		else
		{
			if (HotfixManager)
			{
				HotfixManager->OnHotfixProcessSkipped();
			}

			OnHotfixCheckComplete(EHotfixResult::SuccessNoChange);
		}
	}
}

void UUpdateManager::OnHotfixProgress(uint32 NumDownloaded, uint32 TotalFiles, uint64 NumBytes, uint64 TotalBytes)
{
	UE_LOG(LogHotfixManager, VeryVerbose, TEXT("OnHotfixProgress %d/%d [%" UINT64_FMT "/%" UINT64_FMT "]"), NumDownloaded, TotalFiles, NumBytes, TotalBytes);
	OnUpdateHotfixProgress().Broadcast(NumDownloaded, TotalFiles, NumBytes, TotalBytes);
}

void UUpdateManager::OnHotfixProcessedFile(const FString& FriendlyName, const FString& CachedName)
{
	UE_LOGF(LogHotfixManager, VeryVerbose, "OnHotfixProcessedFile %ls", *FriendlyName);
	OnUpdateHotfixProcessedFile().Broadcast(FriendlyName, CachedName);
}

void UUpdateManager::OnHotfixCheckComplete(EHotfixResult Result)
{
	UE_LOGF(LogHotfixManager, Display, "OnHotfixCheckComplete %d", (int32)Result);

	if (auto HotfixManager = GetHotfixManager<UOnlineHotfixManager>())
	{
		HotfixManager->ClearOnHotfixProgressDelegate_Handle(HotfixProgressDelegateHandle);
		HotfixManager->ClearOnHotfixProcessedFileDelegate_Handle(HotfixProcessedFileDelegateHandle);
		HotfixManager->ClearOnHotfixCompleteDelegate_Handle(HotfixCompleteDelegateHandle);
	}

	LastHotfixResult = Result;

	auto CompletionDelegate = [this, Result]()
	{
		if (TSharedPtr<IUpdateManagerStatsCollector> StrongStatsCollector = StatsCollector.Pin())
		{
			StrongStatsCollector->OnCheckStep_HotfixCheckComplete(Result);
		}

		// Always preload data
		StartInitialPreload();
	};

	// Debug delay delegate firing
	DelayResponse(CompletionDelegate, HotfixCheckCompleteDelay);
}

void UUpdateManager::StartHotfixAvailabilityCheck()
{
	SetUpdateState(EUpdateState::CheckingForHotfix);

	if (FLoadingScreenConfig::CheckForHotfixes())
	{
		UOnlineHotfixManager* HotfixManager = GetHotfixManager<UOnlineHotfixManager>();

		FOnHotfixAvailableComplete CompletionDelegate;
		CompletionDelegate.BindUObject(this, &ThisClass::HotfixAvailabilityCheckComplete);
		HotfixManager->CheckAvailability(CompletionDelegate);
	}
	else
	{
		OnHotfixCheckComplete(EHotfixResult::SuccessNoChange);
	}
}

void UUpdateManager::HotfixAvailabilityCheckComplete(EHotfixResult Result)
{
	UE_LOGF(LogHotfixManager, Display, "HotfixAvailabilityCheckComplete %d", (int32)Result);

	auto CompletionDelegate = [this, Result]()
	{
		UE_LOGF(LogHotfixManager, Display, "External HotfixAvailabilityCheckComplete %d", (int32)Result);
		if (TSharedPtr<IUpdateManagerStatsCollector> StrongStatsCollector = StatsCollector.Pin())
		{
			StrongStatsCollector->OnCheckStep_HotfixCheckComplete(Result);
		}

		switch (Result)
		{
			case EHotfixResult::Success:
				CheckComplete(EUpdateCompletionStatus::UpdateSuccess);
				break;
			case EHotfixResult::SuccessNoChange:
				CheckComplete(EUpdateCompletionStatus::UpdateSuccess_NoChange);
				break;
			case EHotfixResult::Failed:
				CheckComplete(EUpdateCompletionStatus::UpdateFailure_HotfixCheck);
				break;
			default:
				ensure(0 && "No other result codes should reach here!");
				CheckComplete(EUpdateCompletionStatus::UpdateFailure_HotfixCheck);
				break;
		}
	};

	// Debug delay delegate firing
	DelayResponse(CompletionDelegate, HotfixAvailabilityCheckCompleteDelay);
}

void UUpdateManager::StartInitialPreload()
{
	if (TSharedPtr<IUpdateManagerStatsCollector> StrongStatsCollector = StatsCollector.Pin())
	{
		StrongStatsCollector->OnCheckStep_InitialPreloadStarted();
	}

	SetUpdateState(EUpdateState::WaitingOnInitialLoad);

	// Start ticking
	FTSTicker& Ticker = FTSTicker::GetCoreTicker();
	FTickerDelegate TickDelegate = FTickerDelegate::CreateUObject(this, &ThisClass::Tick);
	ensure(!TickerHandle.IsValid());
	TickerHandle = Ticker.AddTicker(TickDelegate, 0.0f);

	LoadStartTime = FPlatformTime::Seconds();
	WorstNumFilesPendingLoadViewed = GetNumAsyncPackages();
}

void UUpdateManager::InitialPreloadComplete()
{
	if (TSharedPtr<IUpdateManagerStatsCollector> StrongStatsCollector = StatsCollector.Pin())
	{
		StrongStatsCollector->OnCheckStep_InitialPreloadComplete();
	}

	SetUpdateState(EUpdateState::InitialLoadComplete);

	if (LastPatchCheckResult == EInstallBundleManagerPatchCheckResult::PatchCheckFailure)
	{
		CheckComplete(EUpdateCompletionStatus::UpdateFailure_PatchCheck);
	}
	else if (LastPatchCheckResult == EInstallBundleManagerPatchCheckResult::ClientPatchRequired)
	{
		CheckComplete(EUpdateCompletionStatus::UpdateSuccess_NeedsPatch);
	}
	else if (LastPatchCheckResult == EInstallBundleManagerPatchCheckResult::ContentPatchRequired)
	{
		CheckComplete(EUpdateCompletionStatus::UpdateSuccess_NeedsRelaunch);
	}
	else
	{
		ensure(LastPatchCheckResult == EInstallBundleManagerPatchCheckResult::NoPatchRequired);
		// Patch check success, check hotfix status
		switch (LastHotfixResult)
		{
			case EHotfixResult::Success:
				CheckComplete(EUpdateCompletionStatus::UpdateSuccess);
				break;
			case EHotfixResult::SuccessNoChange:
				CheckComplete(EUpdateCompletionStatus::UpdateSuccess_NoChange);
				break;
			case EHotfixResult::Failed:
				CheckComplete(EUpdateCompletionStatus::UpdateFailure_HotfixCheck);
				break;
			case EHotfixResult::SuccessNeedsRelaunch:
				CheckComplete(EUpdateCompletionStatus::UpdateSuccess_NeedsRelaunch);
				break;
			case EHotfixResult::SuccessNeedsReload:
				CheckComplete(EUpdateCompletionStatus::UpdateSuccess_NeedsReload);
				break;
		}
	}
}

void UUpdateManager::SetUpdateState(EUpdateState NewState)
{
	if (CurrentUpdateState != NewState)
	{
		FString OldStateStr = UpdateStateEnum ? *UpdateStateEnum->GetNameStringByValue((int64)CurrentUpdateState) : TEXT("Invalid");
		FString NewStateStr = UpdateStateEnum ? *UpdateStateEnum->GetNameStringByValue((int64)NewState) : TEXT("Invalid");

#if CSV_PROFILER_STATS || MISCTRACE_ENABLED
		// Add timing regions
		if (CurrentUpdateState != EUpdateState::UpdateIdle)
		{
			CSV_TRACE_REGION_END(*OldStateStr, TEXT("Hotfix"))
		}
		if (NewState != EUpdateState::UpdateIdle && NewState != EUpdateState::UpdateComplete)
		{
			CSV_TRACE_REGION_BEGIN(*NewStateStr, TEXT("Hotfix"))
		}
#endif

		UE_LOGF(LogHotfixManager, Display, "Update State %ls -> %ls", *OldStateStr, *NewStateStr);
		CurrentUpdateState = NewState;
		OnUpdateStatusChanged().Broadcast(NewState);
	}
}

bool UUpdateManager::Tick(float InDeltaTime)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_UUpdateManager_Tick);
	if (CurrentUpdateState == EUpdateState::WaitingOnInitialLoad)
	{
		WorstNumFilesPendingLoadViewed = FMath::Max(GetNumAsyncPackages(), WorstNumFilesPendingLoadViewed);
		//UE_LOGF(LogInit, Log, "Load progress %f (%d %d)", GetLoadProgress(), GetNumAsyncPackages(), WorstNumFilesPendingLoadViewed);

		if (!IsAsyncLoading())
		{
			const double LoadTime = FPlatformTime::Seconds() - LoadStartTime;
			UE_LOGF(LogHotfixManager, Log, "Finished initial load/hotfix phase in %fs", LoadTime);
			ACCUM_LOADTIME(TEXT("FinishedInitialLoadHotfix"), LoadTime);

			InitialPreloadComplete();

			TickerHandle.Reset();
			return false;
		}
	}

	return true;
}

void UUpdateManager::PostInitProperties()
{
	Super::PostInitProperties();

	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		PopulateContextDefinitions();
	}
}

void UUpdateManager::PostReloadConfig(FProperty* PropertyThatWasLoaded)
{
	Super::PostReloadConfig(PropertyThatWasLoaded);

	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		PopulateContextDefinitions();
	}
}

float UUpdateManager::GetLoadProgress() const
{
	const int32 CurrentNumFiles = GetNumAsyncPackages();
	return (WorstNumFilesPendingLoadViewed > 0) ? FMath::Clamp((WorstNumFilesPendingLoadViewed - CurrentNumFiles) / (float)WorstNumFilesPendingLoadViewed, 0.0f, 1.0f) : 0.0f;
}

bool UUpdateManager::IsHotfixingEnabled() const
{
	if (GIsEditor)
	{
		return false;
	}

	return FLoadingScreenConfig::CheckForHotfixes();
}

bool UUpdateManager::IsBlockingForInitialLoadEnabled() const
{
	return FLoadingScreenConfig::ShouldBlockOnInitialLoad();
}

void UUpdateManager::RegisterDelegates()
{
	FCoreDelegates::ApplicationWillDeactivateDelegate.AddUObject(this, &ThisClass::OnApplicationWillDeactivate);
	FCoreDelegates::ApplicationHasReactivatedDelegate.AddUObject(this, &ThisClass::OnApplicationHasReactivated);
}

void UUpdateManager::UnregisterDelegates()
{
	FCoreDelegates::ApplicationWillDeactivateDelegate.RemoveAll(this);
	FCoreDelegates::ApplicationHasReactivatedDelegate.RemoveAll(this);
}

void UUpdateManager::OnApplicationWillDeactivate()
{
	DeactivatedTime = FDateTime::UtcNow();
}

void UUpdateManager::OnApplicationHasReactivated()
{
	FDateTime Now = FDateTime::UtcNow();

	if ((Now - DeactivatedTime).GetTotalSeconds() > AppSuspendedUpdateCheckTimeSeconds)
	{
		StartUpdateCheck(TEXT("ApplicationReactivated"));
	}
}

FTSTicker::FDelegateHandle UUpdateManager::DelayResponse(DelayCb&& Delegate, float Delay)
{
	return FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateWeakLambda(this, [this, Delegate](float dts)
		{
			Delegate();
			return false;
		}), Delay);
}

void UUpdateManager::PopulateContextDefinitions()
{
	ProcessedUpdateContextDefinitions.Empty();
	ProcessedUpdateContextDefinitions.Reserve(UpdateContextDefinitions.Num());

	// Elements later in the list will override elements with the same name which appear earlier.
	Algo::Transform(UpdateContextDefinitions, ProcessedUpdateContextDefinitions,
	[](const FUpdateContextDefinition& Definition)
	{
		return TTuple<FString, FUpdateContextDefinition>(Definition.Name, Definition);
	});
}

const FUpdateContextDefinition& UUpdateManager::GetContextDefinition(const FString& ContextName) const
{
	if (const FUpdateContextDefinition* Definiton = ProcessedUpdateContextDefinitions.Find(ContextName))
	{
		return *Definiton;
	}

	UE_LOGF(LogHotfixManager, Warning, "Failed to find update context \"%ls\". Using \"Unknown\" definition.", *ContextName);
	return UpdateContextDefinitionUnknown;
}

UWorld* UUpdateManager::GetWorld() const
{
	UGameInstance* GameInstance = GetTypedOuter<UGameInstance>();
	return GameInstance->GetWorld();
}

UGameInstance* UUpdateManager::GetGameInstance() const
{
	return GetTypedOuter<UGameInstance>();
}

FString LexToString(EUpdateCompletionStatus Status)
{
	switch (Status)
	{
		case EUpdateCompletionStatus::UpdateSuccess:
			return TEXT("UpdateSuccess");
		case EUpdateCompletionStatus::UpdateSuccess_NoChange:
			return TEXT("UpdateSuccess_NoChange");
		case EUpdateCompletionStatus::UpdateSuccess_NeedsReload:
			return TEXT("UpdateSuccess_NeedsReload");
		case EUpdateCompletionStatus::UpdateSuccess_NeedsRelaunch:
			return TEXT("UpdateSuccess_NeedsRelaunch");
		case EUpdateCompletionStatus::UpdateSuccess_NeedsPatch:
			return TEXT("UpdateSuccess_NeedsPatch");
		case EUpdateCompletionStatus::UpdateFailure_PatchCheck:
			return TEXT("UpdateFailure_PatchCheck");
		case EUpdateCompletionStatus::UpdateFailure_HotfixCheck:
			return TEXT("UpdateFailure_HotfixCheck");
		case EUpdateCompletionStatus::UpdateFailure_NotLoggedIn:
			return TEXT("UpdateFailure_NotLoggedIn");
		case EUpdateCompletionStatus::UpdateUnknown:
		default:
			return TEXT("UpdateUnknown");
	}
}
