// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "Misc/ConfigCacheIni.h"
#include "UObject/NameTypes.h"

#include "AssetHotfixRegistry.generated.h"

#define UE_API HOTFIX_API

class UOnlineHotfixManager;

UENUM(MinimalAPI)
enum class EHotfixOperation : uint8
{
	RowUpdate,
	AddRow,
	TableUpdate,
	CurveUpdate
};

struct FPendingAssetHotfix
{
	FName AssetClassName;
	FName RowName;
	FString ColumnOrData;
	FString Value;
	FName SourceTag;
	EHotfixOperation Operation = EHotfixOperation::RowUpdate;

	// Value is intentionally excluded from comparison. Two hotfix entries that
	// target the same asset, row, and column represent the same logical hotfix
	// regardless of the value being applied. This allows later entries to
	// replace earlier ones when deduplicating via TArray::AddUnique.
	bool operator==(const FPendingAssetHotfix& Other) const
	{
		return SourceTag == Other.SourceTag
			&& Operation == Other.Operation
			&& AssetClassName == Other.AssetClassName
			&& RowName == Other.RowName
			&& ColumnOrData == Other.ColumnOrData;
	}
};

struct FPendingAssetHotfixEntries
{
	TArray<FPendingAssetHotfix> Entries;
};

struct FAssetHotfixRegistry
{
	UE_API void Reset();
	UE_API TArray<FName> ParseAndStoreHotfixEntries(FName SourceTag, const FConfigSectionMap& AssetHotfixSection);
	UE_API TArray<FName> ParseAndStoreFromRawIniData(FName SourceTag, const FString& IniData);
	UE_API const FPendingAssetHotfixEntries* FindHotfixesForAsset(FName AssetPath) const;
	UE_API void StoreHotfixEntry(FName AssetPath, FPendingAssetHotfix Hotfix);
	UE_API bool RemoveEntriesFromSource(FName SourceTag);

	UE_API TArray<FName> GetAllPendingPaths() const;

	UE_API bool TryApplyHotfixes(
		UObject* Asset,
		UOnlineHotfixManager* HotfixManager,
		bool& bOutEncounteredErrors,
		TSet<class UDataTable*>* ChangedDataTables = nullptr,
		TSet<class UCurveTable*>* ChangedCurveTables = nullptr) const;

	bool HasPendingHotfixes() const { return PendingAssetHotfixes.Num() > 0; }

private:
	TMap<FName, FPendingAssetHotfixEntries> PendingAssetHotfixes;
};

#undef UE_API
