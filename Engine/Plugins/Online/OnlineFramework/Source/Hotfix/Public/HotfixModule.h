// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"

/**
 * Module for Hotfix base implementation
 */
class FHotfixModule : 
	public IModuleInterface
{
private:
// IModuleInterface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
};
