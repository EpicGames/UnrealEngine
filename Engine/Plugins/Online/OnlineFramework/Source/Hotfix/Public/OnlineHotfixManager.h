// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetHotfixRegistry.h"
#include "Misc/ConfigCacheIni.h"
#include "Interfaces/OnlineTitleFileInterface.h"

#include "OnlineHotfixManager.generated.h"

#define UE_API HOTFIX_API

struct FCloudFileHeader;

HOTFIX_API DECLARE_LOG_CATEGORY_EXTERN(LogHotfixManager, Display, All);

class FAsyncLoadingFlushContext;

UENUM()
enum class EHotfixResult : uint8
{
	/** Failed to apply the hotfix */
	Failed,
	/** Hotfix succeeded and is ready to go */
	Success,
	/** Hotfix process succeeded but there were no changes applied */
	SuccessNoChange,
	/** Hotfix succeeded and requires the current level to be reloaded to take effect */
	SuccessNeedsReload,
	/** Hotfix succeeded and requires the process restarted to take effect */
	SuccessNeedsRelaunch
};

/**
 * Delegate fired when a check for hotfix files (but not application) completes
 *
 * @param EHotfixResult status on what happened
 */
DECLARE_DELEGATE_OneParam(FOnHotfixAvailableComplete, EHotfixResult);

/**
 * Delegate fired when the hotfix process has completed
 *
 * @param EHotfixResult status on what happened
 */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnHotfixComplete, EHotfixResult);
typedef FOnHotfixComplete::FDelegate FOnHotfixCompleteDelegate;

/**
 * Delegate fired as progress of hotfix file reading happens
 *
 * @param NumDownloaded the number of files downloaded so far
 * @param TotalFiles the total number of files part of the hotfix
 * @param NumBytes the number of bytes processed so far
 * @param TotalBytes the total size of the hotfix data
 */
DECLARE_MULTICAST_DELEGATE_FourParams(FOnHotfixProgress, uint32, uint32, uint64, uint64);
typedef FOnHotfixProgress::FDelegate FOnHotfixProgressDelegate;

/**
 * Delegate fired for each new/updated file after it is applied
 *
 * @param FriendlyName the human readable version of the file name (DefaultEngine.ini)
 * @param CachedFileName the full path to the file on disk
 */
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnHotfixProcessedFile, const FString&, const FString&);
typedef FOnHotfixProcessedFile::FDelegate FOnHotfixProcessedFileDelegate;

/**
 * Delegate fired for each removed file
 *
 * @param FriendlyName the human readable version of the file name (DefaultEngine.ini)
 */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnHotfixRemovedFile, const FString&);
typedef FOnHotfixRemovedFile::FDelegate FOnHotfixRemovedFileDelegate;

/**
 * Delegate fired for each added/updated file
 *
 * @param FriendlyName the human readable version of the file name (DefaultEngine.ini)
 * @param FileContents the preprocessed contents of the file.
 */
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnHotfixUpdatedFile, const FString&, const TArray<uint8>&);
typedef FOnHotfixUpdatedFile::FDelegate FOnHotfixUpdatedFileDelegate;

/**
 * This class manages the downloading and application of hotfix data
 * Hotfix data is a set of non-executable files downloaded and applied to the game.
 * The base implementation knows how to handle INI, PAK, and locres files.
 * NOTE: Each INI/PAK file must be prefixed by the platform name they are targeted at
 */
UCLASS(MinimalAPI, Config=Engine)
class UOnlineHotfixManager :
	public UObject
{
	GENERATED_BODY()

public:
	// Whether to reapply all hotfixes every time they're rechecked (for debugging).
#if UE_BUILD_SHIPPING
	static constexpr bool bAlwaysReapplyHotfixes = false;
#else
	static bool bAlwaysReapplyHotfixes;
#endif

protected:
	/** The online interface to use for downloading the hotfix files */
	IOnlineTitleFilePtr OnlineTitleFile;

	/** Callbacks for when the title file interface is done */
	FOnEnumerateFilesCompleteDelegate OnEnumerateFilesCompleteDelegate;
	FOnReadFileProgressDelegate OnReadFileProgressDelegate;
	FOnReadFileCompleteDelegate OnReadFileCompleteDelegate;
	FDelegateHandle OnEnumerateFilesCompleteDelegateHandle;
	FDelegateHandle OnEnumerateFilesForAvailabilityCompleteDelegateHandle;
	FDelegateHandle OnReadFileProgressDelegateHandle;
	FDelegateHandle OnReadFileCompleteDelegateHandle;

	/** Callback from FCoreUObjectDelegates when hotfixable asset is loaded */
	FDelegateHandle OnHotfixableAssetLoadedHandle;

	/**
	 * Delegate fired when the hotfix process has completed
	 *
	 * @param status of the hotfix process
	 */
	DEFINE_ONLINE_DELEGATE_ONE_PARAM(OnHotfixComplete, EHotfixResult);

	/**
	 * Delegate fired as the hotfix files are read
	 */
	DEFINE_ONLINE_DELEGATE_FOUR_PARAM(OnHotfixProgress, uint32, uint32, uint64, uint64);

	/**
	 * Delegate fired for each new/updated file after it is applied
	 */
	DEFINE_ONLINE_DELEGATE_TWO_PARAM(OnHotfixProcessedFile, const FString&, const FString&);

	/**
	 * Delegate fired for each removed file
	 */
	DEFINE_ONLINE_DELEGATE_ONE_PARAM(OnHotfixRemovedFile, const FString&);

	/**
	 * Delegate fired for each added/updated file
	 */
	DEFINE_ONLINE_DELEGATE_TWO_PARAM(OnHotfixUpdatedFile, const FString&, const TArray<uint8>&);

	struct FPendingFileDLProgress
	{
		uint64 Progress;

		FPendingFileDLProgress()
		{
			Progress = 0;
		}
	};

	struct FConfigFileBackup
	{
		/** Name of the ini file backed up*/
		FString IniName;
		/** Previous ini data backed up */
		FConfigFile ConfigData;
		/** UClasses reloaded as a result of the current ini */
		TArray<FString> ClassesReloaded;
	};

	/** Holds which files are pending download */
	TMap<FString, FPendingFileDLProgress> PendingHotfixFiles;
	/** The filtered list of files that are part of the hotfix */
	TArray<FCloudFileHeader> HotfixFileList;
	/** The last set of hotfix files that was applied so we can determine whether we are up to date or not */
	TArray<FCloudFileHeader> LastHotfixFileList;
	/** The set of hotfix files that have changed from the last time we applied them */
	TArray<FCloudFileHeader> ChangedHotfixFileList;
	/** The set of hotfix files that have been removed from the last time we applied them */
	TArray<FCloudFileHeader> RemovedHotfixFileList;
	/** Holds which files have been mounted for unmounting */
	TArray<FString> MountedPakFiles;
	/** Backup copies of INI files that change during hotfixing so they can be undone afterward */
	TArray<FConfigFileBackup> IniBackups;
	/** Used to match any PAK files for this platform */
	FString PlatformPrefix;
	/** Used to match any server-only hotfixes */
	FString ServerPrefix;
	/** Used to match any custom config hotfixes */
	FString CustomConfigPrefix;
	/** Normally will be "Default" but could be different if we have a debug prefix */
	FString DefaultPrefix;
	/** Holds a chunk of string that will be swapped for Game during processing pak files (MyGame/Content/Maps -> /Game/Maps) */
	FString GameContentPath;
	/** Tracks how many files are being processed as part of the hotfix */
	uint32 TotalFiles;
	uint32 NumDownloaded;
	/** Tracks the size of the files being processed as part of the hotfix */
	uint64 TotalBytes;
	uint64 NumBytes;
	/** Some title file interfaces aren't re-entrant so handle it ourselves */
	bool bHotfixingInProgress;
	/** Asynchronously flush async loading before starting the hotfixing process. */
	TUniquePtr<FAsyncLoadingFlushContext> AsyncFlushContext;
	/** Set to true if any PAK file contains an update to a level that is currently loaded */
	bool bHotfixNeedsMapReload;
#if !UE_BUILD_SHIPPING
	/** Whether we want to log all of the files that are in a mounted pak file or not */
	bool bLogMountedPakContents;
#endif
	/**
	 * If we have removed or changed a currently mounted PAK file, then we'll need to restart the app
	 * because there's no simple undo for objects that were loaded and possibly rooted
	 */
	uint32 ChangedOrRemovedPakCount;
	/** Our passed-in World */
	TWeakObjectPtr<UWorld> OwnerWorld;
	/** Loaded hotfix contents that were not mapped to any known branch, but might be loaded later.
	The key used here is a plugin tag + branch name, such as "PluginEngine" for example. */
	TMap<FName, TArray<TPair<FString, FString>>> DynamicHotfixContents;

	UE_API virtual void Init();
	UE_API virtual void Cleanup();
	/** Looks at each file returned via the hotfix and processes them */
	UE_API EHotfixResult ApplyHotfix();
	/** Cleans up and fires the delegate indicating it's done */
	UE_DEPRECATED(5.8, "Renamed to FinalizeHotfixProcess")
	void TriggerHotfixComplete(EHotfixResult HotfixResult) { FinalizeHotfixProcess(HotfixResult); }
	/** Finalizes the hotfix process. Handles asset hotfixing and fires delegates when complete. */
	UE_API void FinalizeHotfixProcess(EHotfixResult HotfixResult);
	/** Checks each file listed to see if it is a hotfix file to process */
	UE_API void FilterHotfixFiles();
	/** Starts the async reading process for the hotfix files */
	UE_API void ReadHotfixFiles();
	/** Unmounts any changed PAK files so they can be re-mounted after downloading */
	UE_API void UnmountHotfixFiles();
	/** Stores off the INI file for restoration later */
	UE_API FConfigFileBackup& BackupIniFile(const FString& IniName, const FConfigFile* ConfigFile);
	/** Restores any changed INI files to their default loaded state */
	UE_API void RestoreBackupIniFiles();
	/** Builds the list of files that are different between two runs of the hotfix process */
	UE_API void BuildHotfixFileListDeltas();

	/** Called once the list of hotfix files has been retrieved */
	UE_API void OnEnumerateFilesComplete(bool bWasSuccessful, const FString& ErrorStr);
	/** Called once the list of hotfix files has been retrieved and we only want to see if a hotfix is necessary */
	UE_API void OnEnumerateFilesForAvailabilityComplete(bool bWasSuccessful, const FString& ErrorStr, FOnHotfixAvailableComplete InCompletionDelegate);
	/** Called as files are downloaded to determine when to apply the hotfix data */
	UE_API void OnReadFileComplete(bool bWasSuccessful, const FString& FileName);
	/** Called as files are downloaded to provide progress notifications */
	UE_API void OnReadFileProgress(const FString& FileName, uint64 BytesRead);

	/** @return the config file entry for the ini file name in question */
	UE_API FConfigFile* GetConfigFile(const FString& IniName);
	UE_API FConfigBranch* GetBranch(const FString& IniName);

	/** @return the config cache key used to associate ini file entries within the config cache */
	UE_API FString BuildConfigCacheKey(const FString& IniName);
	/** @return the config file name after stripping any extra info (platform, debug prefix, etc.) */
	UE_API virtual FString GetStrippedConfigFileName(const FString& IniName);

	/** @return the human readable name of the file */
	UE_API const FString GetFriendlyNameFromDLName(const FString& DLName) const;

	UE_API virtual void PostInitProperties() override;
	UE_API virtual void BeginDestroy() override;

	UE_API bool IsMapLoaded(const FString& MapName);

	/** @return our current world */
	UE_API UWorld* GetWorld() const override;

	/** Stop tracking hotfixed assets marked as garbage */
	UE_API void StopTrackingInvalidHotfixedAssets();

	/** Hotfix a dynamic config branch that was just loaded */
	UE_API void HotfixDynamicBranch(const FName& Tag, const FName& Branch, class FConfigModificationTracker* ModificationTracker);

	/** Remove asset hotfix entries for a plugin whose dynamic config branch is being removed */
	UE_API void RemoveHotfixesFromDynamicBranch(const FName& Tag);

protected:

	/** Called after a tag's asset hotfix entries have been removed from the registry. */
	virtual void OnDynamicHotfixesRemoved(FName SourceTag) {}

	/** Store a single hotfix entry in the registry. Used by subclasses that build entries outside the INI parse path. */
	UE_API void RegisterAssetHotfixEntry(FName AssetPath, FPendingAssetHotfix Hotfix);

	/** Apply all registered hotfixes for an already-loaded asset, firing table change notifications. */
	UE_API void ApplyRegisteredHotfixesForAsset(UObject* Asset);

	/** Fires HandleDataTableChanged / OnCurveTableChanged for all modified tables. */
	UE_API static void BroadcastTableChanges(const TSet<class UDataTable*>& ChangedDataTables, const TSet<class UCurveTable*>& ChangedCurveTables);

	/**
	 * Is this hotfix file compatible with the current build
	 * If the file has version information it is compared with compatibility
	 * If the file has NO version information it is assumed compatible
	 *
	 * @param InFilename name of the file to check
	 * @param OutFilename name of file with version information stripped
	 *
	 * @return true if file is compatible, false otherwise
	 */
	UE_API bool IsCompatibleHotfixFile(const FString& InFilename, FString& OutFilename);

	/**
	 * Called when a file needs custom processing (see above). Override this to provide your own processing methods
	 *
	 * @param FileHeader - the header information for the file in question
	 *
	 * @return whether the file was successfully processed
	 */
	UE_API virtual bool ApplyHotfixProcessing(const FCloudFileHeader& FileHeader);
	/**
	 * Called prior to reading the file data.
	 *
	 * @param FileHeader - the header information for the file in question
	 * @param FileData - byte data of the hotfix file. Intentionally not const, so the array is modifiable as part of preprocessing.
	 *
	 * @return whether the file was successfully preprocessed
	 */
	virtual bool PreProcessDownloadedFileData(const FCloudFileHeader& FileHeader, TArray<uint8>& FileData) const { return true; }
	/**
	 * Override this to change the default INI file handling (merge delta INI changes into the config cache)
	 *
	 * @param FileName - the name of the INI being merged into the config cache
	 * @param IniData - the contents of the INI file (expected to be delta contents not a whole file)
	 *
	 * @return whether the merging was successful or not
	 */
	UE_API virtual bool HotfixIniFile(const FString& FileName, const FString& IniData);
	/**
	 * Override this to change the default PAK file handling:
	 *		- mount PAK file immediately
	 *		- Scan for any INI files contained within the PAK file and merge those in
	 *
	 * @param FileName - the name of the PAK file being mounted
	 *
	 * @return whether the mounting of the PAK file was successful or not
	 */
	UE_API virtual bool HotfixPakFile(const FCloudFileHeader& FileHeader);
	/**
	 * Override this to change the default INI file handling (merge whole INI files into the config cache)
	 *
	 * @param FileName - the name of the INI being merged into the config cache
	 *
	 * @return whether the merging was successful or not
	 */
	UE_API virtual bool HotfixPakIniFile(const FString& FileName);

	/**
	 * Override this to change the default caching directory
	 *
	 * @return the default caching directory
	 */
	virtual FString GetCachedDirectory()
	{
		return FPaths::ProjectPersistentDownloadDir();
	}

	/** Fired by FCoreUObjectDelegates when a hotfixable asset is loaded */
	UE_API virtual void OnHotfixableAssetLoaded(UObject* Asset);

	/** Notify used by CheckAvailability() */
	UE_API virtual void OnHotfixAvailablityCheck(const TArray<FCloudFileHeader>& PendingChangedFiles, const TArray<FCloudFileHeader>& PendingRemoveFiles);

	/** Finds the header associated with the file name */
	UE_API FCloudFileHeader* GetFileHeaderFromDLName(const FString& FileName);

	/** Fires the progress delegate with our updated progress */
	UE_API void UpdateProgress(uint32 FileCount, uint64 UpdateSize);

	virtual bool ShouldWarnAboutMissingWhenPatchingFromIni(const FString& AssetPath) const { return true; }

	/** Called after any hotfixes are applied to apply last-second changes to certain asset types from .ini file data */
	UE_API virtual void PatchAssetsFromIniFiles();

	/** Called instead of PatchAssetsFromIniFiles when hotfix-on-load is enabled */
	UE_API virtual void ParseAssetHotfixes();

	/** Called immediately after ParseAssetHotfixes to patch assets already in memory */
	UE_API virtual void ApplyHotfixesToLoadedAssets();

	/** Apply registered hotfixes only to the given asset paths. */
	UE_API void ApplyHotfixesToLoadedAssets(TConstArrayView<FName> AssetPaths);

	/** Called after any hotfixes are applied to apply last-second changes to Config properties from .ini file data */
	UE_API virtual void ReloadConfigsFromIniFiles();

	/** Used in PatchAssetsFromIniFiles to hotfix only a row in a table.
	 *  If ChangedTables is not null then HandleDataTableChanged will not be called and the caller should call it on the data tables in ChangedTables when they're ready to
	 */
	UE_API void HotfixRowUpdate(
		UObject* Asset,
		const FString& AssetPath,
		const FString& RowName,
		const FString& ColumnName,
		const FString& NewValue,
		TArray<FString>& ProblemStrings,
		TSet<class UDataTable*>* ChangedDataTables = nullptr,
		TSet<class UCurveTable*>* ChangedCurveTables = nullptr,
		FName SourceTag = NAME_None);

	/** Used in PatchAssetsFromIniFiles to hotfix a new row in a table.
	 *  If ChangedTables is not null then HandleDataTableChanged will not be called and the caller should call it on the data tables in ChangedTables when they're ready to
	 */
	UE_API void HotfixAddRow(
		UObject* Asset,
		const FString& AssetPath,
		const FString& JsonData,
		TArray<FString>& ProblemStrings,
		TSet<class UDataTable*>* ChangedDataTables = nullptr);

	/** Called after adding table row by HotfixAddRow() */
	virtual void OnHotfixTableAddRow(UObject& Asset, const FName RowName) {}

	/** Used in PatchAssetsFromIniFiles to hotfix an entire table. */
	UE_API void HotfixTableUpdate(UObject* Asset, const FString& AssetPath, const FString& JsonData, TArray<FString>& ProblemStrings);

	/**
	 * Called after modifying table values by HotfixRowUpdate().
	 * @deprecated Override the WithSource variants instead.
	 */
	UE_DEPRECATED(5.8, "Override OnHotfixTableValueInt64WithSource instead.")
	virtual void OnHotfixTableValueInt64(UObject& Asset, const FString& RowName, const FString& ColumnName, const int64& OldValue, const int64& NewValue) { }
	UE_DEPRECATED(5.8, "Override OnHotfixTableValueDoubleWithSource instead.")
	virtual void OnHotfixTableValueDouble(UObject& Asset, const FString& RowName, const FString& ColumnName, const double& OldValue, const double& NewValue) { }
	UE_DEPRECATED(5.8, "Override OnHotfixTableValueFloatWithSource instead.")
	virtual void OnHotfixTableValueFloat(UObject& Asset, const FString& RowName, const FString& ColumnName, const float& OldValue, const float& NewValue) { }
	UE_DEPRECATED(5.8, "Override OnHotfixTableValueStringWithSource instead.")
	virtual void OnHotfixTableValueString(UObject& Asset, const FString& RowName, const FString& ColumnName, const FString& OldValue, const FString& NewValue) { }
	UE_DEPRECATED(5.8, "Override OnHotfixTableValueNameWithSource instead.")
	virtual void OnHotfixTableValueName(UObject& Asset, const FString& RowName, const FString& ColumnName, const FName& OldValue, const FName& NewValue) { }
	UE_DEPRECATED(5.8, "Override OnHotfixTableValueObjectWithSource instead.")
	virtual void OnHotfixTableValueObject(UObject& Asset, const FString& RowName, const FString& ColumnName, const UObject* OldValue, const UObject* NewValue) { }
	UE_DEPRECATED(5.8, "Override OnHotfixTableValueSoftObjectWithSource instead.")
	virtual void OnHotfixTableValueSoftObject(UObject& Asset, const FString& RowName, const FString& ColumnName, const FSoftObjectPtr& OldValue, const FSoftObjectPtr& NewValue) { }

	/** Called when a table row value is patched, with the source tag identifying which hotfix applied the change. Override to track or respond to patches per source. */
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	virtual void OnHotfixTableValueInt64WithSource(UObject& Asset, const FString& RowName, const FString& ColumnName, const int64& OldValue, const int64& NewValue, FName SourceTag) { OnHotfixTableValueInt64(Asset, RowName, ColumnName, OldValue, NewValue); }
	virtual void OnHotfixTableValueDoubleWithSource(UObject& Asset, const FString& RowName, const FString& ColumnName, const double& OldValue, const double& NewValue, FName SourceTag) { OnHotfixTableValueDouble(Asset, RowName, ColumnName, OldValue, NewValue); }
	virtual void OnHotfixTableValueFloatWithSource(UObject& Asset, const FString& RowName, const FString& ColumnName, const float& OldValue, const float& NewValue, FName SourceTag) { OnHotfixTableValueFloat(Asset, RowName, ColumnName, OldValue, NewValue); }
	virtual void OnHotfixTableValueStringWithSource(UObject& Asset, const FString& RowName, const FString& ColumnName, const FString& OldValue, const FString& NewValue, FName SourceTag) { OnHotfixTableValueString(Asset, RowName, ColumnName, OldValue, NewValue); }
	virtual void OnHotfixTableValueNameWithSource(UObject& Asset, const FString& RowName, const FString& ColumnName, const FName& OldValue, const FName& NewValue, FName SourceTag) { OnHotfixTableValueName(Asset, RowName, ColumnName, OldValue, NewValue); }
	virtual void OnHotfixTableValueObjectWithSource(UObject& Asset, const FString& RowName, const FString& ColumnName, const UObject* OldValue, const UObject* NewValue, FName SourceTag) { OnHotfixTableValueObject(Asset, RowName, ColumnName, OldValue, NewValue); }
	virtual void OnHotfixTableValueSoftObjectWithSource(UObject& Asset, const FString& RowName, const FString& ColumnName, const FSoftObjectPtr& OldValue, const FSoftObjectPtr& NewValue, FName SourceTag) { OnHotfixTableValueSoftObject(Asset, RowName, ColumnName, OldValue, NewValue); }
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	UE_API virtual bool ShouldPerformHotfix();

	/** Allow the application to override the dedicated server filename prefix. */
	UE_API virtual FString GetDedicatedServerPrefix() const;

	/** Allow child classes to determine if specific assets should be hotfixed or not */
	UE_API virtual bool ShouldHotfixAsset(const FString& AssetPath) const;

#if !UE_BUILD_SHIPPING
	/** Test function that applies a local file as if it were a hotfix. */
	UE_API void ApplyLocalTestHotfix(FString Filename);
#endif

public:
	UE_API UOnlineHotfixManager();
	UE_API UOnlineHotfixManager(FVTableHelper& Helper);
	UE_API virtual ~UOnlineHotfixManager();

	/**
	 * Override this method to look at the file information for any game specific hotfix processing
	 * NOTE: Make sure to call Super to get default handling of files
	 *
	 * @param FileHeader - the information about the file to determine if it needs custom processing
	 *
	 * @return true if the file needs some kind of processing, false to have hotfixing ignore the file
	 */
	UE_API virtual bool WantsHotfixProcessing(const FCloudFileHeader& FileHeader);

	/** Tells the hotfix manager which OSS to use. Uses the default if empty */
	UPROPERTY(Config)
	FString OSSName;

	/** Tells the factory method which class to contruct */
	UPROPERTY(Config)
	FString HotfixManagerClassName;

	/** Used to prevent development work from interfering with playtests, etc. */
	UPROPERTY(Config)
	FString DebugPrefix;

	/** List of classes that can be hotfixed with [AssetHotfix] from ini files */
	UPROPERTY(Config)
	TArray<FSoftClassPath> HotfixableAssetClasses;

	/** Starts the fetching of hotfix data from the OnlineTitleFileInterface that is registered for this game */
	UFUNCTION(BlueprintCallable, Category="Hotfix")
	UE_API virtual void StartHotfixProcess();

	/** Called when the hotfix process is skipped entirely. */
	UE_API virtual void OnHotfixProcessSkipped();

	/** Array of objects that we're forcing to remain resident because we've applied live hotfixes and won't get an
	    opportunity to reapply changes if the object is evicted from memory. */
	UPROPERTY(Transient)
	TArray<TObjectPtr<UObject>> AssetsHotfixedFromIniFiles;

	/** 
	 * Check for available hotfix files (but do not apply them) 
	 *
	 * @param InCompletionDelegate delegate to fire when the check is complete
	 */
	UE_API virtual void CheckAvailability(FOnHotfixAvailableComplete& InCompletionDelegate);

	static UE_API UClass* GetConfiguredHotfixManagerClass();

	/** Factory method that returns the configured hotfix manager */
	static UE_API UOnlineHotfixManager* Get(UWorld* World);

	static UE_API void ReloadObjectsAffectedByConfigFile(const FString& IniDataFileName, const FString& IniData, const FString& ConfigFilename, TArray<FString>& ReloadedClassesPathNames, bool bUseLoadConfig);

	/** Whether hotfix-on-load is active (delegates bound, asset classes registered). */
	bool IsHotfixOnLoadEnabled() const { return bHotfixOnLoadEnabled; }

private:
	UPROPERTY(Transient)
	TArray<TObjectPtr<UClass>> ResolvedHotfixableAssetClasses;

	FAssetHotfixRegistry AssetHotfixRegistry;
	friend struct FAssetHotfixRegistry;

	/** Evaluates the hotfix-on-load CVar and configures if enabled. Latched to run once. */
	void ConfigureHotfixOnLoad();

	/** Tears down hotfix-on-load (unregisters asset classes, unbinds delegate). Idempotent. */
	void TeardownHotfixOnLoad();

	/** True after ConfigureHotfixOnLoad() has evaluated the CVar. Prevents re-evaluation on subsequent hotfix passes. */
	bool bConfiguredHotfixOnLoad = false;

	/** True when hotfix-on-load is active (delegates bound, asset classes registered). */
	bool bHotfixOnLoadEnabled = false;

	/** True after the first ApplyHotfixesToLoadedAssets() pass */
	bool bHasAppliedHotfixesToLoadedAssets = false;

#if !UE_BUILD_SHIPPING
	/** Validates that hotfix asset paths resolve to registered hotfixable classes in the asset registry. */
	void ValidateHotfixableAssetClasses(TConstArrayView<FName> AssetPaths);
#endif
};

#undef UE_API
