// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Ticker.h"
#include "Misc/DateTime.h"
#include "UpdateManager.generated.h"

#define UE_API HOTFIX_API

enum class EInstallBundleManagerPatchCheckResult : uint32;
enum class EPatchCheckResult : uint8;
struct FOnlineError;

class Error;
class FUniqueNetId;
class UGameInstance;
enum class EHotfixResult : uint8;

/**
 * Various states the update manager flows through as it checks for patches/hotfixes
 */
UENUM(BlueprintType)
enum class EUpdateState : uint8
{
	/** No updates in progress */
	UpdateIdle,
	/** An update is waiting to be triggered at the right time */
	UpdatePending,
	/** Checking for an available patch */
	CheckingForPatch,
	/** Checking with hotfix for available updates */
	CheckingForHotfix,
	/** Waiting for the async loading / preloading to complete */
	WaitingOnInitialLoad,
	/** Preloading complete */
	InitialLoadComplete,
	/** Last update check completed successfully */
	UpdateComplete
};

/**
 * Possible outcomes at the end of an update check
 */
UENUM(BlueprintType)
enum class EUpdateCompletionStatus : uint8
{
	/** Unknown update completion */
	UpdateUnknown,
	/** Update completed successfully, some changes applied */
	UpdateSuccess,
	/** Update completed successfully, no changed needed */
	UpdateSuccess_NoChange,
	/** Update completed successfully, need to reload the map */
	UpdateSuccess_NeedsReload,
	/** Update completed successfully, need to relaunch the game */
	UpdateSuccess_NeedsRelaunch,
	/** Update completed successfully, a patch must be download to continue */
	UpdateSuccess_NeedsPatch,
	/** Update failed in the patch check */
	UpdateFailure_PatchCheck,
	/** Update failed in the hotfix check */
	UpdateFailure_HotfixCheck,
	/** Update failed due to not being logged in */
	UpdateFailure_NotLoggedIn
};

HOTFIX_API FString LexToString(EUpdateCompletionStatus Status);

/**
 * Delegate fired when changes to the update progress have been made
 *
 * @param NewState newest state change while an update check is in progress
 */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnUpdateStatusChanged, EUpdateState /** NewState */)
typedef FOnUpdateStatusChanged::FDelegate FOnUpdateStatusChangedDelegate;

/**
 * Delegate fired when a single update check has completed
 *
 * @param Result result of the update check operation 
 */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnUpdateCheckComplete, EUpdateCompletionStatus /** Result */)
typedef FOnUpdateCheckComplete::FDelegate FOnUpdateCheckCompleteDelegate;

/**
 * Delegate fired when progress on a hotfix download is made
 *
 * @param NumDownloaded the number of files downloaded so far
 * @param TotalFiles the total number of files part of the hotfix
 * @param NumBytes the number of bytes processed so far
 * @param TotalBytes the total size of the hotfix data
 */
DECLARE_MULTICAST_DELEGATE_FourParams(FOnUpdateHotfixProgress, uint32 /** NumDownloaded */, uint32 /** TotalFiles */, uint64 /** NumBytes */, uint64 /** TotalBytes */)
typedef FOnUpdateHotfixProgress::FDelegate FOnUpdateHotfixProgressDelegate;

/**
 * Delegate fired when a single file hotfix is applied
 *
 * @param FriendlyName the human readable version of the file name (DefaultEngine.ini)
 * @param CachedFileName the full path to the file on disk
 */
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnUpdateHotfixProcessedFile, const FString& /** FriendlyName */, const FString& /** CachedFileName */);
typedef FOnUpdateHotfixProcessedFile::FDelegate FOnUpdateHotfixProcessedFileDelegate;

/**
 * Update context definition loaded from config.
 *
 * Defines the behavior to use during an update check.
 */ 
USTRUCT()
struct FUpdateContextDefinition
{
	GENERATED_BODY()
public:
	/** The name used to lookup the context definition. */
	UPROPERTY(Config)
	FString Name;
	/** Whether to perform any checks. */
	UPROPERTY(Config)
	bool bEnabled = true;
	/** Whether to perform an update, or to only check for the presence of an update. */
	UPROPERTY(Config)
	bool bCheckAvailabilityOnly = false;
	/** Whether to check for patches during the update process. */
	UPROPERTY(Config)
	bool bPatchCheckEnabled = true;
	/** Additional tags for extensibility. */
	UPROPERTY(Config)
	TSet<FString> AdditionalTags;
};

/**
 * Collector for aggregating stats related to the update check process.
 *
 * Implement collector and pass into RegisterStatsCollector to receive stats.
 */ 
class IUpdateManagerStatsCollector
{
public:
	virtual ~IUpdateManagerStatsCollector() = default;

	struct FCheckStartedParams
	{
		FString ContextName;
		bool bEnabled = true;
		bool bCheckAvailabilityOnly = false;
		bool bCachedResponse = false;
		double CacheExpiration = 0.0;
	};

	// Overall process/
	virtual void OnCheckStarted(FCheckStartedParams&& Params) = 0;
	virtual void OnCheckComplete(EUpdateCompletionStatus Result) = 0;

	// Individual stages.
	virtual void OnCheckStep_PatchCheckStarted() = 0;
	virtual void OnCheckStep_PatchCheckComplete(EInstallBundleManagerPatchCheckResult PatchResult) = 0;
	virtual void OnCheckStep_HotfixCheckStarted() = 0;
	virtual void OnCheckStep_HotfixCheckComplete(EHotfixResult Result) = 0;
	virtual void OnCheckStep_InitialPreloadStarted() = 0;
	virtual void OnCheckStep_InitialPreloadComplete() = 0;
};

/**
 * Update manager
 *
 * Checks the system and/or backend for the possibility of a patch and hotfix
 * Will not apply a hotfix if a pending patch is available
 * Notifies the game of the result of the check
 * - possibly requires UI to prevent user from playing if a patch is available
 * - possibly requires UI to prevent user from player if a hotfix requires a reload of existing data
 */ 
UCLASS(MinimalAPI, Config=Engine)
class UUpdateManager : public UObject
{
	GENERATED_BODY()

private:

	/** Status update listeners */
	FOnUpdateStatusChanged UpdateStatusChangedDelegates;
	/** Check completion listeners */
	FOnUpdateCheckComplete UpdateCheckCompleteDelegates;
	/** Hotfix download progress listeners */
	FOnUpdateHotfixProgress UpdateHotfixProgressDelegate;
	/** Hotfix application process listeners */
	FOnUpdateHotfixProcessedFile UpdateHotfixProcessedFile;

public:

	UE_API UUpdateManager();
	UE_API virtual ~UUpdateManager();

	/**
	 * Reset so you can call StartCheck again
	 */
	UE_API virtual void Reset();

	/**
	 * Start an patch and hotfix check
	 *
	 * @param bInCheckHotfixOnly check for the availability of a hotfix only (does not apply)
	 */
	UE_DEPRECATED_FORGAME(5.1, "UUpdateManager::StartCheck is deprecated, please use StartUpdateCheck instead.")
	UE_API virtual void StartCheck(bool bInCheckHotfixOnly = false);

	/**
	 * Start an update check using properties defined by context.
	 *
	 * @param ContextName The context definitions to use when updating.
	 */
	UE_API void StartUpdateCheck(const FString& ContextName);
	
	/**
	 *  @return the load progress (0..1)
	 */
	UE_API float GetLoadProgress() const;

	/**
	 * @return true if hotfixing is enabled
	 */
	UE_API bool IsHotfixingEnabled() const;

	/**
	 * @return true if blocking for initial load is enabled
	 */
	UE_API bool IsBlockingForInitialLoadEnabled() const;

	/**
	 * Put the update manager in a pending state so it can alert the game that a check is imminent
	 */
	UE_API void SetPending();

	/**
	 * @return true if at least one update been completed
	 */
	bool HasCompletedInitialUpdate() const { return bInitialUpdateFinished; }

	/**
	 * @return true if the update manager is actively checking or about to
	 */
	bool IsUpdating() const { return !bCheckHotfixAvailabilityOnly && (CurrentUpdateState != EUpdateState::UpdateIdle) && (CurrentUpdateState != EUpdateState::UpdateComplete); }

	/**
	* @return true if the update manager is actively checking
	*/
	bool IsActivelyUpdating() const { return IsUpdating() && (CurrentUpdateState != EUpdateState::UpdatePending); }

	/**
	 * Register a collector for acquiring stats related to the update check process.
	 */
	UE_API void RegisterStatsCollector(const TSharedPtr<IUpdateManagerStatsCollector>& InStatsCollector);

	/**
	 * @return the current state of the update check process
	 */
	EUpdateState GetUpdateState() const { return CurrentUpdateState; }

	/**
	 * @return the last result of the update manager
	 */
	EUpdateCompletionStatus GetCompletionResult() const { return LastCompletionResult[0]; }

	/**
	 * @return delegate triggered when update status has changed 
	 */
	FOnUpdateStatusChanged& OnUpdateStatusChanged() { return UpdateStatusChangedDelegates; }

	/**
	 * @return delegate triggered when update check is complete
	 */
	FOnUpdateCheckComplete& OnUpdateCheckComplete() { return UpdateCheckCompleteDelegates; }

	/**
	 * @return delegate triggered when hotfix file download progress has been made
	 */
	FOnUpdateHotfixProgress& OnUpdateHotfixProgress() { return UpdateHotfixProgressDelegate; }

	/**
	 * @return delegate triggered when hotfix file is applied
	 */
	FOnUpdateHotfixProcessedFile& OnUpdateHotfixProcessedFile() { return UpdateHotfixProcessedFile; }

protected:

	enum class EUpdateStartResult : uint8
	{
		/** Update did not start */
		None,
		/** Update has started */
		UpdateStarted,
		/** Cached result is going to be returned */
		UpdateCached
	};

	/** @return true if update checks are enabled */
	UE_API virtual bool ChecksEnabled() const;
	/** @return true if the backend environment requires update checks */
	UE_DEPRECATED_FORGAME(4.25, "UUpdateManager::EnvironmentWantsPatchCheck is deprecated, please override FPatchCheck::EnvironmentWantsPatchCheck instead.")
	UE_API virtual bool EnvironmentWantsPatchCheck() const;

	/** 
	 * Internal call for StartCheck
	 *
	 * @return whether or not the check started, returned a cached value, or did nothing (already in progress)
	 */
	UE_DEPRECATED_FORGAME(5.1, "UUpdateManager::StartCheckInternal is deprecated, please override StartCheckUpdateInternal instead.")
	UE_API virtual EUpdateStartResult StartCheckInternal(bool bInCheckHotfixOnly);

	/** 
	 * Internal call for StartUpdateCheck
	 *
	 * @return whether or not the check started, returned a cached value, or did nothing (already in progress)
	 */
	UE_API virtual EUpdateStartResult StartUpdateCheckInternal(const FUpdateContextDefinition& ContextDefinition);

	/** Tick function during initial preload */
	UE_API virtual bool Tick(float DeltaTime);

	UE_API virtual void PostInitProperties() override;
	UE_API virtual void PostReloadConfig(FProperty* PropertyThatWasLoaded) override;

	/** Amount of time to wait before starting an update check. */
	UPROPERTY(Config)
	float UpdateCheckStartDelay;
	/** Amount of time to wait before returning a cached response when updates are started. */
	UPROPERTY(Config)
	float UpdateCheckCachedResponseDelay;
	/** Amount of time to wait between the internal hotfix check completing and advancing to the next stage */
	UPROPERTY(Config)
	float HotfixCheckCompleteDelay;
	/** Amount of time to wait at the end of the entire check before notifying listening entities */
	UPROPERTY(Config)
	float UpdateCheckCompleteDelay;
	/** Amount of time to wait between the internal hotfix availability check completing and advancing to the next stage */
	UPROPERTY(Config)
	float HotfixAvailabilityCheckCompleteDelay;
	/** Amount of time to wait at the end of the entire check before notifying listening entities (availability check only) */
	UPROPERTY(Config)
	float UpdateCheckAvailabilityCompleteDelay;

	/** Check the platform OSS for an update */
	UE_DEPRECATED(4.25, "Set FPatchCheck::bCheckPlatformOSSForUpdate using section [PatchCheck] instead.")
	bool bCheckPlatformOSSForUpdate;
	/** Check the default OSS for an update */
	UE_DEPRECATED(4.25, "Set FPatchCheck::bCheckOSSForUpdate using section [PatchCheck] instead.")
	bool bCheckOSSForUpdate;

	/**
	 * Patch check
	 */
	UE_API virtual void StartPatchCheck();
	UE_API void InstallBundlePatchCheckComplete(EInstallBundleManagerPatchCheckResult PatchResult);
	UE_API void PatchCheckComplete(EPatchCheckResult PatchResult);

	/**
	 * Hotfix check
	 */
	UE_API virtual void StartHotfixCheck();
	UE_API virtual void OnHotfixProgress(uint32 NumDownloaded, uint32 TotalFiles, uint64 NumBytes, uint64 TotalBytes);
	UE_API virtual void OnHotfixProcessedFile(const FString& FriendlyName, const FString& CachedName);
	UE_API virtual void OnHotfixCheckComplete(EHotfixResult Result);

	/** Check for the availability of changed hotfix files only */
	UE_API virtual void StartHotfixAvailabilityCheck();
	/** Availability check complete */
	UE_API virtual void HotfixAvailabilityCheckComplete(EHotfixResult Result);

	/**
	 * Preload game assets after patch/hotfix check is complete but before game is alerted
	 */
	UE_API virtual void StartInitialPreload();

	/**
	 * Initial preload of assets is complete
	 */
	UE_API virtual void InitialPreloadComplete();

	/** 
	 * Announce that the update check has completed
	 *
	 * @param Result result of the entire update check (patch/hotfix/etc)
	 * @param bUpdateTimestamp whether or not to update the cache timer
	 */
	UE_API virtual void CheckComplete(EUpdateCompletionStatus Result, bool bUpdateTimestamp = true);

	/** Change the state of the update manager */
	UE_API virtual void SetUpdateState(EUpdateState NewState);

	/** If application is suspended longer than this, trigger an update check when resuming */
	UPROPERTY(Config)
	int32 AppSuspendedUpdateCheckTimeSeconds;

	/** Time when application was deactivated */
	FDateTime DeactivatedTime;

	/** Register deactivate/reactivate delegates */
	UE_API void RegisterDelegates();
	/** Unregister deactivate/reactivate delegates */
	UE_API void UnregisterDelegates();

	/** Called when application is about to be deactivated */
	UE_API virtual void OnApplicationWillDeactivate();
	/** Called when application has been reactivated */
	UE_API virtual void OnApplicationHasReactivated();

protected:
	/** Has the first update completed */
	UPROPERTY()
	bool bInitialUpdateFinished;

	/** Is this run only checking and not applying */
	UPROPERTY()
	bool bCheckHotfixAvailabilityOnly;

	/** Current state of the update */
	UPROPERTY()
	EUpdateState CurrentUpdateState;

	/** What was the maximum number of pending async loads we've seen so far */
	UPROPERTY()
	int32 WorstNumFilesPendingLoadViewed;

	/** Result of the last patch check */	
	EInstallBundleManagerPatchCheckResult LastPatchCheckResult;

	/** Result of the last hotfix */
	UPROPERTY()
	EHotfixResult LastHotfixResult;
	/** Delegates to hotfix updates */
	FDelegateHandle HotfixCompleteDelegateHandle;
	FDelegateHandle HotfixProgressDelegateHandle;
	FDelegateHandle HotfixProcessedFileDelegateHandle;

	/** The time at which we started the initial load after updates completed */
	double LoadStartTime;

	/** Timestamp of last update check (0:normal, 1:availability only) */
	UPROPERTY()
	FDateTime LastUpdateCheck[2];
	/** Last update check result (0:normal, 1:availability only) */
	UPROPERTY()
	EUpdateCompletionStatus LastCompletionResult[2];

	FTSTicker::FDelegateHandle TickerHandle;
	FTSTicker::FDelegateHandle StartCheckInternalTimerHandle;

	TWeakPtr<IUpdateManagerStatsCollector> StatsCollector;

private:

	/**
	 * Helpers
	 */

	/** String output */
	UPROPERTY()
	TObjectPtr<UEnum> UpdateStateEnum;
	UPROPERTY()
	TObjectPtr<UEnum> UpdateCompletionEnum;

	/** Fire a delegate after a given amount of time */
	typedef TFunction<void(void)> DelayCb;
	UE_API FTSTicker::FDelegateHandle DelayResponse(DelayCb&& Delegate, float Delay);

	friend bool SkipPatchCheck(UUpdateManager* UpdateManager);

private:
	UE_API void PopulateContextDefinitions();
	UE_API const FUpdateContextDefinition& GetContextDefinition(const FString& ContextName) const;

	// Definition to use when a context lookup fails.
	UPROPERTY(Config)
	FUpdateContextDefinition UpdateContextDefinitionUnknown;

	// Definitions for update behavior.
	UPROPERTY(Config)
	TArray<FUpdateContextDefinition> UpdateContextDefinitions;

	// Update context definitions which have been processed following a config update.
	TMap<FString, FUpdateContextDefinition> ProcessedUpdateContextDefinitions;

	bool bCurrentUpdatePatchCheckEnabled = true;
protected:

	/** @return a pointer to the hotfix manager */
	template< typename THotfixManager >
	THotfixManager* GetHotfixManager() const
	{
		return Cast<THotfixManager>(THotfixManager::Get(GetWorld()));
	}

	/** @return a pointer to the world */
	UE_API UWorld* GetWorld() const;

	/** @return a pointer to the game instance */
	UE_API UGameInstance* GetGameInstance() const;
};

#undef UE_API
