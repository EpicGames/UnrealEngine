// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class Lobby : ModuleRules
{
	public Lobby(ReadOnlyTargetRules Target) : base(Target)
	{		
		PublicDefinitions.Add("LOBBY_PACKAGE=1");
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PrivateIncludePaths.AddRange(
			new string[] {
			}
			);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreOnline",
				"CoreUObject",
				"NetCore",
				"Engine",
				"OnlineSubsystem",
				"OnlineSubsystemUtils",
			}
			);

		if (Target.Platform == UnrealTargetPlatform.Win64)
		{
		}
		else if (Target.Platform == UnrealTargetPlatform.Mac)
		{
		}

		SetupIrisSupport(Target);
	}
}
