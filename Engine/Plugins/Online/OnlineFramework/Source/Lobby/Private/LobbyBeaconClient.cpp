// Copyright Epic Games, Inc. All Rights Reserved.

#include "LobbyBeaconClient.h"
#include "Engine/GameInstance.h"
#include "GameFramework/PlayerController.h"
#include "Engine/NetConnection.h"
#include "GameFramework/PlayerState.h"
#include "Engine/LocalPlayer.h"
#include "Net/UnrealNetwork.h"
#include "LobbyBeaconHost.h"
#include "LobbyBeaconPlayerState.h"
#include "OnlineSubsystemUtils.h"
#include "OnlineSessionClient.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LobbyBeaconClient)

ALobbyBeaconClient::ALobbyBeaconClient(const FObjectInitializer& ObjectInitializer) :
	Super(ObjectInitializer),
	LobbyState(nullptr),
	PlayerState(nullptr),
	bLoggedIn(false),
	LobbyJoinServerState(ELobbyBeaconJoinState::None)
{
	bOnlyRelevantToOwner = true;
}

void ALobbyBeaconClient::GetLifetimeReplicatedProps(TArray< FLifetimeProperty > & OutLifetimeProps) const
{
	Super::GetLifetimeReplicatedProps(OutLifetimeProps);

	DOREPLIFETIME(ALobbyBeaconClient, LobbyState);
	DOREPLIFETIME(ALobbyBeaconClient, PlayerState);
}

void ALobbyBeaconClient::EndPlay(EEndPlayReason::Type Reason)
{
	Super::EndPlay(Reason);

	if (PlayerState)
	{
		if (PlayerState->ClientActor == this)
		{
			PlayerState->ClientActor = nullptr;
		}

		if (PlayerState->GetOwner() == this)
		{
			PlayerState->SetOwner(nullptr);
		}
	}
}

void ALobbyBeaconClient::OnConnected()
{
	UE_LOGF(LogLobbyBeacon, Verbose, "Lobby beacon connection established.");

	OnLobbyConnectionEstablished().ExecuteIfBound();
	LoginLocalPlayers();
}

void ALobbyBeaconClient::ConnectToLobby(const FOnlineSessionSearchResult& DesiredHost)
{
	// Make this stuff common?
	bool bSuccess = false;
	IOnlineSubsystem* OnlineSub = Online::GetSubsystem(GetWorld());
	if (OnlineSub)
	{
		IOnlineSessionPtr SessionInt = OnlineSub->GetSessionInterface();
		if (SessionInt.IsValid())
		{
			FString ConnectInfo;
			if (SessionInt->GetResolvedConnectString(DesiredHost, NAME_BeaconPort, ConnectInfo))
			{
				FURL ConnectURL(NULL, *ConnectInfo, TRAVEL_Absolute);
				if (InitClient(ConnectURL) && DesiredHost.Session.SessionInfo.IsValid())
				{
					DestSessionId = DesiredHost.Session.SessionInfo->GetSessionId().ToString();
					bSuccess = true;
				}
				else
				{
					UE_LOGF(LogLobbyBeacon, Warning, "ConnectToLobby: Failure to init client beacon with %ls.", *ConnectURL.ToString());
				}
			}
		}
	}

	if (!bSuccess)
	{
		AOnlineBeacon::OnFailure(EBeaconFailureReason::InitFailure, TEXTVIEW("beacon_lobbyclient_init_failure"));
	}
}

void ALobbyBeaconClient::ClientJoinGame_Implementation()
{
	UE_LOGF(LogLobbyBeacon, Log, "ClientJoinGame signal %d", bLoggedIn);
	if (bLoggedIn)
	{
		OnJoiningGame().ExecuteIfBound();
	}
}

void ALobbyBeaconClient::ClientSetInviteFlags_Implementation(const FJoinabilitySettings& Settings)
{
	UGameInstance* GameInstance = GetGameInstance();
	check(GameInstance);
	
	UOnlineSessionClient* OnlineSessionClient = Cast<UOnlineSessionClient>(GameInstance->GetOnlineSession());
	if (OnlineSessionClient)
	{
		OnlineSessionClient->SetInviteFlags(GetWorld(), Settings);
	}
}

void ALobbyBeaconClient::SetPartyOwnerId(const FUniqueNetIdRepl& InUniqueId, const FUniqueNetIdRepl& InPartyOwnerId)
{
	if (bLoggedIn)
	{
		ServerSetPartyOwner(InUniqueId, InPartyOwnerId);
		if (PlayerState)
		{
			PlayerState->PartyOwnerUniqueId = InPartyOwnerId;
		}
	}
	else
	{
		UE_LOGF(LogLobbyBeacon, Warning, "Not logged in when calling SetPartyOwnerId");
	}
}

void ALobbyBeaconClient::DisconnectFromLobby()
{
	if (bLoggedIn)
	{
		UE_LOGF(LogLobbyBeacon, Log, "DisconnectFromLobby %ls Id: %ls", *GetName(), PlayerState ? *PlayerState->UniqueId->ToString() : TEXT("Unknown"));
		ServerDisconnectFromLobby();
		bLoggedIn = false;
	}
	else
	{
		UE_LOGF(LogLobbyBeacon, Verbose, "Not logged in when calling DisconnectFromLobby");
	}
}

void ALobbyBeaconClient::JoiningServer()
{
	if (bLoggedIn)
	{
		if (LobbyJoinServerState == ELobbyBeaconJoinState::None)
		{
			UE_LOGF(LogLobbyBeacon, Log, "JoiningServer %ls Id: %ls", *GetName(), PlayerState ? *PlayerState->UniqueId->ToString() : TEXT("Unknown"));
			LobbyJoinServerState = ELobbyBeaconJoinState::SentJoinRequest;
			ServerNotifyJoiningServer();
		}
		else
		{
			UE_LOGF(LogLobbyBeacon, Warning, "Already joining server, skipping %d", static_cast<int32>(LobbyJoinServerState));
		}
	}
	else
	{
		UE_LOGF(LogLobbyBeacon, Warning, "Not logged in when calling JoiningServer");
	}
}

void ALobbyBeaconClient::KickPlayer(const FUniqueNetIdRepl& PlayerToKick, const FText& Reason)
{
	if (bLoggedIn)
	{
		ServerKickPlayer(PlayerToKick, Reason);
	}
	else
	{
		UE_LOGF(LogLobbyBeacon, Warning, "Not logged in when calling KickPlayer");
	}
}

void ALobbyBeaconClient::LoginLocalPlayers()
{
	FURL URL(nullptr, TEXT(""), TRAVEL_Absolute);
	FUniqueNetIdRepl UniqueIdRepl;

	UWorld* World = GetWorld();
	for (FConstPlayerControllerIterator Iterator = World->GetPlayerControllerIterator(); Iterator; ++Iterator)
	{
		APlayerController* PC = Iterator->Get();
		if (PC && PC->PlayerState && PC->PlayerState->GetUniqueId().IsValid())
		{
			ULocalPlayer* LP = Cast<ULocalPlayer>(PC->Player);
			if (LP)
			{
				// Send the player nickname if available
				FString OverrideName = LP->GetNickname();
				if (OverrideName.Len() > 0)
				{
					URL.AddOption(*FString::Printf(TEXT("Name=%s"), *OverrideName));
				}

				// Send any game-specific url options for this player
				FString GameUrlOptions = LP->GetGameLoginOptions();
				if (GameUrlOptions.Len() > 0)
				{
					URL.AddOption(*FString::Printf(TEXT("%s"), *GameUrlOptions));
				}

				// Send the player unique Id at login
				UniqueIdRepl = LP->GetPreferredUniqueNetId();

				FString UrlString = URL.ToString();
				if (UniqueIdRepl.IsValid())
				{
					ServerLoginPlayer(DestSessionId, UniqueIdRepl, UrlString);
				}
			}
		}
	}
}

void ALobbyBeaconClient::SetLobbyState(ALobbyBeaconState* InLobbyState)
{ 
	AOnlineBeaconHostObject* BeaconHost = GetBeaconOwner();
	if (BeaconHost)
	{
		LobbyState = InLobbyState;
	}
}

bool ALobbyBeaconClient::ServerLoginPlayer_Validate(const FString& InSessionId, const FUniqueNetIdRepl& InUniqueId, const FString& UrlString)
{
	return !InSessionId.IsEmpty() && InUniqueId.IsValid() && !UrlString.IsEmpty();
}

void ALobbyBeaconClient::ServerLoginPlayer_Implementation(const FString& InSessionId, const FUniqueNetIdRepl& InUniqueId, const FString& UrlString)
{
	UE_LOGF(LogLobbyBeacon, Log, "ServerLoginPlayer %ls %ls.", *InUniqueId.ToString(), *UrlString);
	ALobbyBeaconHost* BeaconHost = Cast<ALobbyBeaconHost>(GetBeaconOwner());
	if (BeaconHost)
	{
		BeaconHost->ProcessLogin(this, InSessionId, InUniqueId, UrlString);
	}
}

void ALobbyBeaconClient::ClientLoginComplete_Implementation(const FUniqueNetIdRepl& InUniqueId, bool bWasSuccessful)
{
	UE_LOGF(LogLobbyBeacon, Log, "ClientLoginComplete %ls %ls.", *InUniqueId.ToString(), bWasSuccessful ? TEXT("Success") : TEXT("Failure"));

	bLoggedIn = bWasSuccessful;
	OnLoginComplete().ExecuteIfBound(bLoggedIn);
}

void ALobbyBeaconClient::ClientWasKicked_Implementation(const FText& KickReason)
{
}

bool ALobbyBeaconClient::ServerDisconnectFromLobby_Validate()
{
	return true;
}

void ALobbyBeaconClient::ServerDisconnectFromLobby_Implementation()
{
	UE_LOGF(LogLobbyBeacon, Log, "ServerDisconnectFromLobby %ls Id: %ls", *GetName(), PlayerState ? *PlayerState->UniqueId->ToString() : TEXT("Unknown"));

	ALobbyBeaconHost* BeaconHost = Cast<ALobbyBeaconHost>(GetBeaconOwner());
	if (BeaconHost)
	{
		BeaconHost->ProcessDisconnect(this);
	}
}

bool ALobbyBeaconClient::ServerNotifyJoiningServer_Validate()
{
	return true;
}

void ALobbyBeaconClient::ServerNotifyJoiningServer_Implementation()
{
	UE_LOGF(LogLobbyBeacon, Log, "ServerNotifyJoiningGame %ls Id: %ls", *GetName(), PlayerState ? *PlayerState->UniqueId->ToString() : TEXT("Unknown"));
	ALobbyBeaconHost* BeaconHost = Cast<ALobbyBeaconHost>(GetBeaconOwner());
	if (BeaconHost)
	{
		BeaconHost->ProcessJoinServer(this);
	}
}

void ALobbyBeaconClient::AckJoiningServer()
{
	if (GetNetMode() < NM_Client)
	{
		UE_LOGF(LogLobbyBeacon, Log, "AckJoiningServer %ls Id: %ls", *GetName(), PlayerState ? *PlayerState->UniqueId->ToString() : TEXT("Unknown"));
		LobbyJoinServerState = ELobbyBeaconJoinState::JoinRequestAcknowledged;
		ClientAckJoiningServer();
	}
}

void ALobbyBeaconClient::ClientAckJoiningServer_Implementation()
{
	UE_LOGF(LogLobbyBeacon, Log, "ClientAckJoiningServer %ls Id: %ls LoggedIn: %d", *GetName(), PlayerState ? *PlayerState->UniqueId->ToString() : TEXT("Unknown"), bLoggedIn);
	LobbyJoinServerState = ELobbyBeaconJoinState::JoinRequestAcknowledged;
	OnJoiningGameAck().ExecuteIfBound();
}

bool ALobbyBeaconClient::ServerKickPlayer_Validate(const FUniqueNetIdRepl& PlayerToKick, const FText& Reason)
{
	return true;
}

void ALobbyBeaconClient::ServerKickPlayer_Implementation(const FUniqueNetIdRepl& PlayerToKick, const FText& Reason)
{
	UE_LOGF(LogLobbyBeacon, Log, "ServerKickPlayer %ls -> %ls", (PlayerState != nullptr ? *PlayerState->UniqueId.ToString() : TEXT("")),
			*PlayerToKick.ToString());

	ALobbyBeaconHost* BeaconHost = Cast<ALobbyBeaconHost>(GetBeaconOwner());
	if (BeaconHost)
	{
		BeaconHost->ProcessKickPlayer(this, PlayerToKick, Reason);
	}
}

bool ALobbyBeaconClient::ServerSetPartyOwner_Validate(const FUniqueNetIdRepl& InUniqueId, const FUniqueNetIdRepl& InPartyOwnerId)
{
	return InUniqueId.IsValid() && InPartyOwnerId.IsValid();
}

void ALobbyBeaconClient::ServerSetPartyOwner_Implementation(const FUniqueNetIdRepl& InUniqueId, const FUniqueNetIdRepl& InPartyOwnerId)
{
}

void ALobbyBeaconClient::ClientPlayerJoined_Implementation(const FText& NewPlayerName, const FUniqueNetIdRepl& InUniqueId)
{
	UE_LOGF(LogLobbyBeacon, Log, "ClientPlayerJoined %ls %ls.", *NewPlayerName.ToString(), *InUniqueId.ToDebugString());

	if (GetNetMode() != NM_Standalone)
	{
		IOnlineSessionPtr SessionInt = Online::GetSessionInterface(GetWorld());
		if (SessionInt.IsValid() && InUniqueId.IsValid())
		{
			// Register the player as part of the session
			SessionInt->RegisterPlayer(NAME_GameSession, *InUniqueId, false);
		}
	}

	OnPlayerJoined().ExecuteIfBound(NewPlayerName, InUniqueId);
}

void ALobbyBeaconClient::ClientPlayerLeft_Implementation(const FUniqueNetIdRepl& InUniqueId)
{
	UE_LOGF(LogLobbyBeacon, Log, "ClientPlayerLeft %ls", *InUniqueId.ToDebugString());

	if (GetNetMode() != NM_Standalone)
	{
		IOnlineSessionPtr SessionInt = Online::GetSessionInterface(GetWorld());
		if (SessionInt.IsValid() && InUniqueId.IsValid())
		{
			// Register the player as part of the session
			SessionInt->UnregisterPlayer(NAME_GameSession, *InUniqueId);
		}
	}

	OnPlayerLeft().ExecuteIfBound(InUniqueId);
}

bool ALobbyBeaconClient::ServerCheat_Validate(const FString& Msg)
{
	return !UE_BUILD_SHIPPING;
}

void ALobbyBeaconClient::ServerCheat_Implementation(const FString& Msg)
{
#if !UE_BUILD_SHIPPING

	UNetConnection* NetConnection = GetNetConnection();
	if (NetConnection)
	{
		NetConnection->ConsoleCommand(Msg);
	}
	
#endif
}


