// Copyright Epic Games, Inc. All Rights Reserved.

#include "LobbyBeaconHost.h"
#include "GameFramework/GameModeBase.h"
#include "Engine/NetConnection.h"
#include "Interfaces/OnlineSessionInterface.h"
#include "Kismet/GameplayStatics.h"
#include "GameFramework/GameSession.h"
#include "OnlineBeaconHost.h"
#include "LobbyBeaconClient.h"
#include "LobbyBeaconState.h"
#include "LobbyBeaconPlayerState.h"
#include "OnlineSessionSettings.h"
#include "OnlineSubsystemUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LobbyBeaconHost)

DEFINE_LOG_CATEGORY(LogLobbyBeacon);


ALobbyBeaconHost::ALobbyBeaconHost(const FObjectInitializer& ObjectInitializer) :
	Super(ObjectInitializer),
	SessionName(NAME_None), 
	LobbyState(nullptr)
{
	ClientBeaconActorClass = ALobbyBeaconClient::StaticClass();
	LobbyStateClass = ALobbyBeaconState::StaticClass();

	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
	}
}

bool ALobbyBeaconHost::Init(FName InSessionName)
{
	bool bSuccess = false;

	SessionName = InSessionName;
	if (ensure(ClientBeaconActorClass))
	{
		BeaconTypeName = ClientBeaconActorClass->GetName();
		bSuccess = true;
	}

	return bSuccess;
}

void ALobbyBeaconHost::SetupLobbyState(int32 InMaxPlayers)
{
	if (!LobbyState)
	{
		if (ensure(LobbyStateClass))
		{
			FActorSpawnParameters SpawnInfo;
			SpawnInfo.Owner = this;
			LobbyState = GetWorld()->SpawnActor<ALobbyBeaconState>(LobbyStateClass.Get(), FVector::ZeroVector, FRotator::ZeroRotator, SpawnInfo);

			// Associate with this objects net driver for proper replication
			LobbyState->SetNetDriverName(GetNetDriverName());
		}
	}

	if (LobbyState)
	{
		LobbyState->MaxPlayers = InMaxPlayers;
	}
}

void ALobbyBeaconHost::UpdatePartyLeader(const FUniqueNetIdRepl& PartyMemberId, const FUniqueNetIdRepl& NewPartyLeaderId)
{
	if (LobbyState)
	{
		LobbyState->UpdatePartyLeader(PartyMemberId, NewPartyLeaderId);
	}
}

bool ALobbyBeaconHost::DoesSessionMatch(const FString& InSessionId) const
{
	UWorld* World = GetWorld();

	IOnlineSessionPtr SessionInt = Online::GetSessionInterface(World);
	FNamedOnlineSession* Session = SessionInt.IsValid() ? SessionInt->GetNamedSession(SessionName) : NULL;
	if (Session && Session->SessionInfo.IsValid() && !InSessionId.IsEmpty() && Session->SessionInfo->GetSessionId().ToString() == InSessionId)
	{
		return true;
	}

	return false;
}

ALobbyBeaconPlayerState* ALobbyBeaconHost::HandlePlayerLogin(ALobbyBeaconClient* ClientActor, const FUniqueNetIdRepl& InUniqueId, const FString& Options)
{
	if (LobbyState != nullptr)
	{
		UWorld* World = GetWorld();
		check(World);

		FString NewPlayerName = UGameplayStatics::ParseOption(Options, TEXT("Name")).Left(20);
		if (NewPlayerName.IsEmpty())
		{
			NewPlayerName = TEXT("UnknownUser");
		}

		FString InAuthTicket = UGameplayStatics::ParseOption(Options, TEXT("AuthTicket"));
		UE_LOGF(LogLobbyBeacon, Log, "Lobby beacon received AuthTicket from client for player: UniqueId:[%ls]", *InUniqueId.ToDebugString());

		if (GetNetMode() != NM_Standalone)
		{
			IOnlineSessionPtr SessionInt = Online::GetSessionInterface(World);
			if (SessionInt.IsValid() && InUniqueId.IsValid())
			{
				// Register the player as part of the session
				bool bWasFromInvite = UGameplayStatics::HasOption(Options, TEXT("bIsFromInvite"));
				SessionInt->RegisterPlayer(NAME_GameSession, *InUniqueId, bWasFromInvite);
			}
		}

		FText DisplayName = FText::FromString(NewPlayerName);
		ALobbyBeaconPlayerState* NewLobbyPlayer = LobbyState->AddPlayer(DisplayName, InUniqueId);
		return NewLobbyPlayer;
	}
	else
	{
		UE_LOGF(LogLobbyBeacon, Warning, "No lobby state to handle user logging in!");
		return nullptr;
	}
}

void ALobbyBeaconHost::ProcessLogin(ALobbyBeaconClient* ClientActor, const FString& InSessionId, const FUniqueNetIdRepl& InUniqueId, const FString& UrlString)
{
	UE_LOGF(LogLobbyBeacon, Verbose, "ProcessLogin %ls SessionId %ls %ls %ls from (%ls)",
		ClientActor ? *ClientActor->GetName() : TEXT("NULL"),
		*InSessionId,
		*InUniqueId.ToString(),
		*UrlString,
		ClientActor ? (ClientActor->GetNetConnection() ? *ClientActor->GetNetConnection()->LowLevelDescribe() : TEXT("NULL")) : TEXT("NULL"));


	if (ClientActor)
	{
		bool bSuccess = false;
		if (DoesSessionMatch(InSessionId) && InUniqueId.IsValid())
		{
			FURL InURL(NULL, *UrlString, TRAVEL_Absolute);
			if (InURL.Valid)
			{
				// Make the option string.
				FString Options;
				for (int32 i = 0; i < InURL.Op.Num(); i++)
				{
					Options += TEXT('?');
					Options += InURL.Op[i];
				}

				if (PreLogin(InUniqueId, Options))
				{
					ALobbyBeaconPlayerState* NewPlayer = HandlePlayerLogin(ClientActor, InUniqueId, Options);
					if (NewPlayer && NewPlayer->IsValid())
					{
						bSuccess = true;

						NewPlayer->SetOwner(ClientActor);
						ClientActor->PlayerState = NewPlayer;
						NewPlayer->ClientActor = ClientActor;

						ClientActor->SetLobbyState(LobbyState);
						for (AOnlineBeaconClient* ExistingClient : ClientActors)
						{
							if (ExistingClient != ClientActor)
							{
								ALobbyBeaconClient* LBC = Cast<ALobbyBeaconClient>(ExistingClient);
								LBC->ClientPlayerJoined(NewPlayer->DisplayName, NewPlayer->UniqueId);
							}
						}
					}
				}
			}

			if (LobbyState && !LobbyState->HasLobbyStarted())
			{
				int32 NumLobbyPlayers = LobbyState->GetNumPlayers();
				if (NumLobbyPlayers == LobbyState->GetMaxPlayers())
				{
					// Session is full, load into game
					LobbyState->StartLobby();
				}
				else if (NumLobbyPlayers == 1)
				{
					// First player has joined, start waiting for others
					LobbyState->StartWaiting();
				}
			}
		}

		ClientActor->ClientLoginComplete(InUniqueId, bSuccess);
		ClientActor->bLoggedIn = bSuccess;
		if (bSuccess)
		{
			PostLogin(ClientActor);
		}
		else
		{
			DisconnectClient(ClientActor);
		}
	}
}

bool ALobbyBeaconHost::PreLogin(const FUniqueNetIdRepl& InUniqueId, const FString& Options)
{
	return true;
}

void ALobbyBeaconHost::PostLogin(ALobbyBeaconClient* ClientActor)
{
	// Handle beacon connection termination code (code the GameMode would normal handle)
}

void ALobbyBeaconHost::KickPlayer(ALobbyBeaconClient* ClientActor, const FText& KickReason)
{
	UE_LOGF(LogLobbyBeacon, Log, "KickPlayer for %ls. IsValid %d UNetConnection %ls UNetDriver %ls State %d", *GetNameSafe(ClientActor), IsValid(ClientActor), *GetNameSafe(ClientActor->BeaconConnection), ClientActor->BeaconConnection ? *GetNameSafe(ClientActor->BeaconConnection->Driver) : TEXT("null"), ClientActor->BeaconConnection ? ClientActor->BeaconConnection->GetConnectionState() : -1);
	ClientActor->ClientWasKicked(KickReason);
	DisconnectClient(ClientActor);
}

bool ALobbyBeaconHost::ProcessJoinServer(ALobbyBeaconClient* ClientActor)
{
	bool bSuccess = false;
	if (LobbyState != nullptr)
	{
		ALobbyBeaconPlayerState* Player = LobbyState->GetPlayer(ClientActor);
		if (Player && Player->bInLobby)
		{
			Player->bInLobby = false;
			ClientActor->AckJoiningServer();

			bSuccess = true;
		}
		else
		{
			FString PlayerName = ClientActor ? (ClientActor->PlayerState ? *ClientActor->PlayerState->UniqueId.ToString() : TEXT("Unknown")) : TEXT("Unknown Client Actor");
			UE_LOGF(LogLobbyBeacon, Warning, "Player attempting to join server while not logged in %ls Id: %ls", *GetName(), *PlayerName);
		}
	}
	else
	{
		UE_LOGF(LogLobbyBeacon, Warning, "No lobby state to handle user joining the game!");
	}

	return bSuccess;
}

void ALobbyBeaconHost::ProcessDisconnect(ALobbyBeaconClient* ClientActor)
{
	if (ClientActor)
	{
		AOnlineBeaconHost* BeaconHost = Cast<AOnlineBeaconHost>(GetOwner());
		if (BeaconHost)
		{
			DisconnectClient(ClientActor);
		}
	}
}

bool ALobbyBeaconHost::ProcessKickPlayer(ALobbyBeaconClient* InInstigator, const FUniqueNetIdRepl& PlayerToKick, const FText& Reason)
{
	bool bWasKicked = false;
	if (InInstigator && PlayerToKick.IsValid())
	{
		for (AOnlineBeaconClient* ExistingClient : ClientActors)
		{
			if (ExistingClient != InInstigator)
			{
				ALobbyBeaconClient* LBC = Cast<ALobbyBeaconClient>(ExistingClient);
				if (LBC && LBC->PlayerState && LBC->PlayerState->UniqueId == PlayerToKick)
				{
					// Right now the only eligible lobby kick is a party leader telling the server it kicked a party member
					bool bPartyLeaderKick = (InInstigator->PlayerState->UniqueId == LBC->PlayerState->PartyOwnerUniqueId);
					if (bPartyLeaderKick)
					{
						FText KickReason = NSLOCTEXT("NetworkErrors", "KickedPlayerFromParty", "Kicked from party.");
						KickPlayer(LBC, KickReason);
						bWasKicked = true;
					}
					break;
				}
			}
		}
	}
	return bWasKicked;
}

void ALobbyBeaconHost::HandlePlayerLogout(const FUniqueNetIdRepl& InUniqueId)
{
	if (InUniqueId.IsValid())
	{
		for (AOnlineBeaconClient* ExistingClient : ClientActors)
		{
			ALobbyBeaconClient* LBC = CastChecked<ALobbyBeaconClient>(ExistingClient);
			if (LBC && LBC->PlayerState && LBC->PlayerState->UniqueId.IsValid() && LBC->PlayerState->UniqueId != InUniqueId)
			{
				LBC->ClientPlayerLeft(InUniqueId);
			}
		}

		if (LobbyState)
		{
			LobbyState->RemovePlayer(InUniqueId);
		}
	}
}

void ALobbyBeaconHost::NotifyClientDisconnected(AOnlineBeaconClient* LeavingClientActor)
{
	if (LobbyState)
	{
		ALobbyBeaconPlayerState* Player = LobbyState->GetPlayer(LeavingClientActor);
		if (Player && Player->bInLobby)
		{
			// Handle beacon connection termination code (code the GameMode would normal handle)
			UWorld* World = GetWorld();
			check(World);

			AGameModeBase* GameMode = World->GetAuthGameMode();
			check(GameMode);
			check(GameMode->GameSession);

			// Notify the session (updates reservation beacon, unregisters the player, etc)
			GameMode->GameSession->NotifyLogout(NAME_GameSession, Player->UniqueId);
			HandlePlayerLogout(Player->UniqueId);
		}
	}
	else
	{
		UE_LOGF(LogLobbyBeacon, Warning, "No lobby beacon state to handle disconnection!");
	}

	Super::NotifyClientDisconnected(LeavingClientActor);
}

void ALobbyBeaconHost::AdvertiseSessionJoinability(const FJoinabilitySettings& Settings)
{
}

void ALobbyBeaconHost::DumpState() const
{
	UE_LOGF(LogLobbyBeacon, Display, "Lobby Beacon: %ls", *GetBeaconType());

	if (LobbyState)
	{
		LobbyState->DumpState();
	}
}


