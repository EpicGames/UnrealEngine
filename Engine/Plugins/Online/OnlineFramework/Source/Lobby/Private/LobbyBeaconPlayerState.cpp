// Copyright Epic Games, Inc. All Rights Reserved.

#include "LobbyBeaconPlayerState.h"
#include "Net/UnrealNetwork.h"
#include "LobbyBeaconClient.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LobbyBeaconPlayerState)

ALobbyBeaconPlayerState::ALobbyBeaconPlayerState(const FObjectInitializer& ObjectInitializer) :
	Super(ObjectInitializer),
	bInLobby(false),
	ClientActor(nullptr)
{
	bReplicates = true;
	bAlwaysRelevant = true;
	NetDriverName = NAME_BeaconNetDriver;
}

void ALobbyBeaconPlayerState::PostInitializeComponents()
{
	Super::PostInitializeComponents();

}

void ALobbyBeaconPlayerState::GetLifetimeReplicatedProps(TArray< FLifetimeProperty >& OutLifetimeProps) const
{
	Super::GetLifetimeReplicatedProps(OutLifetimeProps);

	DOREPLIFETIME(ALobbyBeaconPlayerState, DisplayName);
	DOREPLIFETIME(ALobbyBeaconPlayerState, UniqueId);
	DOREPLIFETIME(ALobbyBeaconPlayerState, PartyOwnerUniqueId);
	DOREPLIFETIME(ALobbyBeaconPlayerState, bInLobby);
	DOREPLIFETIME_CONDITION(ALobbyBeaconPlayerState, ClientActor, COND_OwnerOnly);
}

bool ALobbyBeaconPlayerState::IsNetRelevantFor(const AActor* RealViewer, const AActor* ViewTarget, const FVector& SrcLocation) const
{
	UClass* BeaconClass = RealViewer->GetClass();
	return (BeaconClass && BeaconClass->IsChildOf(ALobbyBeaconClient::StaticClass()));
}

bool ALobbyBeaconPlayerState::IsValid() const
{ 
	return UniqueId.IsValid();
}

void ALobbyBeaconPlayerState::OnRep_UniqueId()
{
	if (UniqueIdReplicatedEvent.IsBound())
	{
		UniqueIdReplicatedEvent.Broadcast(UniqueId);
	}
}

void ALobbyBeaconPlayerState::OnRep_PartyOwner()
{
	if (PartyOwnerChangedEvent.IsBound())
	{
		PartyOwnerChangedEvent.Broadcast(UniqueId);
	}
}

void ALobbyBeaconPlayerState::OnRep_InLobby()
{
	if (PlayerStateChangedEvent.IsBound())
	{
		PlayerStateChangedEvent.Broadcast(UniqueId);
	}
}

