// Copyright Epic Games, Inc. All Rights Reserved.

#include "LobbyBeaconState.h"
#include "LobbyBeaconHost.h"
#include "TimerManager.h"
#include "Engine/World.h"
#include "Net/UnrealNetwork.h"
#include "LobbyBeaconPlayerState.h"
#include "LobbyBeaconClient.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LobbyBeaconState)

void FLobbyPlayerStateActorInfo::PreReplicatedRemove(const FLobbyPlayerStateInfoArray& InArraySerializer)
{
	check(InArraySerializer.ParentState);
	if (LobbyPlayerState)
	{
		InArraySerializer.ParentState->OnPlayerLobbyStateRemoved().Broadcast(LobbyPlayerState);
	}
}

void FLobbyPlayerStateActorInfo::PostReplicatedAdd(const FLobbyPlayerStateInfoArray& InArraySerializer)
{
	check(InArraySerializer.ParentState);
	if (LobbyPlayerState)
	{
		InArraySerializer.ParentState->OnPlayerLobbyStateAdded().Broadcast(LobbyPlayerState);
	}
	else
	{
		UE_LOGF(LogLobbyBeacon, Verbose, "PostReplicatedAdd with a null LobbyPlayerState actor, expect a future PostReplicatedChange");
	}
}

void FLobbyPlayerStateActorInfo::PostReplicatedChange(const FLobbyPlayerStateInfoArray& InArraySerializer)
{
	check(InArraySerializer.ParentState);
	if (LobbyPlayerState)
	{
		InArraySerializer.ParentState->OnPlayerLobbyStateAdded().Broadcast(LobbyPlayerState);
	}
	else
	{
		UE_LOGF(LogLobbyBeacon, Verbose, "PostReplicatedChange to a null LobbyPlayerState actor");
	}
}

ALobbyBeaconPlayerState* FLobbyPlayerStateInfoArray::AddPlayer(const FText& PlayerName, const FUniqueNetIdRepl& UniqueId)
{
	check(ParentState);
	UWorld* World = ParentState->GetWorld();
	check(World);

	ALobbyBeaconPlayerState* NewPlayer = ParentState->CreateNewPlayer(PlayerName, UniqueId);
	if (NewPlayer)
	{
		int32 Idx = Players.Add(NewPlayer);
		MarkItemDirty(Players[Idx]);

		// server calls "on rep" also
		Players[Idx].PostReplicatedAdd(*this);

		return Players[Idx].LobbyPlayerState;
	}

	return nullptr;
}

void FLobbyPlayerStateInfoArray::RemovePlayer(const FUniqueNetIdRepl& UniqueId)
{
	for (int32 PlayerIdx = 0; PlayerIdx < Players.Num(); PlayerIdx++)
	{
		FLobbyPlayerStateActorInfo& Player = Players[PlayerIdx];
		if (Player.LobbyPlayerState && Player.LobbyPlayerState->UniqueId == UniqueId)
		{
			if (Player.LobbyPlayerState->GetNetMode() < NM_Client)
			{
				// server calls "on rep" also
				Player.PreReplicatedRemove(*this);

				Player.LobbyPlayerState->Destroy();
				Player.LobbyPlayerState = nullptr;
				Players.RemoveAtSwap(PlayerIdx);
				MarkArrayDirty();
				break;
			}
		}
	}
}

ALobbyBeaconPlayerState* FLobbyPlayerStateInfoArray::GetPlayer(const FUniqueNetIdRepl& UniqueId)
{
	for (FLobbyPlayerStateActorInfo& Player : Players)
	{
		if (Player.LobbyPlayerState && Player.LobbyPlayerState->UniqueId == UniqueId)
		{
			return Player.LobbyPlayerState;
		}
	}

	return nullptr;
}

ALobbyBeaconPlayerState* FLobbyPlayerStateInfoArray::GetPlayer(const AOnlineBeaconClient* ClientActor)
{
	if (ClientActor && ClientActor->GetNetMode() < NM_Client)
	{
		for (FLobbyPlayerStateActorInfo& Player : Players)
		{
			if (Player.LobbyPlayerState && Player.LobbyPlayerState->ClientActor == ClientActor)
			{
				return Player.LobbyPlayerState;
			}
		}
	}

	return nullptr;
}

void FLobbyPlayerStateInfoArray::DumpState() const
{
	int32 Count = 0;
	for (const FLobbyPlayerStateActorInfo& PlayerState : Players)
	{
		const ALobbyBeaconPlayerState* Player = PlayerState.LobbyPlayerState;
		if (Player)
		{
			UE_LOGF(LogLobbyBeacon, Display, "[%d] %ls %ls %ls", ++Count, *Player->DisplayName.ToString(), *Player->UniqueId.ToString(), Player->bInLobby ? TEXT("In Lobby") : TEXT("In Game"));
		}
	}
}

ALobbyBeaconState::ALobbyBeaconState(const FObjectInitializer& ObjectInitializer) :
	Super(ObjectInitializer),
	MaxPlayers(0),
	LobbyBeaconPlayerStateClass(nullptr),
	LastTickTime(0.0),
	bLobbyStarted(false),
	WaitForPlayersTimeRemaining(0.0f)
{
	bReplicates = true;
	bAlwaysRelevant = true;
	NetDriverName = NAME_BeaconNetDriver;

	LobbyBeaconPlayerStateClass = ALobbyBeaconPlayerState::StaticClass();

	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		Players.ParentState = this;
	}
}

void ALobbyBeaconState::PostInitProperties()
{
	Super::PostInitProperties();
	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
        // Set instances back to 0, it will be set in StartWaiting
		WaitForPlayersTimeRemaining = 0.0f;
	}
}

void ALobbyBeaconState::PostInitializeComponents()
{
	Super::PostInitializeComponents();

	if (GetLocalRole() == ROLE_Authority)
	{
		FTimerDelegate TimerDelegate = FTimerDelegate::CreateUObject(this, &ALobbyBeaconState::OneSecTick);
		GetWorldTimerManager().SetTimer(OneSecTimerHandle, TimerDelegate, 1.0f, true);

		LastTickTime = FPlatformTime::Seconds();
	}
}

bool ALobbyBeaconState::IsNetRelevantFor(const AActor* RealViewer, const AActor* ViewTarget, const FVector& SrcLocation) const
{
	UClass* BeaconClass = RealViewer->GetClass();
	return (BeaconClass && BeaconClass->IsChildOf(ALobbyBeaconClient::StaticClass()));
}

void ALobbyBeaconState::GetLifetimeReplicatedProps(TArray< FLifetimeProperty >& OutLifetimeProps) const
{
	Super::GetLifetimeReplicatedProps(OutLifetimeProps);

	DOREPLIFETIME(ALobbyBeaconState, Players);
	DOREPLIFETIME(ALobbyBeaconState, bLobbyStarted);
	DOREPLIFETIME(ALobbyBeaconState, WaitForPlayersTimeRemaining);
}

void ALobbyBeaconState::StartWaiting()
{
	if (GetLocalRole() == ROLE_Authority)
	{
		WaitForPlayersTimeRemaining = GetClass()->GetDefaultObject<ALobbyBeaconState>()->WaitForPlayersTimeRemaining;
		OnRep_WaitForPlayersTimeRemaining();

		LastTickTime = FPlatformTime::Seconds();
	}
}

void ALobbyBeaconState::StartLobby()
{
	if (GetLocalRole() == ROLE_Authority)
	{
		WaitForPlayersTimeRemaining = 0.0f;
		OnRep_WaitForPlayersTimeRemaining();

		LastTickTime = FPlatformTime::Seconds();

		bLobbyStarted = true;
		OnRep_LobbyStarted();
	}
}

void ALobbyBeaconState::OnRep_LobbyStarted()
{
	if (bLobbyStarted)
	{
		OnLobbyStarted().Broadcast();
	}
}

void ALobbyBeaconState::OnRep_WaitForPlayersTimeRemaining()
{
	if (!bLobbyStarted)
	{
		OnLobbyWaitingForPlayersUpdate().Broadcast();
	}
}

void ALobbyBeaconState::OneSecTick()
{
	if (GetLocalRole() == ROLE_Authority)
	{
		double CurrTime = FPlatformTime::Seconds();
		double DeltaTime = (CurrTime - LastTickTime);
		LastTickTime = CurrTime;

		// Handle pre-lobby started functionality
		if (!bLobbyStarted)
		{
			OnPreLobbyStartedTickInternal(DeltaTime);
		}
		// Handle lobby started functionality
		else
		{
			OnPostLobbyStartedTickInternal(DeltaTime);
		}
	}
}

void ALobbyBeaconState::OnPreLobbyStartedTickInternal(double DeltaTime)
{
	const bool bRequireFullLobby = RequireFullLobbyToStart();

	// If a full game is required, don't even worry about the timer. It has to be allowed to go indefinitely until the game is
	// full, at which point the lobby will be started automatically.
	if (!bRequireFullLobby && WaitForPlayersTimeRemaining > 0.0f)
	{
		WaitForPlayersTimeRemaining -= DeltaTime;
		if (WaitForPlayersTimeRemaining <= 0.0f)
		{
			WaitForPlayersTimeRemaining = 0.0f;
			StartLobby();
		}
	}
}

void ALobbyBeaconState::OnPostLobbyStartedTickInternal(double DeltaTime)
{
#if 0
	LobbyTimeRemaining -= DeltaTime;
	if (LobbyTimeRemaining <= 0)
	{
		LobbyTimeRemaining = 0.0;
		TArray<FLobbyPlayerStateActorInfo>& AllPlayers = Players.GetAllPlayers();
		for (const FLobbyPlayerStateActorInfo& Player : AllPlayers)
		{
			ALobbyBeaconPlayerState* PlayerState = Player.LobbyPlayerState;
			if (PlayerState && PlayerState->bInLobby && PlayerState->ClientActor)
			{
				ALobbyBeaconClient* LobbyClient = Cast<ALobbyBeaconClient>(PlayerState->ClientActor);
				LobbyClient->ClientJoinGame();
			}
		}
	}
#endif
}

ALobbyBeaconPlayerState* ALobbyBeaconState::CreateNewPlayer(const FText& PlayerName, const FUniqueNetIdRepl& UniqueId)
{
	UWorld* World = GetWorld();
	check(World);

	ALobbyBeaconPlayerState* NewPlayer = nullptr;
	if (ensure(LobbyBeaconPlayerStateClass))
	{
		FActorSpawnParameters SpawnInfo;
		NewPlayer = World->SpawnActor<ALobbyBeaconPlayerState>(LobbyBeaconPlayerStateClass, FVector::ZeroVector, FRotator::ZeroRotator, SpawnInfo);
		if (NewPlayer)
		{
			// Associate with this objects net driver for proper replication
			NewPlayer->SetNetDriverName(NetDriverName);
			NewPlayer->DisplayName = PlayerName;
			NewPlayer->UniqueId = UniqueId;
			NewPlayer->bInLobby = true;
		}
	}

	return NewPlayer;
}

ALobbyBeaconPlayerState* ALobbyBeaconState::AddPlayer(const FText& PlayerName, const FUniqueNetIdRepl& UniqueId)
{
	if (GetLocalRole() == ROLE_Authority)
	{
		return Players.AddPlayer(PlayerName, UniqueId);
	}

	return nullptr;
}

void ALobbyBeaconState::RemovePlayer(const FUniqueNetIdRepl& UniqueId)
{
	if (GetLocalRole() == ROLE_Authority)
	{
		Players.RemovePlayer(UniqueId);
	}
}

void ALobbyBeaconState::UpdatePartyLeader(const FUniqueNetIdRepl& PartyMemberId, const FUniqueNetIdRepl& NewPartyLeaderId)
{
	ALobbyBeaconPlayerState* ReportingPlayer = GetPlayer(PartyMemberId);
	if (ReportingPlayer)
	{
		FUniqueNetIdRepl& OldPartyLeaderId = ReportingPlayer->PartyOwnerUniqueId;
		if (OldPartyLeaderId != NewPartyLeaderId)
		{
			ReportingPlayer->PartyOwnerUniqueId = NewPartyLeaderId;
			if (OldPartyLeaderId.IsValid())
			{
				TArray<FLobbyPlayerStateActorInfo> PlayersArray = Players.GetAllPlayers();
				for (const FLobbyPlayerStateActorInfo& Item : PlayersArray)
				{
					if (Item.LobbyPlayerState && Item.LobbyPlayerState != ReportingPlayer && Item.LobbyPlayerState->PartyOwnerUniqueId == OldPartyLeaderId)
					{
						Item.LobbyPlayerState->PartyOwnerUniqueId = NewPartyLeaderId;
					}
				}
			}
		}
	}
}

ALobbyBeaconPlayerState* ALobbyBeaconState::GetPlayer(const FUniqueNetIdRepl& UniqueId)
{
	return Players.GetPlayer(UniqueId);
}

ALobbyBeaconPlayerState* ALobbyBeaconState::GetPlayer(const AOnlineBeaconClient* ClientActor)
{
	if (GetLocalRole() == ROLE_Authority)
	{
		return Players.GetPlayer(ClientActor);
	}

	return nullptr;
}

ALobbyBeaconPlayerState* ALobbyBeaconState::GetPlayer(const FString& UniqueId)
{
	TArray<FLobbyPlayerStateActorInfo> PlayersArray = Players.GetAllPlayers();
	
	for (const FLobbyPlayerStateActorInfo& Item : PlayersArray)
	{
		if (Item.LobbyPlayerState && UniqueId.Equals(Item.LobbyPlayerState->UniqueId.ToString()))
		{
			return GetPlayer(Item.LobbyPlayerState->UniqueId);
		}
	}

	return nullptr;
}

void ALobbyBeaconState::DumpState() const
{
	UE_LOGF(LogLobbyBeacon, Display, "Players:");
	Players.DumpState();
}

bool ALobbyBeaconState::RequireFullLobbyToStart() const
{
	return false;
}

