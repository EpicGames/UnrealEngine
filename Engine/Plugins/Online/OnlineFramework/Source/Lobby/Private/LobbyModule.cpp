// Copyright Epic Games, Inc. All Rights Reserved.

#include "LobbyModule.h"

IMPLEMENT_MODULE(FLobbyModule, Lobby);

DEFINE_LOG_CATEGORY(LogLobby);

#if STATS
//DEFINE_STAT(STAT_LobbyStat1);
#endif

void FLobbyModule::StartupModule()
{	
}

void FLobbyModule::ShutdownModule()
{
}

