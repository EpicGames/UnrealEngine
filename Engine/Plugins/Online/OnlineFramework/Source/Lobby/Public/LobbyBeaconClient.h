// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineBeaconClient.h"


#include "LobbyBeaconClient.generated.h"

#define UE_API LOBBY_API

struct FJoinabilitySettings;

class ALobbyBeaconPlayerState;
class ALobbyBeaconState;
class FOnlineSessionSearchResult;

/**
 * Delegate called a connection with the lobby beacon is established (but not logged in yet)
 */
DECLARE_DELEGATE(FOnLobbyConnectionEstablished);

/**
 * Delegate called when a player joins the lobby
 */
DECLARE_DELEGATE_TwoParams(FOnLobbyPlayerJoined, const FText& /** DisplayName */, const FUniqueNetIdRepl& /** UniqueId */);

/**
 * Delegate called when a player leaves the lobby
 */
DECLARE_DELEGATE_OneParam(FOnLobbyPlayerLeft, const FUniqueNetIdRepl& /** UniqueId */);

/**
 * Delegate called when the login handshake for this client is complete
 */
DECLARE_DELEGATE_OneParam(FOnLobbyLoginComplete, bool /* bWasSuccessful*/);

/**
 * Delegate called when the player is joining the game from the lobby
 */
DECLARE_DELEGATE(FOnJoiningGame);

UENUM()
enum class ELobbyBeaconJoinState : uint8
{
	/** Unknown, beacon may be connected but no intent to actually join the server */
	None,
	/** Join request has been sent, waiting for a response */
	SentJoinRequest,
	/** Join request has been acknowledged */
	JoinRequestAcknowledged
};

/**
 * A beacon client used for quality timings to a specified session
 */
UCLASS(MinimalAPI, transient, config=Engine, notplaceable)
class ALobbyBeaconClient : public AOnlineBeaconClient
{
	GENERATED_UCLASS_BODY()

	/** Client view of the lobby state */
	UPROPERTY(Replicated)
	TObjectPtr<ALobbyBeaconState> LobbyState;
	/** Player state associated with this beacon (@todo not splitscreen safe) */
	UPROPERTY(Replicated)
	TObjectPtr<ALobbyBeaconPlayerState> PlayerState;

	/** Clear out any references to this in PlayerState */
	UE_API virtual void EndPlay(EEndPlayReason::Type Reason) override;
	
	//~ Begin AOnlineBeaconClient Interface
	UE_API virtual void OnConnected() override;
	//~ End AOnlineBeaconClient Interface

	/**
	 * Initiate a connection to the lobby host beacon
	 *
	 * @param DesiredHost desired host destination
	 */
	UE_API virtual void ConnectToLobby(const FOnlineSessionSearchResult& DesiredHost);

	/**
	 * Tell the client to join the game
	 */
	UFUNCTION(client, reliable)
	UE_API virtual void ClientJoinGame();

	/**
	 * Graceful disconnect from server with no intent of joining further
	 */
	UE_API virtual void DisconnectFromLobby();

	/**
	 * Graceful notification that this client is going to join the server
	 */
	UE_API virtual void JoiningServer();

	/**
	 * Ask the server to kick a given player (may not succeed)
	 *
	 * @param PlayerToKick player kick request
	 * @param Reason reason for the kick to tell client if this succeeds
	 */
	UE_API void KickPlayer(const FUniqueNetIdRepl& PlayerToKick, const FText& Reason);

	/**
	 * Tell the server to set a party owner
	 *
	 * @param InUniqueId unique id of the player making the change
	 * @param InPartyOwnerId unique id of the party owner
	 */
	UE_API virtual void SetPartyOwnerId(const FUniqueNetIdRepl& InUniqueId, const FUniqueNetIdRepl& InPartyOwnerId);

	/** Send updated session settings to client */
	UFUNCTION(reliable, client)
	UE_API void ClientSetInviteFlags(const FJoinabilitySettings& Settings);

	/** @return delegate fired when a connection with the lobby beacon is established */
	FOnLobbyConnectionEstablished& OnLobbyConnectionEstablished() { return LobbyConnectionEstablished; }
	/** @return delegate fired when login handshaking is complete */
	FOnLobbyLoginComplete& OnLoginComplete() { return LoginCompleteDelegate; }
	/** @return delegate fired when a new player joins the lobby */
	FOnLobbyPlayerJoined& OnPlayerJoined() { return PlayerJoinedDelegate; }
	/** @return delegate fired when an existing player leaves the lobby */
	FOnLobbyPlayerLeft& OnPlayerLeft() { return PlayerLeftDelegate; }
	/** @return delegate fired when this player is told to join the game by the server */
	FOnJoiningGame& OnJoiningGame() { return JoiningGame; }
	/** @return delegate fired when the server acknowledges the client request to join the server */
	FOnJoiningGame& OnJoiningGameAck() { return JoiningGameAck; }
	

	/** @return true if this client is correctly logged in to the beacon, false otherwise */
	bool IsLoggedIn() const { return bLoggedIn; }

	/** Run a cheat command on the server */
	UFUNCTION(Reliable, Server, WithValidation)
	UE_API void ServerCheat(const FString& Msg);

protected:

	/** Has this beacon been properly logged in */
	bool bLoggedIn;

	/** True once the server has acknowledged our join intent */
	UPROPERTY()
	ELobbyBeaconJoinState LobbyJoinServerState;

	/** Session Id of the destination host */
	FString DestSessionId;
	
	/** Delegate broadcast when first connected to the lobby beacon (clientside) */
	FOnLobbyConnectionEstablished LobbyConnectionEstablished;
	/** Delegate broadcast when login is complete (clientside) */
	FOnLobbyLoginComplete LoginCompleteDelegate;
	/** Delegate broadcast when a new player joins (clientside) */
	FOnLobbyPlayerJoined PlayerJoinedDelegate;
	/** Delegate broadcast when an existing player leaves (clientside) */
	FOnLobbyPlayerLeft PlayerLeftDelegate;
	/** Delegate broadcast when this player is told to join the game by the server (clientside) */
	FOnJoiningGame JoiningGame;
	/** Delegate broadcast when the server acknowledges the client request to join the server (clientside) */
	FOnJoiningGame JoiningGameAck;
	/**
	 * Set the lobby state for this client beacon
	 * 
	 * @param InLobbyState reference to the lobby state
	 */
	UE_API void SetLobbyState(ALobbyBeaconState* InLobbyState);
	
	/**
	 * Internal function to log in a local players when first connected to the beacon
	 */
	UE_API virtual void LoginLocalPlayers();

	/**
	 * Attempt to login a single local player with the lobby beacon
	 *
	 * @param InSessionId session id that the client is expecting to connect with
	 * @param InUniqueId unique id of the new player
	 * @param UrlString URL containing player options (name, etc)
	 */
	UFUNCTION(server, reliable, WithValidation)
	UE_API virtual void ServerLoginPlayer(const FString& InSessionId, const FUniqueNetIdRepl& InUniqueId, const FString& UrlString);

	/**
	 * Make a graceful disconnect with the server
	 */
	UFUNCTION(server, reliable, WithValidation)
	UE_API virtual void ServerDisconnectFromLobby();

	/**
	 * Make a graceful request to actually join the server
	 */
	UFUNCTION(server, reliable, WithValidation)
	UE_API virtual void ServerNotifyJoiningServer();

	/**
	 * Acknowledge that client is traveling
	 */
	UE_API void AckJoiningServer();
	UFUNCTION(client, reliable)
	UE_API virtual void ClientAckJoiningServer();

	/**
	 * Make a request to kick a given player
	 *
	 * @param PlayerToKick player kick request
	 * @param Reason reason for the kick to tell client if this succeeds
	 */
	UFUNCTION(server, reliable, WithValidation)
	UE_API virtual void ServerKickPlayer(const FUniqueNetIdRepl& PlayerToKick, const FText& Reason);

	/**
	 * Make a request to set the party owner for the given player
	 *
	 * @param InUniqueId id of the requesting player
	 * @param PartyOwnerUniqueId id the party owner
	 */
	UFUNCTION(Server, Reliable, WithValidation)
	UE_API virtual void ServerSetPartyOwner(const FUniqueNetIdRepl& InUniqueId, const FUniqueNetIdRepl& InPartyOwnerId);

	/**
	 * Client notification result for a single login attempt
	 *
	 * @param InUniqueId id of player involved
	 * @param bWasSuccessful result of the login attempt
	 */
	UFUNCTION(client, reliable)
	UE_API void ClientLoginComplete(const FUniqueNetIdRepl& InUniqueId, bool bWasSuccessful);

	/**
	 * This was client was kicked by the server
	 *
	 * @param KickReason reason the server kicked the local player
	 */
	UFUNCTION(client, reliable)
	UE_API void ClientWasKicked(const FText& KickReason);

	/**
	 * Client notification that another player has joined the lobby
	 *
	 * @param NewPlayerName display name of new player
	 * @param InUniqueId unique id of new player
	 */
	UFUNCTION(client, reliable)
	UE_API virtual void ClientPlayerJoined(const FText& NewPlayerName, const FUniqueNetIdRepl& InUniqueId);

	/**
	 * Client notification that another player has left the lobby
	 *
	 * @param InUniqueId unique id of new player
	 */
	UFUNCTION(client, reliable)
	UE_API virtual void ClientPlayerLeft(const FUniqueNetIdRepl& InUniqueId);

	friend class ALobbyBeaconHost;
};

#undef UE_API
