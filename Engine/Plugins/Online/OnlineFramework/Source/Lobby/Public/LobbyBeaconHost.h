// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineBeaconHostObject.h"

#include "LobbyBeaconHost.generated.h"

#define UE_API LOBBY_API

LOBBY_API DECLARE_LOG_CATEGORY_EXTERN(LogLobbyBeacon, Log, All);

class ALobbyBeaconClient;
class ALobbyBeaconPlayerState;
class ALobbyBeaconState;
class AOnlineBeaconClient;
struct FJoinabilitySettings;
struct FUniqueNetIdRepl;

/**
 * Host object for maintaining a lobby before players actually join a server ready to receive them
 */
UCLASS(MinimalAPI, transient, config=Engine)
class ALobbyBeaconHost : public AOnlineBeaconHostObject
{
	GENERATED_UCLASS_BODY()

	//~ Begin AOnlineBeaconHostObject Interface 
	UE_API void NotifyClientDisconnected(AOnlineBeaconClient* LeavingClientActor);
	//~ End AOnlineBeaconHostObject Interface 

	/**
	 * Initialize the lobby beacon, creating an object to maintain state
	 *
	 * @param InSessionName name of session the beacon is associated with
	 */
	UE_API virtual bool Init(FName InSessionName);

	/**
	 * Create the lobby game state and associate it with the game
	 *
	 * @param InMaxPlayers max number of players allowed in the lobby
	 */
	UE_API virtual void SetupLobbyState(int32 InMaxPlayers);

	/**
	 * Update the party leader for a given player
	 *
	 * @param PartyMemberId player reporting a new party leader
	 * @param NewPartyLeaderId the new party leader
	 */
	UE_API void UpdatePartyLeader(const FUniqueNetIdRepl& PartyMemberId, const FUniqueNetIdRepl& NewPartyLeaderId);

	/**
	 * Actually kick a given player from the lobby
	 *
	 * @param ClientActor client connection to kick
	 * @param KickReason reason for the kick
	 */
	UE_API void KickPlayer(ALobbyBeaconClient* ClientActor, const FText& KickReason);

	/**
	 * Handle a detected disconnect of an existing player on the server
	 *
	 * @param InUniqueId unique id of the player
	 */
	UE_API virtual void HandlePlayerLogout(const FUniqueNetIdRepl& InUniqueId);

	/**
	 * Tell all connected beacon clients about the current joinability settings
	 *
	 * @param Setting current joinability settings
	 */
	UE_API virtual void AdvertiseSessionJoinability(const FJoinabilitySettings& Settings);

	/**
	 * Does the session associated with the beacon match the incoming request
	 *
	 * @param InSessionId incoming session id
	 *
	 * @return true if sessions match, false otherwise
	 */
	UE_API bool DoesSessionMatch(const FString& InSessionId) const;

	/**
	 * Output current state of beacon to log
	 */
	UE_API void DumpState() const;

protected:

	/** Name of session this beacon is associated with */
	FName SessionName;

	/** Class to use for the lobby beacon state */
	UPROPERTY()
	TSoftClassPtr<ALobbyBeaconState> LobbyStateClass;

	/** Actor representing the state of the lobby (similar to AGameState) */
	UPROPERTY()
	TObjectPtr<ALobbyBeaconState> LobbyState;


	UE_API virtual bool PreLogin(const FUniqueNetIdRepl& InUniqueId, const FString& Options);

	/**
	 * Notification call that a new lobby connection has been successfully establish
	 *
	 * @param ClientActor new lobby client connection
	 */
	UE_API virtual void PostLogin(ALobbyBeaconClient* ClientActor);

	/**
	 * Process the login for a given connection
	 *
	 * @param Client client beacon making the request
	 * @param InSessionId id of the session that is being checked
	 * @param InUniqueId id of the player logging in
	 * @param UrlString URL containing player options (name, etc)
	 */
	UE_API void ProcessLogin(ALobbyBeaconClient* ClientActor, const FString& InSessionId, const FUniqueNetIdRepl& InUniqueId, const FString& UrlString);

	/**
	 * Handle a request from a client when they are actually joining the server (needed for keeping player around when lobby beacon disconnects)
	 *
	 * @param ClientActor client that is making the request
	 */
	UE_API virtual bool ProcessJoinServer(ALobbyBeaconClient* ClientActor);

	/**
	 * Handle a request to disconnect a given client from the lobby beacon
	 * Notifies the owning beacon host to do its own cleanup
	 *
	 * @param ClientActor client that is making the request
	 */
	UE_API void ProcessDisconnect(ALobbyBeaconClient* ClientActor);

	/**
	 * Handle a request from a client to kick another player (may not succeed)
	 *
	 * @param Instigator player making the request
	 * @param PlayerToKick id of the player to kick
	 * @param Reason reason for the kick
	 *
	 * @return true if the player was kicked, false if not
	 */
	UE_API virtual bool ProcessKickPlayer(ALobbyBeaconClient* Instigator, const FUniqueNetIdRepl& PlayerToKick, const FText& Reason);

	/**
	 * Handle a player logging in via the host beacon
	 *
	 * @param ClientActor client that is making the request
	 * @param InUniqueId unique id of the player
	 * @param Options game options passed in by the client at login
	 *
	 * @return new player state object for the logged in player, null if there was any failure
	 */
	UE_API virtual ALobbyBeaconPlayerState* HandlePlayerLogin(ALobbyBeaconClient* ClientActor, const FUniqueNetIdRepl& InUniqueId, const FString& Options);

	friend ALobbyBeaconClient;
};

#undef UE_API
