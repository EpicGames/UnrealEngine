// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/Info.h"
#include "GameFramework/OnlineReplStructs.h"
#include "LobbyBeaconPlayerState.generated.h"

#define UE_API LOBBY_API

class AOnlineBeaconClient;

/**
 * Lightweight representation of a player while connected to the game through the lobby
 * exists for the lifetime of a player whether they are in the lobby or not
 * assumption that the data here doesn't change often and locks when they actually join the server
 */
UCLASS(MinimalAPI, transient, config = Game, notplaceable)
class ALobbyBeaconPlayerState : public AInfo
{
	GENERATED_UCLASS_BODY()

	//~ Begin AActor Interface
	UE_API virtual void PostInitializeComponents() override;
	UE_API virtual bool IsNetRelevantFor(const AActor* RealViewer, const AActor* ViewTarget, const FVector& SrcLocation) const override;
	//~ End AActor Interface

	/** @return true if this data structure is valid */
	UE_API bool IsValid() const;

	/** Visible friendly player name */
	UPROPERTY(Replicated)
	FText DisplayName;

	/** Player unique id */
	UPROPERTY(ReplicatedUsing = OnRep_UniqueId)
	FUniqueNetIdRepl UniqueId;

    /** Party owner id */
	UPROPERTY(ReplicatedUsing = OnRep_PartyOwner)
	FUniqueNetIdRepl PartyOwnerUniqueId;

	/** Is the player in the lobby or game */
	UPROPERTY( ReplicatedUsing = OnRep_InLobby )
	bool bInLobby;

	/** Reference to the beacon actor related to this player */
	UPROPERTY(Replicated)
	TObjectPtr<AOnlineBeaconClient> ClientActor;

	/**
	 * Delegate fired when this player state has changed in some way
	 *
	 * @param UniqueId id of the player that changed
	 */
	DECLARE_EVENT_OneParam( ALobbyBeaconPlayerState, FOnPlayerStateChanged, const FUniqueNetIdRepl& /** UniqueId */ );

	/** @return delegate fired when the unique id of the player has been replicated */
	inline FOnPlayerStateChanged& OnUniqueIdReplicated() { return UniqueIdReplicatedEvent; }
	/** @return delegate fired when the state of the player has changed in some way */
	inline FOnPlayerStateChanged& OnPlayerStateChanged() { return PlayerStateChangedEvent; }
	/** @return delegate fired when the party owner of this player has changed */
	inline FOnPlayerStateChanged& OnPartyOwnerChanged() { return PartyOwnerChangedEvent; }

protected:
	
	/** Unique Id has replicated */
	UFUNCTION()
	UE_API void OnRep_UniqueId();

	/** Party owner has changed */
	UFUNCTION()
	UE_API void OnRep_PartyOwner();

	/** Player has joined or left the lobby */
	UFUNCTION()
	UE_API void OnRep_InLobby();

private:
	/** Delegate fired when player unique id is replicated */
	FOnPlayerStateChanged UniqueIdReplicatedEvent;
	/** Delegate fired when player state changes */
	FOnPlayerStateChanged PlayerStateChangedEvent;
	/** Delegate fired when party owner changes */
	FOnPlayerStateChanged PartyOwnerChangedEvent;
};

#undef UE_API
