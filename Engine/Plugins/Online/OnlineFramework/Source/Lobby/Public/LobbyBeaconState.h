// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Net/Serialization/FastArraySerializer.h"
#include "GameFramework/Info.h"
#include "LobbyBeaconState.generated.h"

#define UE_API LOBBY_API

class ALobbyBeaconHost;

class ALobbyBeaconPlayerState;
class ALobbyBeaconState;
class AOnlineBeaconClient;
struct FLobbyPlayerStateInfoArray;
struct FUniqueNetIdRepl;

/**
 * Delegate fired when the lobby is open to players
 */
DECLARE_MULTICAST_DELEGATE(FOnLobbyStarted);

/**
 * Delegate fired as time counts down waiting for players in the lobby
 */
DECLARE_MULTICAST_DELEGATE(FOnLobbyWaitingForPlayersUpdate)

/**
 * Delegate fired when the player state in the lobby has changed (add/remove/etc)
 *
 * @param PlayerState player of interest (post addition / pre removal)
 */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnPlayerLobbyStateChanged, ALobbyBeaconPlayerState* /** PlayerState */);

/**
 * Replication structure for a single beacon player state
 */
USTRUCT()
struct FLobbyPlayerStateActorInfo : public FFastArraySerializerItem
{
	GENERATED_USTRUCT_BODY()

public:

	// Constructors
	FLobbyPlayerStateActorInfo() :
		LobbyPlayerState(nullptr)
	{}

	FLobbyPlayerStateActorInfo(ALobbyBeaconPlayerState* InPlayerState) :
		LobbyPlayerState(InPlayerState)
	{}

	/** Actual player state actor */
	UPROPERTY()
	TObjectPtr<ALobbyBeaconPlayerState> LobbyPlayerState;

	/** Player state removal */
	void PreReplicatedRemove(const FLobbyPlayerStateInfoArray& InArraySerializer);

	/**
	 * Player state additions 
	 * This may only be a notification that the array is growing and the actor data 
	 * isn't available yet, in which case the actual element will be null
	 */
	void PostReplicatedAdd(const FLobbyPlayerStateInfoArray& InArraySerializer);

	/**
	 * Player state element has changed
	 * In this specific case only happens when the actor pointer is set.
	 * This should only occur if the player state actor was already 
	 * serialized to the client at the time of the PostReplicatedAdd call
	 */
	void PostReplicatedChange(const FLobbyPlayerStateInfoArray& InArraySerializer);
};

/** Struct for fast TArray replication of lobby player state */
USTRUCT()
struct FLobbyPlayerStateInfoArray : public FFastArraySerializer
{
	GENERATED_USTRUCT_BODY()

	/** Implement support for fast TArray replication */
	bool NetDeltaSerialize(FNetDeltaSerializeInfo & DeltaParms)
	{
		return FFastArraySerializer::FastArrayDeltaSerialize<FLobbyPlayerStateActorInfo, FLobbyPlayerStateInfoArray>(Players, DeltaParms, *this);
	}

	/**
	 * Create and replicate the appropriate player state for the new player
	 *
	 * @param PlayerName display name of new player
	 * @param UniqueId unique id of the new player
	 *
	 * @return player state representation of the new player
	 */
	ALobbyBeaconPlayerState* AddPlayer(const FText& PlayerName, const FUniqueNetIdRepl& UniqueId);

	/**
	 * Remove an existing player from the lobby
	 *
	 * @param UniqueId unique id of the player to remove
	 */
	void RemovePlayer(const FUniqueNetIdRepl& UniqueId);

	/**
	 * Get an existing player
	 *
	 * @param UniqueId unique id for the player of interest
	 *
	 * @return player state representation of an existing player
	 */
	ALobbyBeaconPlayerState* GetPlayer(const FUniqueNetIdRepl& UniqueId);

	/**
	 * Get an existing player
	 *
	 * @param ClientActor beacon actor for the player of interest
	 *
	 * @return player state representation of an existing player
	 */
	ALobbyBeaconPlayerState* GetPlayer(const AOnlineBeaconClient* ClientActor);

	/**
	 * Get all players in the lobby
	 *
	 * @return all player representations in the lobby
	 */
	TArray<FLobbyPlayerStateActorInfo>& GetAllPlayers() { return Players; }
	const TArray<FLobbyPlayerStateActorInfo>& GetAllPlayers() const { return Players; }

	/** @return number of players currently in the lobby */
	int32 GetNumPlayers() const { return Players.Num(); }

	/**
	 * Output current state of players to log
	 */
	void DumpState() const;

private:

	/** All of the players in the lobby */
	UPROPERTY()
	TArray<FLobbyPlayerStateActorInfo> Players;

	/** Owning lobby beacon for this array of players */
	UPROPERTY(NotReplicated)
	TObjectPtr<ALobbyBeaconState> ParentState = nullptr;

	friend ALobbyBeaconState;
	friend FLobbyPlayerStateActorInfo;
};

/** Specified to allow fast TArray replication */
template<>
struct TStructOpsTypeTraits<FLobbyPlayerStateInfoArray> : public TStructOpsTypeTraitsBase2<FLobbyPlayerStateInfoArray>
{
	enum
	{
		WithNetDeltaSerializer = true,
	};
};

/**
 * Shared state of the game from the lobby perspective
 * Duplicates much of the data in the traditional AGameState object for sharing with players
 * connected via beacon only
 */
UCLASS(MinimalAPI, transient, config = Game, notplaceable)
class ALobbyBeaconState : public AInfo
{
	GENERATED_UCLASS_BODY()

	//~ Begin UObject Interface
	UE_API virtual void PostInitProperties() override;
	//~ End UObject Interface

	//~ Begin AActor Interface
	UE_API virtual void PostInitializeComponents() override;
	UE_API virtual bool IsNetRelevantFor(const AActor* RealViewer, const AActor* ViewTarget, const FVector& SrcLocation) const override;
	//~ End AActor Interface

	/**
	 * Start the lobby for incoming players
	 */
	UE_API void StartLobby();

	/**
	 * Start the waiting for other players (first player logged in success)
	 */
	UE_API virtual void StartWaiting();

	/**
	 * Determines whether the beacon state requires a full game before allowing a lobby to start; False by default
	 * 
	 * @return Whether the beacon state requires a full game before allowing a lobby to start; False by default
	 */
	UE_API virtual bool RequireFullLobbyToStart() const;

	/**
	 * Create and replicate the appropriate player state for the new player
	 *
	 * @param PlayerName display name of new player
	 * @param UniqueId unique id of the new player
	 *
	 * @return player state representation of the new player
	 */
	UE_API ALobbyBeaconPlayerState* AddPlayer(const FText& PlayerName, const FUniqueNetIdRepl& UniqueId);

	/**
	 * Remove an existing player from the lobby
	 *
	 * @param UniqueId unique id of the player to remove
	 */
	UE_API void RemovePlayer(const FUniqueNetIdRepl& UniqueId);

	/**
	 * Update the party leader for a given player
	 *
	 * @param PartyMemberId player reporting a new party leader
	 * @param NewPartyLeaderId the new party leader
	 */
	UE_API void UpdatePartyLeader(const FUniqueNetIdRepl& PartyMemberId, const FUniqueNetIdRepl& NewPartyLeaderId);

	/**
	 * Get an existing player in the lobby
	 *
	 * @param UniqueId unique id for the player of interest
	 *
	 * @return player state representation of an existing player
	 */
	UE_API ALobbyBeaconPlayerState* GetPlayer(const FUniqueNetIdRepl& UniqueId);

	/**
	* Get an existing player in the lobby
	*
	* @param UniqueId unique id for the player of interest passed as string
	*
	* @return player state representation of an existing player
	*/
	UE_API ALobbyBeaconPlayerState* GetPlayer(const FString& UniqueId);

	/**
	 * Get an existing player in the lobby
	 *
	 * @param ClientActor beacon actor for the player of interest
	 *
	 * @return player state representation of an existing player
	 */
	UE_API ALobbyBeaconPlayerState* GetPlayer(const AOnlineBeaconClient* ClientActor);

	/** @return delegate fired when a player is added to the lobby */
	FOnPlayerLobbyStateChanged& OnPlayerLobbyStateAdded() { return PlayerLobbyStateAdded; }

	/** @return delegate fired when a player is removed from the lobby */
	FOnPlayerLobbyStateChanged& OnPlayerLobbyStateRemoved() { return PlayerLobbyStateRemoved; }

	/** @return delegate fired when the lobby is started */
	FOnLobbyStarted& OnLobbyStarted() { return LobbyStarted; }

	/** @return delegate fired as time counts down waiting for players */
	FOnLobbyWaitingForPlayersUpdate& OnLobbyWaitingForPlayersUpdate() { return LobbyWaitingForPlayersUpdate; }

	/** @return true if the lobby has started, false otherwise */
	bool HasLobbyStarted() const { return bLobbyStarted; }

	/** @return number of players currently in the lobby */
	int32 GetNumPlayers() const { return Players.GetNumPlayers(); }

    /** @return number of players total allowed in the lobby */
	int32 GetMaxPlayers() const { return MaxPlayers; }

	/**
	 * Output current state of lobby to log
	 */
	UE_API void DumpState() const;
	
protected:

	/** Total number of players allowed in the lobby */
	UPROPERTY()
	int32 MaxPlayers;

	/** Class to use for lobby beacon player states */
	UPROPERTY()
	TSubclassOf<ALobbyBeaconPlayerState> LobbyBeaconPlayerStateClass;

	/** Time that last tick occurred for calculating wait time */
	double LastTickTime;

	/** Has the lobby already been started */
	UPROPERTY(ReplicatedUsing = OnRep_LobbyStarted)
	bool bLobbyStarted;

	/** Amount of time waiting for other players before starting the lobby */
	UPROPERTY(config, ReplicatedUsing = OnRep_WaitForPlayersTimeRemaining)
	float WaitForPlayersTimeRemaining;

	/** Array of players currently in the game, lobby or otherwise */
	UPROPERTY(Replicated)
	FLobbyPlayerStateInfoArray Players;

	/** Handle the lobby starting */
	UFUNCTION()
	UE_API void OnRep_LobbyStarted();

	/** Handle notification of time left to wait for lobby to start */
	UFUNCTION()
	UE_API void OnRep_WaitForPlayersTimeRemaining();

	/** Delegate fired when the lobby starts */
	FOnLobbyStarted LobbyStarted;

	/** Delegate fired as time counts down waiting for players in the lobby */
	FOnLobbyWaitingForPlayersUpdate LobbyWaitingForPlayersUpdate;

	/** Delegate fired when a player is added to the lobby */
	FOnPlayerLobbyStateChanged PlayerLobbyStateAdded;

	/** Delegate fired when a player is removed from the lobby */
	FOnPlayerLobbyStateChanged PlayerLobbyStateRemoved;

	/** Handle to manage a one sec timer */
	FTimerHandle OneSecTimerHandle;

	/**
	 * Create a new player for this lobby beacon state
	 *
	 * @param PlayerName name of the player
	 * @param UniqueId uniqueid of the player
	 *
	 * @return new player state actor or nullptr if the player should be disconnected
	 */
	UE_API virtual ALobbyBeaconPlayerState* CreateNewPlayer(const FText& PlayerName, const FUniqueNetIdRepl& UniqueId);

	/** One sec tick function to handle countdowns */
	UE_API void OneSecTick();

	/**
	 * Internal helper function called as part of OneSecTick if the lobby hasn't been started yet
	 * 
	 * @param DeltaTime	Time slice since last tick
	 */
	UE_API virtual void OnPreLobbyStartedTickInternal(double DeltaTime);
	
	/**
	 * Internal helper function called as part of OneSecTick if the lobby has been started already
	 * 
	 * @Param DeltaTime	Time slice since last tick
	 */
	UE_API virtual void OnPostLobbyStartedTickInternal(double DeltaTime);

	friend ALobbyBeaconHost;
	friend FLobbyPlayerStateInfoArray;
};

#undef UE_API
