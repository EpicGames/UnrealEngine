// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class LoginFlow : ModuleRules
{
	public LoginFlow(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreOnline",
				"CoreUObject",
				"InputCore",
				"Slate",
			}
		);

		PrivateDependencyModuleNames.AddRange(
		new string[]
			{
				"SlateCore",
				"WebBrowser",
				"OnlineSubsystem",
			}
		);

		DynamicallyLoadedModuleNames.AddRange(
		new string[] {
				"Analytics",
				"AnalyticsET",
			}
		);

		PrivateIncludePathModuleNames.AddRange(
			new string[] {
			}
		);
	}
}
