// Copyright Epic Games, Inc. All Rights Reserved.

#include "LoginFlowManager.h"
#include "ILoginFlowModule.h"
#include "Interfaces/OnlineIdentityInterface.h"

#include "IWebBrowserSingleton.h"
#include "Misc/Paths.h"
#include "WebBrowserModule.h"
#include "IWebBrowserCookieManager.h"

#include "OnlineSubsystem.h"
#include "Widgets/Layout/SBox.h"
#include "Framework/Application/SlateApplication.h"

DEFINE_LOG_CATEGORY(LogLoginFlow);

struct FLoginFlowProperties
{
	/** Instance of the login flow */
	FString InstanceId;
	/** Delegate fired on every RedirectURL detected by the web interface */
	FOnLoginRedirectURL OnRedirectURL;
	/** Wrapper slate widget around the actual login flow web page */
	TSharedPtr<SBox> PopupHolder;
	/** Delegate fired externally when the login flow is dismissed */
	FLoginFlowManager::FOnPopupDismissed OnPopupDismissed;
	/** Delegate fired when the login flow is complete for any reason */
	FOnLoginFlowComplete OnComplete;
	/** Structure containing results of login flow attempt */
	FLoginFlowResult Result;
	/** Is the login flow actively being shown */
	bool bIsDisplayed;
};

struct FAccountCreationFlowProperties
{
	/** Instance of the login flow */
	FString InstanceId;
	/** Delegate fired on every RedirectURL detected by the web interface */
	FOnLoginRedirectURL OnRedirectURL;
	/** Wrapper slate widget around the actual creation flow web page */
	TSharedPtr<SBox> PopupHolder;
	/** Delegate fired externally when the creation flow is dismissed */
	FLoginFlowManager::FOnPopupDismissed OnPopupDismissed;
	/** Delegate fired when the creation flow is complete for any reason */
	FOnLoginFlowComplete OnComplete;
	/** Structure containing results of creation flow attempt */
	FLoginFlowResult Result;
	/** Is the creation flow actively being shown */
	bool bIsDisplayed;
};

FLoginFlowManager::FLoginFlowManager()
{
}

FLoginFlowManager::~FLoginFlowManager()
{
	Reset();
}

void FLoginFlowManager::Reset()
{
	// if we're in an active flow, just fire dismissal then shut down
	if (PendingLogin.IsValid())
	{
		PendingLogin->OnPopupDismissed.ExecuteIfBound();
		PendingLogin = nullptr;
	}

	if (PendingAccountCreation.IsValid())
	{
		PendingAccountCreation->OnPopupDismissed.ExecuteIfBound();
		PendingAccountCreation = nullptr;
	}

	for (auto& OnlineParamPair : OnlineSubsystemsMap)
	{
		FOnlineParams& OnlineParams = OnlineParamPair.Value;
		IOnlineSubsystem* OnlineSub = IOnlineSubsystem::Get(OnlineParams.OnlineIdentifier);
		if (OnlineSub != nullptr)
		{
			IOnlineExternalUIPtr OnlineExternalUI = OnlineSub->GetExternalUIInterface();
			IOnlineIdentityPtr OnlineIdentity = OnlineSub->GetIdentityInterface();
			if (OnlineExternalUI.IsValid())
			{
				OnlineExternalUI->ClearOnLoginFlowUIRequiredDelegate_Handle(OnlineParams.LoginFlowUIRequiredDelegateHandle);
				OnlineExternalUI->ClearOnCreateAccountFlowUIRequiredDelegate_Handle(OnlineParams.AccountCreationFlowUIRequiredDelegateHandle);
			}
			if (OnlineIdentity.IsValid())
			{
				OnlineIdentity->ClearOnLoginFlowLogoutDelegate_Handle(OnlineParams.LoginFlowLogoutDelegateHandle);
			}
		}
	}
	OnlineSubsystemsMap.Empty();
}

bool FLoginFlowManager::AddLoginFlow(FName OnlineIdentifier, const FOnDisplayPopup& InPopupDelegate, const FOnDisplayPopup& InCreationFlowPopupDelegate, bool bPersistCookies, bool bConsumeInput)
{
	bool bSuccess = false;

	FOnlineParams* Params = OnlineSubsystemsMap.Find(OnlineIdentifier);
	if (Params == nullptr)
	{
		IOnlineSubsystem* OnlineSub = IOnlineSubsystem::Get(OnlineIdentifier);
		if (OnlineSub != nullptr)
		{
			// get information from OSS and bind to the OSS delegate
			IOnlineIdentityPtr OnlineIdentity = OnlineSub->GetIdentityInterface();
			IOnlineExternalUIPtr OnlineExternalUI = OnlineSub->GetExternalUIInterface();
			if (OnlineIdentity.IsValid() && OnlineExternalUI.IsValid())
			{				
				FString ContextName = FString::Printf(TEXT("LoginFlowContext_%s"), *OnlineIdentifier.ToString());

				FOnlineParams& NewParams = OnlineSubsystemsMap.Add(OnlineIdentifier);
				NewParams.OnlineIdentifier = OnlineIdentifier;
				NewParams.OnLoginFlowPopup = InPopupDelegate;
				NewParams.OnAccountCreationFlowPopup = InCreationFlowPopupDelegate;
				NewParams.BrowserContextSettings = MakeShared<FBrowserContextSettings>(ContextName);
				NewParams.BrowserContextSettings->bPersistSessionCookies = bPersistCookies;
				NewParams.bRegisteredContext = false;
				NewParams.bConsumeInput = bConsumeInput;

				NewParams.LoginFlowLogoutDelegateHandle = OnlineIdentity->AddOnLoginFlowLogoutDelegate_Handle(FOnLoginFlowLogoutDelegate::CreateSP(this, &FLoginFlowManager::OnLoginFlowLogout, OnlineIdentifier));
				
				NewParams.LoginFlowUIRequiredDelegateHandle = OnlineExternalUI->AddOnLoginFlowUIRequiredDelegate_Handle(FOnLoginFlowUIRequiredDelegate::CreateSP(this, &FLoginFlowManager::OnLoginFlowStarted, OnlineIdentifier));
				NewParams.AccountCreationFlowUIRequiredDelegateHandle = OnlineExternalUI->AddOnCreateAccountFlowUIRequiredDelegate_Handle(FOnCreateAccountFlowUIRequiredDelegate::CreateSP(this, &FLoginFlowManager::OnAccountCreationFlowStarted, OnlineIdentifier));
				bSuccess = true;
			}
		}
		else
		{
			UE_LOGF(LogLoginFlow, Warning, "No OSS specified. Login flow will be disabled.");
		}
	}
	else
	{
		UE_LOGF(LogLoginFlow, Warning, "OSS already registered, skipping [%ls]", *OnlineIdentifier.ToString());
	}

	return bSuccess;
}

void FLoginFlowManager::RegisterBrowserContext(FOnlineParams* Params)
{
	if (!Params->bRegisteredContext)
	{
		IWebBrowserSingleton* WebBrowserSingleton = IWebBrowserModule::Get().GetSingleton();
#if !UE_BUILD_SHIPPING
		WebBrowserSingleton->SetDevToolsShortcutEnabled(true);
#endif
		if (Params->BrowserContextSettings->bPersistSessionCookies)
		{
			// Taken from FWebBrowserSingleton
			FString CachePath(FPaths::Combine(WebBrowserSingleton->ApplicationCacheDir(), TEXT("webcache")));
			CachePath = FPaths::ConvertRelativePathToFull(CachePath);
			Params->BrowserContextSettings->CookieStorageLocation = CachePath;
		}

		if (!WebBrowserSingleton->RegisterContext(*Params->BrowserContextSettings))
		{
			UE_LOGF(LogLoginFlow, Warning, "Failed to register context in web browser singleton for %ls", *Params->BrowserContextSettings->Id);
		}

		Params->bRegisteredContext = true;
	}
}

bool FLoginFlowManager::HasLoginFlow(FName OnlineIdentifier)
{
	return OnlineSubsystemsMap.Find(OnlineIdentifier) != nullptr;
}

void FLoginFlowManager::OnLoginFlowStarted(const FString& RequestedURL, const FOnLoginRedirectURL& OnRedirectURL, const FOnLoginFlowComplete& OnLoginFlowComplete, bool& bOutShouldContinueLogin, FName InOnlineIdentifier)
{
	bOutShouldContinueLogin = false;

	FOnlineParams* Params = OnlineSubsystemsMap.Find(InOnlineIdentifier);
	if (Params)
	{
		// check for login in flight (one at a time)
		if (IsFlowInProgress())
		{
			UE_LOGF(LogLoginFlow, Error, "Simultaneous login flows not supported");
			return;
		}

		// make sure we have a display callback currently bound
		if (!Params->OnLoginFlowPopup.IsBound())
		{
			UE_LOGF(LogLoginFlow, Error, "Login did not have display code bound to OnLoginFlowStarted.");
			return;
		}

		bOutShouldContinueLogin = true;

		RegisterBrowserContext(Params);

		// save the pending order for reference later
		PendingLogin = MakeUnique<FLoginFlowProperties>();
		PendingLogin->InstanceId = FGuid::NewGuid().ToString();
		PendingLogin->OnRedirectURL = OnRedirectURL;
		PendingLogin->OnComplete = OnLoginFlowComplete;
		PendingLogin->bIsDisplayed = false;

		// create a placeholder widget to display while this process is going on
		SAssignNew(PendingLogin->PopupHolder, SBox);

		// give the widget to the App to display and get back a callback we should use to dismiss it
		PendingLogin->OnPopupDismissed = Params->OnLoginFlowPopup.Execute(PendingLogin->PopupHolder.ToSharedRef());

		// generate a login flow chromium widget
		ILoginFlowModule::FCreateSettings CreateSettings;
		CreateSettings.Url = RequestedURL;
		CreateSettings.bConsumeInput = Params->bConsumeInput;
		CreateSettings.BrowserContextSettings = Params->BrowserContextSettings;

		// Setup will allow login flow module events to trigger passed in delegates
		CreateSettings.CloseCallback = FOnLoginFlowRequestClose::CreateSP(this, &FLoginFlowManager::OnLoginFlow_Close, PendingLogin->InstanceId);
		CreateSettings.ErrorCallback = FOnLoginFlowError::CreateSP(this, &FLoginFlowManager::OnLoginFlow_Error, PendingLogin->InstanceId);
		CreateSettings.RedirectCallback = FOnLoginFlowRedirectURL::CreateSP(this, &FLoginFlowManager::OnLoginFlow_RedirectURL, PendingLogin->InstanceId);
		TSharedRef<SWidget> LoginFlowWidget = ILoginFlowModule::Get().CreateLoginFlowWidget(CreateSettings);

		// make sure none of the delegates were called during creation (would invalidate this ptr)
		if (PendingLogin.IsValid())
		{
			// put this widget into the holder
			PendingLogin->PopupHolder->SetContent(LoginFlowWidget);
			PendingLogin->bIsDisplayed = true;

			// focus the login flow widget
			FSlateApplication::Get().SetKeyboardFocus(LoginFlowWidget, EFocusCause::SetDirectly);
		}
	}
	else
	{
		UE_LOGF(LogLoginFlow, Error, "Online platform requesting login flow not registered [%ls]", *InOnlineIdentifier.ToString());
	}
}

void FLoginFlowManager::OnLoginFlow_Error(ELoginFlowErrorResult ErrorType, const FString& ErrorInfo, FString InstanceId)
{
	if (!PendingLogin.IsValid() || PendingLogin->InstanceId != InstanceId)
	{
		// assume we got canceled
		return;
	}

	FString ErrorString = ErrorInfo;
	if (ErrorString.IsEmpty())
	{
		// try to get more info from the type
		switch (ErrorType)
		{
		case ELoginFlowErrorResult::LoadFail:
			ErrorString = PendingLogin->bIsDisplayed ? LOGIN_PAGELOADFAILED : LOGIN_CEFLOADFAILED;
			break;
		default:
			ErrorString = LOGIN_ERROR_UNKNOWN;
			break;
		}
	}

	PendingLogin->Result.Error.SetFromErrorCode(ErrorString);
	FinishLogin();
}

void FLoginFlowManager::OnLoginFlow_Close(const FString& CloseInfo, FString InstanceId)
{
	if (!PendingLogin.IsValid() || PendingLogin->InstanceId != InstanceId)
	{
		// assume we got canceled
		return;
	}
 
	PendingLogin->Result.Error.SetFromErrorCode(CloseInfo);
	FinishLogin();
}

bool FLoginFlowManager::OnLoginFlow_RedirectURL(const FString& RedirectURL, FString InstanceId)
{
	if (!PendingLogin.IsValid() || PendingLogin->InstanceId != InstanceId)
	{
		return false;
	}

	FLoginFlowResult Result = PendingLogin->OnRedirectURL.Execute(RedirectURL);
	if (Result.IsComplete())
	{
		PendingLogin->Result = Result;
		FinishLogin();
		return true;
	}

	return false;
}

void FLoginFlowManager::CancelLoginFlow()
{
	if (!PendingLogin.IsValid() || PendingLogin->Result.IsComplete())
	{
		return;
	}

	PendingLogin->Result.Error.SetFromErrorCode(LOGIN_CANCELLED);
	FinishLogin();
}

void FLoginFlowManager::FinishLogin()
{
	check(PendingLogin.IsValid());

	if (!PendingLogin->Result.Error.WasSuccessful())
	{
		UE_LOGF(LogLoginFlow, Warning, "Login Flow failed with error: %ls", *PendingLogin->Result.Error.ToLogString());
	}

	// fire the login complete callback
	PendingLogin->OnComplete.ExecuteIfBound(PendingLogin->Result);
	
	// dismiss the popup
	PendingLogin->OnPopupDismissed.ExecuteIfBound();

	// clear this object so we can handle another login
	PendingLogin.Reset();
}


void FLoginFlowManager::OnAccountCreationFlow_Error(ELoginFlowErrorResult ErrorType, const FString& ErrorInfo, FString InstanceId)
{
	if (!PendingAccountCreation.IsValid() || PendingAccountCreation->InstanceId != InstanceId)
	{
		// assume we got canceled
		return;
	}

	FString ErrorString = ErrorInfo;
	if (ErrorString.IsEmpty())
	{
		// try to get more info from the type
		switch (ErrorType)
		{
			case ELoginFlowErrorResult::LoadFail:
				ErrorString = PendingAccountCreation->bIsDisplayed ? LOGIN_PAGELOADFAILED : LOGIN_CEFLOADFAILED;
				break;
			default:
				ErrorString = LOGIN_ERROR_UNKNOWN;
				break;
		}
	}

	PendingAccountCreation->Result.Error.SetFromErrorCode(ErrorString);
	FinishAccountCreation();
}

void FLoginFlowManager::OnAccountCreationFlow_Close(const FString& CloseInfo, FString InstanceId)
{
	if (!PendingAccountCreation.IsValid() || PendingAccountCreation->InstanceId != InstanceId)
	{
		// assume we got canceled
		return;
	}

	PendingAccountCreation->Result.Error.SetFromErrorCode(CloseInfo);
	FinishAccountCreation();
}

bool FLoginFlowManager::OnAccountCreationFlow_RedirectURL(const FString& RedirectURL, FString InstanceId)
{
	if (!PendingAccountCreation.IsValid() || PendingAccountCreation->InstanceId != InstanceId)
	{
		return false;
	}

	FLoginFlowResult Result = PendingAccountCreation->OnRedirectURL.Execute(RedirectURL);
	if (Result.IsComplete())
	{
		PendingAccountCreation->Result = Result;
		FinishAccountCreation();
		return true;
	}

	return false;
}

void FLoginFlowManager::OnAccountCreationFlowStarted(const FString& RequestedURL, const FOnLoginRedirectURL& OnRedirectURL, const FOnLoginFlowComplete& OnAccountCreationFlowComplete, bool& bOutShouldContinueAccountCreation, FName InOnlineIdentifier)
{
	bOutShouldContinueAccountCreation = false;

	FOnlineParams* Params = OnlineSubsystemsMap.Find(InOnlineIdentifier);
	if (Params)
	{
		// check for login in flight (one at a time)
		if (IsFlowInProgress())
		{
			UE_LOGF(LogLoginFlow, Error, "Simultaneous login flows not supported");
			return;
		}

		// make sure we have a display callback currently bound
		if (!Params->OnAccountCreationFlowPopup.IsBound())
		{
			UE_LOGF(LogLoginFlow, Error, "Account creation did not have display code bound to OnLoginFlowStarted.");
			return;
		}

		bOutShouldContinueAccountCreation = true;

		RegisterBrowserContext(Params);

		// save the pending order for reference later
		PendingAccountCreation = MakeUnique<FAccountCreationFlowProperties>();
		PendingAccountCreation->InstanceId = FGuid::NewGuid().ToString();
		PendingAccountCreation->OnRedirectURL = OnRedirectURL;
		PendingAccountCreation->OnComplete = OnAccountCreationFlowComplete;
		PendingAccountCreation->bIsDisplayed = false;

		// create a placeholder widget to display while this process is going on
		SAssignNew(PendingAccountCreation->PopupHolder, SBox);

		// give the widget to the App to display and get back a callback we should use to dismiss it
		PendingAccountCreation->OnPopupDismissed = Params->OnAccountCreationFlowPopup.Execute(PendingAccountCreation->PopupHolder.ToSharedRef());

		// generate a login flow chromium widget
		ILoginFlowModule::FCreateSettings CreateSettings;
		CreateSettings.Url = RequestedURL;
		CreateSettings.BrowserContextSettings = Params->BrowserContextSettings;

		// Setup will allow login flow module events to trigger passed in delegates
		CreateSettings.CloseCallback = FOnLoginFlowRequestClose::CreateSP(this, &FLoginFlowManager::OnAccountCreationFlow_Close, PendingAccountCreation->InstanceId);
		CreateSettings.ErrorCallback = FOnLoginFlowError::CreateSP(this, &FLoginFlowManager::OnAccountCreationFlow_Error, PendingAccountCreation->InstanceId);
		CreateSettings.RedirectCallback = FOnLoginFlowRedirectURL::CreateSP(this, &FLoginFlowManager::OnAccountCreationFlow_RedirectURL, PendingAccountCreation->InstanceId);
		TSharedRef<SWidget> AccountCreationFlowWidget = ILoginFlowModule::Get().CreateLoginFlowWidget(CreateSettings);

		// make sure none of the delegates were called during creation (would invalidate this ptr)
		if (PendingAccountCreation.IsValid())
		{
			// put this widget into the holder
			PendingAccountCreation->PopupHolder->SetContent(AccountCreationFlowWidget);
			PendingAccountCreation->bIsDisplayed = true;

			// focus the login flow widget
			FSlateApplication::Get().SetKeyboardFocus(AccountCreationFlowWidget, EFocusCause::SetDirectly);
		}
	}
	else
	{
		UE_LOGF(LogLoginFlow, Error, "Online platform requesting account creation flow not registered [%ls]", *InOnlineIdentifier.ToString());
	}
}

void FLoginFlowManager::CancelAccountCreationFlow()
{
	if (!PendingAccountCreation.IsValid() || PendingAccountCreation->Result.IsComplete())
	{
		return;
	}

	PendingAccountCreation->Result.Error.SetFromErrorCode(LOGIN_CANCELLED);
	FinishAccountCreation();
}

void FLoginFlowManager::FinishAccountCreation()
{
	check(PendingAccountCreation.IsValid());

	if (!PendingAccountCreation->Result.Error.WasSuccessful())
	{
		UE_LOGF(LogLoginFlow, Warning, "Account Creation Flow failed with error: %ls", *PendingAccountCreation->Result.Error.ToLogString());
	}

	// fire the login complete callback
	PendingAccountCreation->OnComplete.ExecuteIfBound(PendingAccountCreation->Result);

	// dismiss the popup
	PendingAccountCreation->OnPopupDismissed.ExecuteIfBound();

	// clear this object so we can handle another login
	PendingAccountCreation.Reset();
}

void FLoginFlowManager::OnLoginFlowLogout(const TArray<FString>& LoginDomains, FName InOnlineIdentifier)
{
	FOnlineParams* Params = OnlineSubsystemsMap.Find(InOnlineIdentifier);
	if (Params)
	{
		if (LoginDomains.Num() > 0)
		{
			IWebBrowserSingleton* WebBrowserSingleton = IWebBrowserModule::Get().GetSingleton();
			if (WebBrowserSingleton)
			{
				TOptional<FString> ContextId;
				if (Params->BrowserContextSettings.IsValid())
				{
					ContextId = Params->BrowserContextSettings->Id;
				}
				TSharedPtr<IWebBrowserCookieManager> CookieManager = WebBrowserSingleton->GetCookieManager(ContextId);
				if (CookieManager.IsValid())
				{
					for (const FString& LoginDomain : LoginDomains)
					{
						CookieManager->DeleteCookies(LoginDomain);
					}
				}
			}
		}
	}
	else
	{
		UE_LOGF(LogLoginFlow, Error, "No login flow registered for online subsystem %ls", *InOnlineIdentifier.ToString());
	}
}


