// Copyright Epic Games, Inc. All Rights Reserved.

#include "LoginFlowViewModel.h"
#include "LoginFlowManager.h"
#include "SLoginFlow.h"

/**
 * Implements the LoginFlow module.
 */
class FLoginFlowModule
	: public ILoginFlowModule
{
public:

	// ILoginFlowModule interface

	virtual TSharedPtr<ILoginFlowManager> CreateLoginFlowManager() const override
	{
		return MakeShared<FLoginFlowManager>();
	}

	virtual TSharedRef<class SWidget> CreateLoginFlowWidget(const ILoginFlowModule::FCreateSettings& Settings) const override
	{
		TSharedRef<FLoginFlowViewModel> LoginFlowViewModel = FLoginFlowViewModelFactory::Create(
			Settings.Url, 
			Settings.BrowserContextSettings,
			Settings.CloseCallback, 
			Settings.ErrorCallback,
			Settings.RedirectCallback, 
			Settings.bConsumeInput);

		TSharedPtr<SWidget> LoginFlowWidget = nullptr;
		if (Settings.StyleSet)
		{
			LoginFlowWidget = SNew(SLoginFlow, LoginFlowViewModel).StyleSet(Settings.StyleSet);
		}
		else
		{
			LoginFlowWidget = SNew(SLoginFlow, LoginFlowViewModel);
		}

		return LoginFlowWidget.ToSharedRef();
	}


public:

	// IModuleInterface interface

	virtual void StartupModule() override
	{
	}

	virtual void ShutdownModule() override
	{
	}
};


IMPLEMENT_MODULE(FLoginFlowModule, LoginFlow);
