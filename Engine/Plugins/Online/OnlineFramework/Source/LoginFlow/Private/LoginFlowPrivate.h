// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

 /* Dependencies
 *****************************************************************************/


 /* Interfaces
 *****************************************************************************/




#define FACTORY(ReturnType, Type, ...) \
class Type##Factory \
{ \
public: \
	static ReturnType Create(__VA_ARGS__); \
}; 

#define IFACTORY(ReturnType, Type, ...) \
class Type##Factory \
{ \
public: \
	virtual ReturnType Create(__VA_ARGS__) = 0; \
};

