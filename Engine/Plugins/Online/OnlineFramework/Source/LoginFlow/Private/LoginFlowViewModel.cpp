// Copyright Epic Games, Inc. All Rights Reserved.

#include "LoginFlowViewModel.h"

#include "HAL/PlatformProcess.h"
#include "Internationalization/Text.h"

class FLoginFlowViewModelFactory;
class FLoginFlowViewModelFactoryFactory;

class FLoginFlowViewModelImpl 
	: public FLoginFlowViewModel
{
public:

	virtual ~FLoginFlowViewModelImpl()
	{
	}

	virtual FString GetLoginFlowUrl() override
	{
		return LoginFlowStartingUrl;
	}

	virtual void HandleRequestClose(const FString& CloseInfo) override
	{
		OnRequestClose.ExecuteIfBound(CloseInfo);
	}

	virtual void HandleLoadError() override
	{
		OnError.ExecuteIfBound(ELoginFlowErrorResult::LoadFail, TEXT(""));
	}

	virtual bool HandleBrowserUrlChanged(const FText& Url) override
	{
		#if (0) // HandleBeforeBrowse seems to do all that is required atm
		{
			if (OnRedirectURL.IsBound())
			{
				return OnRedirectURL.Execute(Url.ToString());
			}
		}
		#endif
		return false;
	}

	virtual bool HandleBeforeBrowse(const FString& Url) override
	{
		if (OnRedirectURL.IsBound())
		{
			return OnRedirectURL.Execute(Url);
		}
		return false;
	}

	virtual bool HandleNavigation(const FString& Url) override
	{
		bool bHandled = true;
		check(!Url.IsEmpty())
		FPlatformProcess::LaunchURL(*Url, nullptr, nullptr);
		return bHandled;
	}

	virtual const TSharedPtr<FBrowserContextSettings>& GetBrowserContextSettings() const
	{
		return BrowserContextSettings;
	}

	virtual bool ShouldConsumeInput() const override
	{
		return bConsumeInput;
	}

private:

	FLoginFlowViewModelImpl(
		const FString& InLoginFlowStartingUrl,
		const TSharedPtr<FBrowserContextSettings>& InBrowserContextSettings,
		const FOnLoginFlowRequestClose& InOnRequestClose,
		const FOnLoginFlowError& InOnError,
		const FOnLoginFlowRedirectURL& InOnRedirectURL,
		bool bInConsumeInput)
		: LoginFlowStartingUrl(InLoginFlowStartingUrl)
		, BrowserContextSettings(InBrowserContextSettings)
		, OnRequestClose(InOnRequestClose)
		, OnError(InOnError)
		, OnRedirectURL(InOnRedirectURL)
		, bConsumeInput(bInConsumeInput)
	{ }

	void Initialize()
	{
	}
	
private:

	const FString LoginFlowStartingUrl;
	const TSharedPtr<FBrowserContextSettings> BrowserContextSettings;
	const FOnLoginFlowRequestClose OnRequestClose;
	const FOnLoginFlowError OnError;
	const FOnLoginFlowRedirectURL OnRedirectURL;
	bool bConsumeInput;

	friend FLoginFlowViewModelFactory;
};

TSharedRef<FLoginFlowViewModel> FLoginFlowViewModelFactory::Create(
	const FString& LoginFlowStartingUrl,
	const TSharedPtr<FBrowserContextSettings>& BrowserContextSettings,
	const FOnLoginFlowRequestClose& OnRequestClose,
	const FOnLoginFlowError& OnError,
	const FOnLoginFlowRedirectURL& OnRedirectURL,
	bool bConsumeInput)
{
	TSharedRef< FLoginFlowViewModelImpl > ViewModel = MakeShareable(
		new FLoginFlowViewModelImpl(
				LoginFlowStartingUrl,
				BrowserContextSettings,
				OnRequestClose,
				OnError,
				OnRedirectURL,
				bConsumeInput));
	ViewModel->Initialize();
	return ViewModel;
}
