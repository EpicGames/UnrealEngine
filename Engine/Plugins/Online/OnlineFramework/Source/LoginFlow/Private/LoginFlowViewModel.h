// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "ILoginFlowModule.h"
#include "LoginFlowPrivate.h"

struct FBrowserContextSettings;

/**
 * Represents the view transformation for the login flow.
 */
class FLoginFlowViewModel 
	: public TSharedFromThis<FLoginFlowViewModel>
{
public:
	virtual ~FLoginFlowViewModel() = default;

	/** @return the starting web page */
	virtual FString GetLoginFlowUrl() = 0;

	/**
	 * Handle a request to close the browser window
	 *
	 * @param CloseInfo reason for the closure
	 */
	virtual void HandleRequestClose(const FString& CloseInfo) = 0;

	/**
	 * Handle a browser window load error
	 */
	virtual void HandleLoadError() = 0;
	
	/**
	 * Browser window URL has changed
	 *
	 * @param Url new location
	 */
	virtual bool HandleBrowserUrlChanged(const FText& Url) = 0;

	virtual bool HandleBeforeBrowse(const FString& Url) = 0;

	virtual bool HandleNavigation(const FString& Url) = 0;

	/** @return the context settings for the underlying browser */
	virtual const TSharedPtr<FBrowserContextSettings>& GetBrowserContextSettings() const = 0;

	/** @return whether or not the login flow should consume any input not handled by the browser window */
	virtual bool ShouldConsumeInput() const = 0;
};

FACTORY(TSharedRef<FLoginFlowViewModel>, FLoginFlowViewModel,
		 const FString& HomePage,
		 const TSharedPtr<FBrowserContextSettings>& BrowserContextSettings,
		 const FOnLoginFlowRequestClose& OnRequestClose,
		 const FOnLoginFlowError& OnError,
	     const FOnLoginFlowRedirectURL& OnRedirectURL,
	     const bool bConsumeInput);
