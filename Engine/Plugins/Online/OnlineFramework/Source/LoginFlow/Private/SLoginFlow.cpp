// Copyright Epic Games, Inc. All Rights Reserved.

#include "SLoginFlow.h"
#include "LoginFlowViewModel.h"
#include "SWebBrowser.h"
#include "IWebBrowserWindow.h"
#include "IWebBrowserPopupFeatures.h"
#include "IWebBrowserDialog.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/SWindow.h"

class SLoginFlowImpl 
	: public SLoginFlow
{
public:
	virtual void Construct(const FArguments& InArgs, const TSharedRef<FLoginFlowViewModel>& InViewModel) override
	{
		bEncounteredError = false;
		bOpenDevTools = false;
		ViewModel = InViewModel;
		StyleSet = InArgs._StyleSet;

		FString HomeUrl = ViewModel->GetLoginFlowUrl();

		SUserWidget::Construct(SUserWidget::FArguments()
		[
			SNew(SOverlay)
			+ SOverlay::Slot()
			[
				SAssignNew(BrowserContainer, SOverlay)
				+ SOverlay::Slot()
				[
					SAssignNew(MainBrowser, SWebBrowserView)
					.ShowErrorMessage(false)
					.SupportsTransparency(true)
					.InitialURL(HomeUrl)
					.BackgroundColor(StyleSet->GetColor("LoginFlow.ContentArea.BackgroundColor").ToFColor(true))
					.OnLoadError(this, &SLoginFlowImpl::HandleLoadError)
					.OnLoadUrl(this, &SLoginFlowImpl::HandleLoadUrl)
					.OnUrlChanged(this, &SLoginFlowImpl::HandleBrowserUrlChanged)
					.OnBeforePopup(this, &SLoginFlowImpl::HandleBeforePopup)
					.OnBeforeNavigation(this, &SLoginFlowImpl::HandleBrowserBeforeBrowse)
					.OnCreateWindow(this, &SLoginFlowImpl::HandleBrowserCreateWindow)
					.OnCloseWindow(this, &SLoginFlowImpl::HandleBrowserCloseWindow)
					.OnShowDialog(this, &SLoginFlowImpl::HandleShowDialog)
					.OnUnhandledKeyDown(this, &SLoginFlowImpl::HandleKey)
					.OnUnhandledKeyUp(this, &SLoginFlowImpl::HandleKey)
					.OnUnhandledKeyChar(this, &SLoginFlowImpl::HandleKeyChar)
					.ContextSettings(ViewModel->GetBrowserContextSettings().IsValid() ? *ViewModel->GetBrowserContextSettings() : TOptional<FBrowserContextSettings>())
				]
			]
			+ SOverlay::Slot()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				+ SHorizontalBox::Slot()
				.Padding(0, 50, 70, 0)
				.VAlign(VAlign_Top)
				.AutoWidth()
				[
					SNew(SButton)
						//.ButtonStyle(StyleSet, "LoginFlow.ContentArea.CloseButton")
						.OnClicked(this, &SLoginFlowImpl::HandleCloseOverlayClicked)
						.Visibility(this, &SLoginFlowImpl::GetCloseOverlayVisibility)
						.Cursor(EMouseCursor::Hand)
				]
			]
		]);

		if(HomeUrl.IsEmpty())
		{
			bEncounteredError = true;
		}
	}

	virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
	{
		check(IsInGameThread());

		if (bEncounteredError)
		{
			HandleLoadError();
		}

		{
			// Flush the navigation queue
			FScopeLock ScopeLock(&NavigationQueueCS);
			while(NavigationQueue.Num())
			{
				FString Url = NavigationQueue[0];
				NavigationQueue.RemoveAt(0);
				HandleBeforePopup(Url, TEXT(""));
			}
		}
	}

	virtual bool SupportsKeyboardFocus() const override
	{
		return true;
	}

	virtual FReply OnFocusReceived(const FGeometry& MyGeometry, const FFocusEvent& InFocusEvent) override
	{
		FReply Reply = FReply::Handled();

		if (InFocusEvent.GetCause() != EFocusCause::Cleared)
		{
			Reply.SetUserFocus(MainBrowser.ToSharedRef(), InFocusEvent.GetCause());
		}

		return Reply;
	}

	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override
	{
		if (InKeyEvent.GetKey() == EKeys::Escape)
		{
			// Close any overlays first.
			if (IsShowingOverlay())
			{
				CloseTopOverlayBrowser();
			}
			else
			{
				ViewModel->HandleRequestClose(TEXT("Escape"));
			}

			return FReply::Handled();
		}
		return FReply::Unhandled();
	}

	virtual ~SLoginFlowImpl()
	{
	}

private:

	void AddWebOverlay(const TSharedRef< SWidget >& Content)
	{
		if (BrowserContainer.IsValid())
		{
			BrowserContainer->AddSlot()
			.Padding(35.0f)
			[
				Content
			];
		}
	}

	bool IsShowingOverlay() const
	{
		// We are showing overlaid browsers if the browser container contains more than one slot.
		return BrowserContainer->GetNumWidgets() > 1;
	}

	void RemoveWebOverlay(const TSharedRef< SWidget >& Content)
	{
		if (IsShowingOverlay())
		{
			BrowserContainer->RemoveSlot(Content);
		}
	}

	void CloseTopOverlayBrowser()
	{
		if (IsShowingOverlay())
		{
			// Find the browser widget in the top most overlay slot
			const auto BrowserContainerChildren = BrowserContainer->GetChildren();
			TSharedRef<SWidget> LastSlotWidget = BrowserContainerChildren->GetChildAt(BrowserContainerChildren->Num() - 1);

			// Find and call close on the IWebBrowserWindow associated with the widget in the last overlay slot
			for (auto& Kvp : BrowserOverlayWidgets)
			{
				const TSharedPtr<SWebBrowserView> FoundBrowserViewWidget = Kvp.Value.Pin();
				if (FoundBrowserViewWidget.IsValid() && FoundBrowserViewWidget == LastSlotWidget)
				{
					TSharedPtr<IWebBrowserWindow> WebBrowserWindow = Kvp.Key.Pin();
					if (WebBrowserWindow->IsValid() && !WebBrowserWindow->IsClosing())
					{
						WebBrowserWindow->CloseBrowser(false);
					}
				}
			}
		}
	}

	FReply HandleCloseOverlayClicked()
	{
		CloseTopOverlayBrowser();
		return FReply::Handled();
	}

	EVisibility GetCloseOverlayVisibility() const
	{
		return IsShowingOverlay() ? EVisibility::Visible : EVisibility::Hidden;
	}


	bool HandleBeforePopup(FString Url, FString Target)
	{
		UE_LOGF(LogLoginFlow, Log, "HandleBeforePopup %ls %ls", *Url, *Target);

		// Chrome debug tools are allowed to open a popup window.
		if(Url.Contains(TEXT("chrome-devtools")))
		{
			bOpenDevTools = true;
			return false;
		}

		// Popups that provide valid Target will be spawned as an overlay on top of the login flow.
		if (!Target.IsEmpty() && 
			!Target.StartsWith("_blank") &&
			!Target.StartsWith("blank"))
		{
			return false;
		}

		// Remaining popups are redirected to external browser.
		if(IsInGameThread())
		{
			ViewModel->HandleNavigation(Url);
		}
		else
		{
			FScopeLock ScopeLock(&NavigationQueueCS);
			NavigationQueue.Add(Url);
		}
		return true;
	}

	void HandleLoadError()
	{
		UE_LOGF(LogLoginFlow, Warning, "HandleLoadError");
		//@todo: Should forward whatever info we can about the load error
		ViewModel->HandleLoadError();
		bEncounteredError = false;
	}

	void HandleOverlayBrowserLoadError(TWeakPtr<IWebBrowserWindow> BrowserWindowPtr)
	{
		UE_LOGF(LogLoginFlow, Warning, "HandleOverlayBrowserLoadError");

		TSharedPtr<IWebBrowserWindow> WebBrowserWindow = BrowserWindowPtr.Pin();
		if (WebBrowserWindow->IsValid() && !WebBrowserWindow->IsClosing())
		{
			// We encountered an error so just close the browser overlay.
			// @todo: Relay this error info to the user.
			WebBrowserWindow->CloseBrowser(false);
		}
	}

	bool HandleLoadUrl(const FString& Method, const FString& Url, FString& OutResponse)
	{
		//UE_LOGF(LogLoginFlow, Log, "HandleLoadUrl %ls %ls", *Method, *Url);
		return false;
	}

	void HandleBrowserUrlChanged(const FText& Url)
	{
		#if (0) // HandleBrowserBeforeBrowse seems to do all that is required atm
		{
			FString CurrentURL = MainBrowser->GetUrl();
			UE_LOGF(LogLoginFlow, Log, "HandleBrowserUrlChanged Current: %ls New: %ls", *CurrentURL, *Url.ToString());

			ViewModel->HandleBrowserUrlChanged(Url);
		}
		#endif
	}

	bool HandleBrowserCloseWindow(const TWeakPtr<IWebBrowserWindow>& BrowserWindowPtr)
	{
		UE_LOGF(LogLoginFlow, Log, "HandleBrowserCloseWindow");

		TSharedPtr<IWebBrowserWindow> BrowserWindow = BrowserWindowPtr.Pin();
		if(BrowserWindow.IsValid())
		{
			if(!BrowserWindow->IsClosing())
			{
				// If the browser is not set to close, we tell the browser to close which will call back into this handler function.
				BrowserWindow->CloseBrowser(false);
			}
			else
			{
				// Close any matching overlay
				const TWeakPtr<SWebBrowserView>* FoundBrowserViewWidgetPtr = BrowserOverlayWidgets.Find(BrowserWindow);
				if (FoundBrowserViewWidgetPtr != nullptr)
				{
					TSharedPtr<SWebBrowserView> FoundBrowserViewWidget = FoundBrowserViewWidgetPtr->Pin();
					if (FoundBrowserViewWidget.IsValid())
					{
						RemoveWebOverlay(FoundBrowserViewWidget.ToSharedRef());
					}
					BrowserOverlayWidgets.Remove(BrowserWindow);
					return true;
				}

				// Close any matching window
				const TWeakPtr<SWindow>* FoundBrowserWindowWidgetPtr = BrowserWindowWidgets.Find(BrowserWindow);
				if(FoundBrowserWindowWidgetPtr != nullptr)
				{
					TSharedPtr<SWindow> FoundBrowserWindowWidget = FoundBrowserWindowWidgetPtr->Pin();
					if(FoundBrowserWindowWidget.IsValid())
					{
						FoundBrowserWindowWidget->RequestDestroyWindow();
					}
					BrowserWindowWidgets.Remove(BrowserWindow);
					return true;
				}
			}
		}

		return false;
	}

	bool HandleBrowserCreateWindow(const TWeakPtr<IWebBrowserWindow>& NewBrowserWindow, const TWeakPtr<IWebBrowserPopupFeatures>& PopupFeatures)
	{
		UE_LOGF(LogLoginFlow, Log, "HandleBrowserCreateWindow");

		TSharedPtr<IWebBrowserPopupFeatures> PopupFeaturesSP = PopupFeatures.Pin();
		check(PopupFeatures.IsValid())

		// All allowed popups, with the exception of the dev tools, are spawned as an overlay on top of the login flow browser.
		if (bOpenDevTools)
		{
			bOpenDevTools = false;
			// Dev tools spawn in a new window.
			TSharedPtr<SWindow> ParentWindow = FSlateApplication::Get().FindWidgetWindow(SharedThis(this));
			if (ParentWindow.IsValid())
			{
				const int PosX = PopupFeaturesSP->IsXSet() ? PopupFeaturesSP->GetX() : 100;
				const int PosY = PopupFeaturesSP->IsYSet() ? PopupFeaturesSP->GetY() : 100;
				const FVector2D BrowserWindowPosition(PosX, PosY);

				const int Width = PopupFeaturesSP->IsWidthSet() ? PopupFeaturesSP->GetWidth() : 800;
				const int Height = PopupFeaturesSP->IsHeightSet() ? PopupFeaturesSP->GetHeight() : 600;
				const FVector2D BrowserWindowSize(Width, Height);

				const ESizingRule SizeingRule = PopupFeaturesSP->IsResizable() ? ESizingRule::UserSized : ESizingRule::FixedSize;

				TSharedPtr<IWebBrowserWindow> NewBrowserWindowSP = NewBrowserWindow.Pin();
				check(NewBrowserWindowSP.IsValid());

				TSharedRef<SWindow> BrowserWindowWidget =
					SNew(SWindow)
					.Title(FText::GetEmpty())
					.ClientSize(BrowserWindowSize)
					.ScreenPosition(BrowserWindowPosition)
					.AutoCenter(EAutoCenter::None)
					.SizingRule(SizeingRule)
					.SupportsMaximize(SizeingRule != ESizingRule::FixedSize)
					.SupportsMinimize(SizeingRule != ESizingRule::FixedSize)
					.HasCloseButton(true)
					.CreateTitleBar(true)
					.IsInitiallyMaximized(PopupFeaturesSP->IsFullscreen())
					.LayoutBorder(FMargin(0));

				// Setup browser widget.
				TSharedPtr<SWebBrowser> BrowserWidget;
					BrowserWindowWidget->SetContent(
						SNew(SBorder)
						.VAlign(VAlign_Fill)
						.HAlign(HAlign_Fill)
						.Padding(0)
						[
							SAssignNew(BrowserWidget, SWebBrowser, NewBrowserWindowSP)
							.ShowControls(PopupFeaturesSP->IsToolBarVisible())
							.ShowAddressBar(PopupFeaturesSP->IsLocationBarVisible())
							.OnCreateWindow(this, &SLoginFlowImpl::HandleBrowserCreateWindow)
							.OnCloseWindow(this, &SLoginFlowImpl::HandleBrowserCloseWindow)
							.OnShowDialog(this, &SLoginFlowImpl::HandleShowDialog)
						]);

				// Setup some OnClose stuff.
			{
				struct FLocal
				{
					static void RequestDestroyWindowOverride(const TSharedRef<SWindow>& Window, TWeakPtr<IWebBrowserWindow> BrowserWindowPtr)
					{
						TSharedPtr<IWebBrowserWindow> BrowserWindow = BrowserWindowPtr.Pin();
						if (BrowserWindow.IsValid())
						{
							if (BrowserWindow->IsClosing())
							{
								FSlateApplicationBase::Get().RequestDestroyWindow(Window);
							}
							else
							{
								// Notify the browser window that we would like to close it.  On the CEF side, this will 
								//  result in a call to FWebBrowserHandler::DoClose only if the JavaScript onbeforeunload
								//  event handler allows it.
								BrowserWindow->CloseBrowser(false);
							}
						}
					}
				};

				BrowserWindowWidget->SetRequestDestroyWindowOverride(FRequestDestroyWindowOverride::CreateStatic(&FLocal::RequestDestroyWindowOverride, TWeakPtr<IWebBrowserWindow>(NewBrowserWindow)));
			}

			FSlateApplication::Get().AddWindow(BrowserWindowWidget);
			BrowserWindowWidget->BringToFront();
			FSlateApplication::Get().SetKeyboardFocus(BrowserWidget, EFocusCause::SetDirectly);

			BrowserWindowWidgets.Add(NewBrowserWindow, BrowserWindowWidget);
			return true;
			}
		}
		else
		{
			TSharedPtr<IWebBrowserWindow> NewBrowserWindowSP = NewBrowserWindow.Pin();
			check(NewBrowserWindowSP.IsValid());

			TSharedRef<SWebBrowserView> NewBrowserToOverlay =
				SNew(SWebBrowserView, NewBrowserWindowSP)
				.ShowErrorMessage(false)
				.SupportsTransparency(true)
				.OnLoadError(this, &SLoginFlowImpl::HandleOverlayBrowserLoadError, NewBrowserWindow)
				.OnBeforePopup(this, &SLoginFlowImpl::HandleBeforePopup)
				.OnCreateWindow(this, &SLoginFlowImpl::HandleBrowserCreateWindow)
				.OnCloseWindow(this, &SLoginFlowImpl::HandleBrowserCloseWindow)
				.OnBeforeNavigation(this, &SLoginFlowImpl::HandleBrowserBeforeBrowse)
				.OnShowDialog(this, &SLoginFlowImpl::HandleShowDialog);

			AddWebOverlay(NewBrowserToOverlay);
			BrowserOverlayWidgets.Add(NewBrowserWindow, NewBrowserToOverlay);

			return true;
		}

		return false;
	}

	bool HandleBrowserBeforeBrowse(const FString& Url, const FWebNavigationRequest& Request)
	{
		UE_LOGF(LogLoginFlow, Log, "HandleBrowserBeforeBrowse Main:%d Redirect:%d URL: %ls", Request.bIsMainFrame, Request.bIsRedirect, *Url);
		return ViewModel->HandleBeforeBrowse(Url);
	}

	EWebBrowserDialogEventResponse HandleShowDialog(const TWeakPtr<IWebBrowserDialog> & DialogPtr)
	{
		// @todo: Due to an OS X crash, we continue all dialogs with the default action to prevent them from displaying using native windows.  In the future, we should add custom handlers/ui for these dialogs.
		EWebBrowserDialogEventResponse WebDialogHandling = EWebBrowserDialogEventResponse::Continue;
		return WebDialogHandling;
	}

	bool HandleKey(const FKeyEvent& KeyEvent)
	{
		return ViewModel->ShouldConsumeInput();
	}

	bool HandleKeyChar(const FCharacterEvent& CharacterEvent)
	{
		return ViewModel->ShouldConsumeInput();
	}

private:

	TSharedPtr<FLoginFlowViewModel> ViewModel;

	/* Container for the main login flow browser and any additional optional overlay browsers */
	TSharedPtr<SOverlay> BrowserContainer;
	TMap<TWeakPtr<IWebBrowserWindow>, TWeakPtr<SWebBrowserView>> BrowserOverlayWidgets;

	/* The persistent main login flow browser */
	TSharedPtr<SWebBrowserView> MainBrowser;

	TMap<TWeakPtr<IWebBrowserWindow>, TWeakPtr<SWindow>> BrowserWindowWidgets;
	
	/** Holds navigation requests that are requested outside of the game thread. */
	TArray<FString> NavigationQueue;
	FCriticalSection NavigationQueueCS;

	bool bEncounteredError;
	bool bOpenDevTools;
	const ISlateStyle* StyleSet;
};


TSharedRef<SLoginFlow> SLoginFlow::New()
{
	return MakeShareable(new SLoginFlowImpl());
}

