// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SUserWidget.h"
#include "Styling/CoreStyle.h"

class FLoginFlowViewModel;

/**
 * Top level widget expected to handle login flow through to completion (success, cancel, error)
 */
class SLoginFlow
	: public SUserWidget
{
public:

	SLATE_USER_ARGS(SLoginFlow)
		: _StyleSet(&FCoreStyle::Get())
	{}
	SLATE_ARGUMENT(const ISlateStyle*, StyleSet)

	SLATE_END_ARGS()

	virtual void Construct(const FArguments& InArgs, const TSharedRef<FLoginFlowViewModel>& InViewModel) = 0;
};
