// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Delegates/Delegate.h"

class IOnlineSubsystem;
class SWidget;

/**
 * Create and configure one of these to enable web login flow in your application
 *
 * OnlineSubsystemFacebook for Windows requires this
 */
class ILoginFlowManager
	: public TSharedFromThis<ILoginFlowManager>
{
public:
	ILoginFlowManager() {}
	virtual ~ILoginFlowManager() = default;

	/**
	 * Called when a popup is ready to be dismissed.
	 */
	DECLARE_DELEGATE(FOnPopupDismissed);

	/**
	 * Fired when a login flow Pop-Up (web) window needs to be displayed.
	 *
	 * @Returns Delegate that will be called when the popup is dismissed (to clean up any slate borders created)
	 * @Param PopupWidget The widget that contains the browser window.
	 */
	DECLARE_DELEGATE_RetVal_OneParam(FOnPopupDismissed, FOnDisplayPopup, const TSharedRef<SWidget>& /*LoginWidget*/);

	/**
	 * Register an online subsystem with the login flow factory (call at startup)
	 * 
	 * @param OnlineIdentifier online subsystem identifier requiring a login flow UI
	 * @param InPopupDelegate external delegate to receive widgets from the login flow
	 * @param InCreationFlowPopupDelegate external delegate to receive widgets from the account creation flow
	 * @param bPersistCookies let the global web context manage cookies, or keep them in memory only
	 * @param bConsumeInput whether or not input, not handled by the widget, should be consumed an not bubbled up
	 * @return whether or not the login flow was successfully added
	 */
	virtual bool AddLoginFlow(FName OnlineIdentifier, const FOnDisplayPopup& InPopupDelegate, const FOnDisplayPopup& InCreationFlowPopupDelegate, bool bPersistCookies = true, bool bConsumeInput = false) = 0;

	/**
	 * Has a given login flow been setup
	 *
	 * @return true if login flow added, false otherwise
	 */
	virtual bool HasLoginFlow(FName OnlineIdentifier) = 0;

	/** Cancel an active login flow */
	virtual void CancelLoginFlow() = 0;
	/** Cancel an active account creation flow */
	virtual void CancelAccountCreationFlow() = 0;

	/** Cleanup and remove all registered login flows, detaching from online subsystems */
	virtual void Reset() = 0;
};
