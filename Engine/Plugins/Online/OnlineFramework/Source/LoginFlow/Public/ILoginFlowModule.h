// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "Modules/ModuleManager.h"

class ILoginFlowManager;

class IOnlineSubsystem;
class ISlateStyle;

DECLARE_LOG_CATEGORY_EXTERN(LogLoginFlow, Log, All);

enum class ELoginFlowErrorResult : uint8
{
	Unknown,
	/** Webpage failed to load */
	LoadFail,  
};

/**
 * Delegate executed when there is an error dealing with the entirety of the login flow
 *
 * @param ErrorType type of login flow failure
 * @param ErrorInfo extended information about the error
 */
DECLARE_DELEGATE_TwoParams(FOnLoginFlowError, ELoginFlowErrorResult /*ErrorType*/, const FString& /*ErrorInfo*/);

/**
 * Delegate executed when the login flow is handling a closure of the browser window
 *
 * @param CloseInfo reason for the closure
 */
DECLARE_DELEGATE_OneParam(FOnLoginFlowRequestClose, const FString& /*CloseInfo*/);

/**
 * Delegate executed when a redirect URL is about to be handled by the browser window
 *
 * @param RedirectURL destination of the redirect
 *
 * @return true if the redirect was handled, false otherwise
 */
DECLARE_DELEGATE_RetVal_OneParam(bool, FOnLoginFlowRedirectURL, const FString& /*RedirectURL*/);

struct FBrowserContextSettings;

/**
 * Interface for the login flow module.
 */
class ILoginFlowModule
	: public IModuleInterface
{
public:

	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static inline ILoginFlowModule& Get()
	{
		return FModuleManager::LoadModuleChecked<ILoginFlowModule>("LoginFlow");
	}

	/**
	 * Checks to see if this module is loaded and ready.  It is only valid to call Get() if IsAvailable() returns true.
	 *
	 * @return True if the module is loaded and ready to use
	 */
	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded("LoginFlow");
	}

	/**
	 * Create a login flow manager that will handle the proper interactions between the code requiring a login flow UI and the application
	 */
	virtual TSharedPtr<ILoginFlowManager> CreateLoginFlowManager() const = 0;

	struct FCreateSettings
	{
		/** Starting URL for login flow */
		FString Url;
		/** Optional Slate style for the internal widgets */
		ISlateStyle* StyleSet;
		/** Optional browser context settings */
		TSharedPtr<FBrowserContextSettings> BrowserContextSettings;
		/** Delegate to fire on widget closure */
		FOnLoginFlowRequestClose CloseCallback;
		/** Delegate to fire on flow error */
		FOnLoginFlowError ErrorCallback;
		/** Delegate to fire for every URL redirect */
		FOnLoginFlowRedirectURL RedirectCallback;
		/** Whether or not the widget should consume input not handled by the browser */
		bool bConsumeInput;

		FCreateSettings()
			: StyleSet(nullptr)
			, bConsumeInput(false)
		{}
	};

	/**
	 * Creates a login flow widget.
	 *
	 * @return The new widget.
	 */
	virtual TSharedRef<class SWidget> CreateLoginFlowWidget(const FCreateSettings& Settings) const = 0;

public:

	/** Virtual destructor. */
	virtual ~ILoginFlowModule() = default;
};
