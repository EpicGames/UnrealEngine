// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "ILoginFlowManager.h"
#include "Interfaces/OnlineExternalUIInterface.h"

enum class ELoginFlowErrorResult : uint8;
struct FBrowserContextSettings;

class FOnlineSubsystemMcp;
class SWidget;
struct FLoginFlowProperties;
struct FAccountCreationFlowProperties;

/**
 * Create and configure one of these to enable web login flow in your application
 *
 * OnlineSubsystemFacebook and OnlineSubsystemGoogle for Windows requires this
 */
class FLoginFlowManager
	: public ILoginFlowManager
{
public:
	FLoginFlowManager();
	~FLoginFlowManager();

	//~ Begin ILoginFlowManager interface
	virtual bool AddLoginFlow(FName OnlineIdentifier, const FOnDisplayPopup& InPopupDelegate, const FOnDisplayPopup& InCreationFlowPopupDelegate, bool bPersistCookies, bool bConsumeInput) override;
	virtual bool HasLoginFlow(FName OnlineIdentifier) override;
	virtual void CancelLoginFlow() override;
	virtual void CancelAccountCreationFlow() override;
	virtual void Reset() override;
	//~ End ILoginFlowManager interface

private:

	/**
	 * Login flow
	 */

	/** Delegate fired by the web engine on any error */
	void OnLoginFlow_Error(ELoginFlowErrorResult ErrorType, const FString& ErrorInfo, FString InstanceId);
	/** Delegate fired when the browser window is closed */
	void OnLoginFlow_Close(const FString& CloseInfo, FString InstanceId);
	/** Delegate fired when the browser window indicates a URL redirect */
	bool OnLoginFlow_RedirectURL(const FString& RedirectURL, FString InstanceId);
	/** Delegate fired when a login flow is requested by an external provider */
	void OnLoginFlowStarted(const FString& RequestedURL, const FOnLoginRedirectURL& OnRedirectURL, const FOnLoginFlowComplete& OnLoginFlowComplete, bool& bOutShouldContinueLogin, FName InOnlineIdentifier);
	/** Finish login flow, notifying listeners */
	void FinishLogin();

	/** Delegate fired by online identity when a logout/cleanup is requested */
	void OnLoginFlowLogout(const TArray<FString>& LoginDomains, FName OnlineIdentifier);

private:

	/**
	 * Account creation
	 */

	/** Delegate fired by the web engine on any error */
	void OnAccountCreationFlow_Error(ELoginFlowErrorResult ErrorType, const FString& ErrorInfo, FString InstanceId);
	/** Delegate fired when the browser window is closed */
	void OnAccountCreationFlow_Close(const FString& CloseInfo, FString InstanceId);
	/** Delegate fired when the browser window indicates a URL redirect */
	bool OnAccountCreationFlow_RedirectURL(const FString& RedirectURL, FString InstanceId);
	/** Delegate fired when an account creation flow is requested by an external provider */
	void OnAccountCreationFlowStarted(const FString& RequestedURL, const FOnLoginRedirectURL& OnRedirectURL, const FOnLoginFlowComplete& OnAccountCreationFlowComplete, bool& bOutShouldContinueAccountCreation, FName InOnlineIdentifier);
	/** Finish account creation flow, notifying listeners */
	void FinishAccountCreation();


private:

	bool IsFlowInProgress() const { return PendingLogin.IsValid() || PendingAccountCreation.IsValid(); }

	struct FOnlineParams
	{
		/** Online identifier <subsystem>:<instancename> that describes the OnlineSubsystem */
		FName OnlineIdentifier;
		/** Single-cast delegate instance (bind to this to handle login flow display) */
		FOnDisplayPopup OnLoginFlowPopup;
		/** Handle to bound login flow ui required delegate */
		FDelegateHandle LoginFlowUIRequiredDelegateHandle;
		/** Handle to bound login flow logout delegate */
		FDelegateHandle LoginFlowLogoutDelegateHandle;
		/** Single-cast delegate instance (bind to this to handle account creation flow display) */
		FOnDisplayPopup OnAccountCreationFlowPopup;
		/** Handle to bound account creation flow ui required delegate */
		FDelegateHandle AccountCreationFlowUIRequiredDelegateHandle;
		/** Optional browser context settings if bPersistCookies is false */
		TSharedPtr<FBrowserContextSettings> BrowserContextSettings;
		/** Has this context been registered */
		bool bRegisteredContext = false;
		/** Whether or not keyboard input, not handled by the browser, should be consumed */
		bool bConsumeInput = false;
	};

	/**
	 * Register a flow with the web browser singleton
	 *
	 * @param Params online flow parameters to register
	 */
	void RegisterBrowserContext(FOnlineParams* Params);

	/** Mapping of online subsystem identifiers to the parameters they have setup for login flow */
	TMap<FName, FOnlineParams> OnlineSubsystemsMap;

	/** Properties related to the current login attempt */
	TUniquePtr<FLoginFlowProperties> PendingLogin;
	/** Properties related to the current account creation attempt */
	TUniquePtr<FAccountCreationFlowProperties> PendingAccountCreation;
};
