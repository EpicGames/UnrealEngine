// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using EpicGames.Core;

public class Party : ModuleRules
{
	[ConfigFile(ConfigHierarchyType.Game, "OnlineFrameworkParty")]
	bool bUseXbl = false;

	public Party(ReadOnlyTargetRules Target) : base(Target)
	{
		ConfigCache.ReadSettings(DirectoryReference.FromFile(Target.ProjectFile), Target.Platform, this, Target.CustomConfig);

		PublicDefinitions.Add("PARTY_PLATFORM_SESSIONS_PSN=" + (bUsesPSNSessions ? "1" : "0"));
		PublicDefinitions.Add("PARTY_PLATFORM_SESSIONS_XBL=" + (bUsesXBLSessions ? "1" : "0"));
		PublicDefinitions.Add("PARTY_PLATFORM_INVITE_PERMISSIONS=" + (bUsesPlatformInvitePermissions? "1" : "0"));
		PublicDefinitions.Add("PARTY_PLATFORM_XBL_INVITE_PERMISSIONS=" + (bUsesXblInvitePermissions ? "1" : "0"));

		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[] {
				"CoreOnline",
				"OnlineSubsystem",
				"OnlineSubsystemUtils",
			}
			);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"Engine",
				"OnlineFrameworkCommon",
				"OnlineServicesInterface",
				"OnlineServicesOSSAdapter"
			}
			);

		if (bUseXbl)
		{
			PrivateDependencyModuleNames.Add("OnlineSubsystemGDK");
		}
	}

	protected virtual bool bUsesPSNSessions { get { return false; } }
	protected virtual bool bUsesXBLSessions { get { return bUseXbl; } }
	protected virtual bool bUsesPlatformInvitePermissions { get { return bUseXbl; } }
	protected virtual bool bUsesXblInvitePermissions { get { return bUseXbl; } }
}
