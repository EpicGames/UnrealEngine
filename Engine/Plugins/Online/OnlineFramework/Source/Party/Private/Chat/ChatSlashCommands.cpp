// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chat/ChatSlashCommands.h"
#include "Chat/SocialChatManager.h"
#include "SocialManager.h"


#define LOCTEXT_NAMESPACE "ChatSlashCommandManager"

FAutoCompleteStruct::FAutoCompleteStruct(const FString& InFullString, const TWeakPtr<const FChatSlashCommand>& InCmd, TWeakObjectPtr<USocialUser> InOptionalTargetUser)
	: FullString(InFullString), SlashCommand(InCmd), OptionalTargetUser(InOptionalTargetUser)
{
	FRegisteredSlashCommands::TokenizeMessage(FullString, Tokens);
}

bool FRegisteredSlashCommands::TryExecuteCommandByMatchingText(const FString& UserTypedText)
{
	if (UserTypedText.StartsWith(TEXT("/")))
	{
		//find exact match in remaining complete strings 
		for (TSharedPtr<FAutoCompleteStruct>& AutoCompleteDatum : AutoCompleteData)
		{
			if (AutoCompleteDatum.IsValid())
			{
				if (AutoCompleteDatum->FullString == UserTypedText)
				{
					if (AutoCompleteDatum->SlashCommand.IsValid())
					{
						AutoCompleteDatum->SlashCommand.Pin()->ExecuteSlashCommand( AutoCompleteDatum->OptionalTargetUser.Get() );
						return true;
					}
				}
			}
		}
	}

	return false;
}

void FRegisteredSlashCommands::RegisterCommand(const TSharedPtr<FChatSlashCommand>& NewSlashCommand)
{
	if (NewSlashCommand.IsValid())
	{
		for (const TSharedPtr<FChatSlashCommand>& Cmd : RegisteredCustomSlashCommands)
		{
			if (Cmd.IsValid())
			{
				if (Cmd->GetCommandNameString() == NewSlashCommand->GetCommandNameString())
				{
					UE_LOGF(LogParty, Warning, "Attempting to register duplicate slash command");
					return;
				}
			}
		}
		for (const TSharedPtr<FChatSlashCommand> Cmd : RegisteredInteractionSlashCommands)
		{
			if (Cmd.IsValid())
			{
				if (Cmd->GetCommandNameString() == NewSlashCommand->GetCommandNameString())
				{
					UE_LOGF(LogParty, Warning, "Attempting to register duplicate slash command");
					return;
				}
			}
		}

		if (USocialToolkit* Toolkit = MyToolkit.Get())
		{
			NewSlashCommand->Init(*Toolkit);
		}

		RegisteredCustomSlashCommands.Add(NewSlashCommand);
	}
}

bool FRegisteredSlashCommands::IsEnabled()
{
	if (USocialToolkit* Toolkit = MyToolkit.Get())
	{
		return Toolkit->GetChatManager().AreSlashCommandsEnabled();
	}
	return false;
}

bool FRegisteredSlashCommands::NotifyUserTextChanged(const FText& InText) 
{
	FString InTextAsString = InText.ToString();
	TArray<FString> Tokens;
	TokenizeMessage(InTextAsString, Tokens);

	if (InTextAsString.Len() > 0 && InTextAsString[0] == TEXT("/")[0])
	{
		AutoCompleteData.Reset();
		PrepareInteractionAutocompleteStrings(Tokens);
		for (const TSharedPtr<FChatSlashCommand>& Cmd : RegisteredCustomSlashCommands)
		{
			Cmd->GetAutoCompleteStrings(AutoCompleteData, Tokens);
		}
	}
	else if (InTextAsString.Len() == 0)
	{
		AutoCompleteData.Reset();
	}

	if (SpaceWasJustTyped(InTextAsString))
	{
		//attempt spacebar execution on exact matches to autocomplete data
		for (TSharedPtr<FAutoCompleteStruct>& AutoCompleteDatum : AutoCompleteData)
		{
			if (AutoCompleteDatum)
			{
				if (TokensExactMatch(AutoCompleteDatum->Tokens, Tokens))
				{
					TSharedPtr<const FChatSlashCommand> Cmd = AutoCompleteDatum->SlashCommand.Pin();
					if (Cmd->HasSpacebarExecuteFunctionality())
					{
						if (Cmd->RequiresUserForExecution())
						{
							if (AutoCompleteDatum->OptionalTargetUser.IsValid())
							{
								Cmd->ExecuteSlashCommand(AutoCompleteDatum->OptionalTargetUser.Get());
								return true;
							}
						}
						else
						{
							Cmd->ExecuteSlashCommand(AutoCompleteDatum->OptionalTargetUser.Get());
							return true;
						}
					}
				}
			}
		}
		//attempt spacebar execution based on partial completion
		for (const TSharedPtr<FChatSlashCommand>& Cmd : RegisteredCustomSlashCommands)
		{
			if (Cmd->CanExecuteSpacebarFromPartialTokens(Tokens))
			{
				Cmd->ExecuteSlashCommand(nullptr);
				return true;
			}
		}
	}

	return false;
}

void FRegisteredSlashCommands::Init(USocialToolkit& Toolkit)
{
	const TArray<FSocialInteractionHandle> RegisteredInteractions = USocialManager::GetRegisteredInteractions();
	for (FSocialInteractionHandle Interaction : RegisteredInteractions)
	{
		if (Interaction.GetSlashCommandToken().Len() > 0)
		{
			TSharedPtr<FInteractionCommandWrapper> SlashCommand = MakeShared<FInteractionCommandWrapper>(Interaction);
			SlashCommand->Init(Toolkit);
			RegisteredInteractionSlashCommands.Add(SlashCommand);
		}
	}

	FInternationalization::Get().OnCultureChanged().AddSP(SharedThis(this), &FRegisteredSlashCommands::HandleCultureChanged);

	MyToolkit = &Toolkit;
}

void FRegisteredSlashCommands::TokenizeMessage(const FString& InChatText, TArray<FString>& Tokens)
{
	//this is simple, but wrapping in a method provides consistent behavior for how slash commands will be tokenized
	InChatText.ParseIntoArray(Tokens, TEXT(" "), true);
}

bool FRegisteredSlashCommands::TokensExactMatch(TArray<FString>& TokensLHS, TArray<FString>& TokensRHS)
{
	if (TokensRHS.Num() != TokensLHS.Num())
	{
		return false;
	}

	int32 NumTokens = TokensLHS.Num();

	for (int Token = 0; Token < NumTokens; ++Token)
	{
		if (TokensLHS[Token] != TokensRHS[Token])
		{
			return false;
		}
	}
	return true;
}

bool FRegisteredSlashCommands::CmdMatchesFirstToken(const FString& CmdString, const TArray<FString>& Tokens)
{
	int32 NumTokens = Tokens.Num();

	if (NumTokens == 0)
	{
		return false;
	}
	else if (NumTokens == 1)
	{
		//old system used "Contains"
		return CmdString.StartsWith(Tokens[0]);
	}
	else
	{
		//more than one token means first token should match exactly
		return Tokens[0] == CmdString;
	}
}

void FRegisteredSlashCommands::PrepareInteractionAutocompleteStrings(const TArray<FString>& UserTextTokens) 
{
	if (USocialToolkit* Toolkit = MyToolkit.Get())
	{
		if (UserTextTokens.Num() != 0)
		{
			TArray<TSharedPtr<FInteractionCommandWrapper>> RelevantCommands;
			for (const TSharedPtr<FInteractionCommandWrapper>& InteractionCmd : RegisteredInteractionSlashCommands)
			{
				if (CmdMatchesFirstToken(InteractionCmd->GetCommandNameString(), UserTextTokens))
				{
					RelevantCommands.Add(InteractionCmd);
				}
			}

			//only add autocomplete with user names after first characters have been typed and we've narrowed down the list to a single command
			if (RelevantCommands.Num() == 1)
			{
				//only 1 viable command, so auto compete user names
				const TSharedPtr<FInteractionCommandWrapper>& OnlyViableCmd = RelevantCommands[0];
				
				//these checks are expensive, only do once and filter down based on name.
				if (!bValidUsersCached)
				{					
					for (USocialUser* User : Toolkit->GetAllUsers())
					{
						if (User)
						{
							OnlyViableCmd->TryCacheValidAutoCompleteUser(*User, UserTextTokens);
						}
					}
					bValidUsersCached = true;
				}
				OnlyViableCmd->GetAutoCompleteStrings(AutoCompleteData, UserTextTokens);
			}
			else
			{
				//clear stale users while narrowing down commands to single command
				if (bValidUsersCached)
				{
					for (const TSharedPtr<FInteractionCommandWrapper>& InteractionCmd : RegisteredInteractionSlashCommands)
					{
						InteractionCmd->ResetUserCache();
					}
				}
				bValidUsersCached = false;
				
				//multiple commands, only autocomplete command names (ie first token)
				for (const TSharedPtr<FInteractionCommandWrapper>& InteractionCmd : RelevantCommands)
				{
					InteractionCmd->GetAutoCompleteStrings(AutoCompleteData, UserTextTokens);
				}
			}
		}
	}
	else
	{
		UE_LOGF(LogParty, Warning, "Warning: Slash Command Component not initialized with a toolkit. Slash commands will be disabled until intialization.");
	}
}

void FRegisteredSlashCommands::HandleCultureChanged() const
{
	for (const TSharedPtr<FChatSlashCommand>& CustomCmd : RegisteredCustomSlashCommands)
	{
		CustomCmd->RecacheForLocalization();
	}
	for (const TSharedPtr<FChatSlashCommand> InteractionCmd : RegisteredInteractionSlashCommands)
	{
		InteractionCmd->RecacheForLocalization();
	}
}

bool FRegisteredSlashCommands::SpaceWasJustTyped(const FString& NewUserText)
{
	bool bSpaceTyped = false;
	uint32 NewLen = NewUserText.Len();

	bool bUserAddedChar = NewLen > LastQueryTextLen;
	if (bUserAddedChar && NewLen != 0)
	{
		bSpaceTyped = NewUserText[NewLen - 1] == TEXT(" ")[0];
	}

	//update internal state for next query; if text is cleared NewLen will be written to 1 when / is typed
	LastQueryTextLen = NewLen;

	return bSpaceTyped;
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

void FChatSlashCommand::RecacheForLocalization() const
{
	//re-read the source text reference to get the new localization
	CommandNameString = CommandNameTextSrc.ToString();
}

//////////////////////////////////////////////////////////////////////////////////
// Chat Slash Command

FChatSlashCommand::FChatSlashCommand(const FText& InCommandName) 
	: CommandNameTextSrc(InCommandName)
{
	//done outside of initializer list to avoid pesky header file order definition compile errors/warnings
	CommandNameString = InCommandName.ToString();
}

void FChatSlashCommand::Init(USocialToolkit& InToolkit)
{
	MyToolkit = &InToolkit;
}

void FChatSlashCommand::GetAutoCompleteStrings(TArray< TSharedPtr<FAutoCompleteStruct> >& OutStringContainer, const TArray<FString>& UserTextTokens) const
{
	//default behavior is simply checking if first token match command name; override this for more complex behavior.
	if (FRegisteredSlashCommands::CmdMatchesFirstToken(CommandNameString, UserTextTokens) && IsEnabled())
	{
		OutStringContainer.Add(MakeShared<FAutoCompleteStruct>(CommandNameString, SharedThis(this), nullptr));
	}
}

#undef LOCTEXT_NAMESPACE
FInteractionCommandWrapper::FInteractionCommandWrapper(FSocialInteractionHandle Interaction) 
	: FChatSlashCommand(FText::FromString(Interaction.GetSlashCommandToken())),
	WrappedInteraction(Interaction)
{
	CacheStringDataForLocalization();
}

void FInteractionCommandWrapper::ExecuteSlashCommand(USocialUser* OptionalTargetUser) const
{
	if (OptionalTargetUser)
	{
		WrappedInteraction.ExecuteInteraction(*OptionalTargetUser);
	}
}

void FInteractionCommandWrapper::ResetUserCache()
{
	CachedValidUsers.Reset();
}

void FInteractionCommandWrapper::TryCacheValidAutoCompleteUser(USocialUser& User, const TArray<FString>& UserTextTokens)
{	
	//if user has typed partial name, check that name matches user; first token is command token
	if (UserTextTokens.Num() > 1)
	{
		const FString& TypedUserName = UserTextTokens.Last();
		if (!User.GetDisplayName().StartsWith(TypedUserName))
		{
			//player is not typing this user name, early out
			return;
		}
	}

	if (WrappedInteraction.IsAvailable(User))
	{
		CachedValidUsers.Add(&User);
	}
}

void FInteractionCommandWrapper::GetAutoCompleteStrings(TArray< TSharedPtr<FAutoCompleteStruct> >& OutStringContainer, const TArray<FString>& UserTextTokens) const
{
	if(FRegisteredSlashCommands::CmdMatchesFirstToken(GetCommandNameString(), UserTextTokens))
	{
		if (CachedValidUsers.Num() != 0)
		{
			for (TWeakObjectPtr<USocialUser> CachedUser : CachedValidUsers)
			{
				if (CachedUser.IsValid())
				{
					bool bUserMatchesText = true;

					if (UserTextTokens.Num() >= 2)
					{
						const FString& TypedUserName = UserTextTokens.Last();
						bUserMatchesText = CachedUser->GetDisplayName().StartsWith(TypedUserName);
					}

					if(bUserMatchesText)
					{
						FString AutoCompleteStringForUser = GetCommandNameString() + TEXT(" ") + CachedUser->GetDisplayName();
						OutStringContainer.Add(MakeShared<FAutoCompleteStruct>(AutoCompleteStringForUser, SharedThis(this), CachedUser));
					}
				}
			}
		}
		else
		{
			//no users checks yet, just populate command name until user data available for auto-complete
			OutStringContainer.Add(MakeShared<FAutoCompleteStruct>(GetCommandNameString(), SharedThis(this), nullptr));
		}
	}
}

void FInteractionCommandWrapper::RecacheForLocalization() const
{
	//don't call super, we're going to custom process interaction names (eg remove spaces)
	CacheStringDataForLocalization();
}

void FInteractionCommandWrapper::CacheStringDataForLocalization() const
{
	if (WrappedInteraction.IsValid())
	{
		//this will be up-to-date with current set localization
		CommandNameString = TEXT("/") + WrappedInteraction.GetSlashCommandToken();

		//compress the localized slash command name into a single token by removing spaces
		TArray<FString> TokensToCompress; 
		FRegisteredSlashCommands::TokenizeMessage(CommandNameString, TokensToCompress);
		FString TokensCompressed = TEXT("");
		for (FString& Token : TokensToCompress)
		{
			TokensCompressed += Token;
		}
		CommandNameString= TokensCompressed;
	}
	else
	{
		//UE_LOGF(LogParty, Warning, "null slash command interaction detected, FInteractionCommandWrapper::RecacheForLocalization");
	}
}


