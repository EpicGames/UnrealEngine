// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chat/CommonSlashCommands.h"
#include "Chat/SocialChatManager.h"

/////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Channel Changing Slash Command
FChannelChangeSlashCommand::FChannelChangeSlashCommand(const FText& CommandText, ESocialChannelType InChannelType) :
	FChatSlashCommand(CommandText),
	ChannelType(InChannelType)
{
}

void FChannelChangeSlashCommand::ExecuteSlashCommand(USocialUser* OptionalTargetUser) const
{
	if (USocialToolkit* Tookit = GetToolkit())
	{
		if (USocialChatChannel* Channel = Tookit->GetChatManager().GetChatRoomForType(ChannelType))
		{
			Tookit->GetChatManager().FocusChatChannel(*Channel);
		}
	}
}

bool FChannelChangeSlashCommand::IsEnabled() const 
{
	if (USocialToolkit* Tookit = GetToolkit())
	{
		if (USocialChatChannel* Channel = Tookit->GetChatManager().GetChatRoomForType(ChannelType))
		{
			return true;
		}
	}
	return false;
}

bool FChannelChangeSlashCommand::CanExecuteSpacebarFromPartialTokens(const TArray<FString>& UserTextTokens) const
{
	if (UserTextTokens.Num() == 1 && GetCommandNameString().StartsWith(UserTextTokens[0]))
	{
		return true;
	}
	return false;
}


////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FPartyChannelSlashCommand::FPartyChannelSlashCommand() :
	FChannelChangeSlashCommand(NSLOCTEXT("SlashCommands", "PartySlashCommand", "/party"), ESocialChannelType::Party)
{
}

FGlobalChannelSlashCommand::FGlobalChannelSlashCommand() :
	FChannelChangeSlashCommand(NSLOCTEXT("SlashCommands", "GlobalSlashCommand", "/global"), ESocialChannelType::General)
{
}

FTeamChannelSlashCommand::FTeamChannelSlashCommand() :
	FChannelChangeSlashCommand(NSLOCTEXT("SlashCommands", "TeamSlashCommand", "/team"), ESocialChannelType::Team)
{
}

FFounderChannelSlashCommand::FFounderChannelSlashCommand() :
	FChannelChangeSlashCommand(NSLOCTEXT("SlashCommands", "FounderSlashCommand", "/founder"), ESocialChannelType::Founder)
{
}
///////////////////////////////////////////////////////////////////////////////////////////////////////////////////


FReplySlashCommand::FReplySlashCommand() :
	FChatSlashCommand(NSLOCTEXT("SlashCommands", "ReplySlashCommand", "/reply"))
{}

void FReplySlashCommand::Init(USocialToolkit& InToolkit)
{
	FChatSlashCommand::Init(InToolkit);

	USocialChatManager& ChatManager = InToolkit.GetChatManager();
	ChatManager.OnChannelCreated().AddSP(this, &FReplySlashCommand::HandleChannelCreated);
	ChatManager.OnChannelLeft().AddSP(this, &FReplySlashCommand::HandleChannelLeft);
}

void FReplySlashCommand::ExecuteSlashCommand(USocialUser* OptionalTargetUser) const 
{

	if (USocialToolkit* Tookit = GetToolkit())
	{
		if (LastUserChannel.IsValid())
		{
			Tookit->GetChatManager().FocusChatChannel(*LastUserChannel);
		}
	}
}

bool FReplySlashCommand::IsEnabled() const 
{
	return this->LastUserChannel != nullptr;
}

void FReplySlashCommand::HandleChannelCreated(USocialChatChannel& NewChannel)
{
	if (ESocialChannelType::Private == NewChannel.GetChannelType())
	{
		TWeakPtr<FChatSlashCommand> WeakThisPtr = this->AsShared();
		NewChannel.OnMessageReceived().AddLambda([WeakThisPtr, &NewChannel, this](FSocialChatMessageRef Messsage) {
			if (WeakThisPtr.IsValid())
			{
 				this->LastUserChannel = &NewChannel;
			}
		});
	}
}

void FReplySlashCommand::HandleChannelLeft(USocialChatChannel& LeavingChannel)
{
	LeavingChannel.OnMessageReceived().RemoveAll(this);

	if (LastUserChannel.IsValid())
	{
		if (LastUserChannel == &LeavingChannel)
		{
			LastUserChannel = nullptr;
		}
	}
}
