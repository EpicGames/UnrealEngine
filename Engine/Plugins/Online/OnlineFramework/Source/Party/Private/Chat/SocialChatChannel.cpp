// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chat/SocialChatChannel.h"
#include "Chat/SocialChatManager.h"
#include "Chat/SocialChatMessage.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SocialChatChannel)

#define LOCTEXT_NAMESPACE "SocialChatRoomChannel"

void USocialChatChannel::InjectLocalMessage(const TSharedRef<FSocialLocalChatMessage>& LocalMessage)
{
	//@todo DanH SocialChat: Inject local message
}

void USocialChatChannel::NotifyUserJoinedChannel(USocialUser& User)
{
	static FText UserJoinedMessage = LOCTEXT("SocialChatRoom_MemberJoined", "{0} has joined.");

	if (ChannelType == ESocialChannelType::Party ||
		ChannelType == ESocialChannelType::Team)
	{
		AddSystemMessage(FText::Format(UserJoinedMessage, FText::FromString(User.GetDisplayName())));
	}

	OnUserJoinedChannel().Broadcast(User);
}

void USocialChatChannel::NotifyUserLeftChannel(USocialUser& User)
{
	static FText UserLeftMessage = LOCTEXT("SocialChatRoom_MemberExit", "{0} has left.");
	
	if (ChannelType == ESocialChannelType::Party ||
		ChannelType == ESocialChannelType::Team)
	{
		AddSystemMessage(FText::Format(UserLeftMessage, FText::FromString(User.GetDisplayName())));
	}

	OnUserLeftChannel().Broadcast(User);
}

void USocialChatChannel::NotifyChannelUserChanged(USocialUser& User)
{
	OnChannelUserChanged().Broadcast(User);
}

void USocialChatChannel::NotifyMessageReceived(const TSharedRef<FChatMessage>& ChatMessage)
{
	USocialUser* SendingUser = GetOwningToolkit().FindUser(ChatMessage->GetUserId());
	if (ensure(SendingUser))
	{
		FSocialChatMessageRef UserChatMessage = FSocialUserChatMessage::Create(*SendingUser, ChatMessage.Get(), ChannelType);
		AddMessageInternal(UserChatMessage);
	}
}

IOnlineChatPtr USocialChatChannel::GetChatInterface() const
{
	IOnlineSubsystem* OSS = GetOwningToolkit().GetSocialOss(ESocialSubsystem::Primary);
	return OSS ? OSS->GetChatInterface() : nullptr;
}

void USocialChatChannel::SanitizeMessage(FString& RawMessage) const
{
	RawMessage.ReplaceInline(TEXT("&amp;"), TEXT("&"));
	RawMessage.ReplaceInline(TEXT("&quot;"), TEXT("\""));
	RawMessage.ReplaceInline(TEXT("&apos;"), TEXT("'"));
	RawMessage.ReplaceInline(TEXT("&lt;"), TEXT("<"));
	RawMessage.ReplaceInline(TEXT("&gt;"), TEXT(">"));
}

void USocialChatChannel::AddSystemMessage(const FText& MessageBody)
{
	AddMessageInternal(FSocialSystemChatMessage::Create(TEXT("System"), MessageBody.ToString(), ChannelType, EChatSystemMessagePurpose::Info));
}

void USocialChatChannel::AddMessageInternal(FSocialChatMessageRef NewMessage)
{
	//@todo DanH SocialChat: Profanity filtering #required
	//@todo DanH SocialChat: Some heuristic for ditching old messages #suggested

	//@todo DanH Chat: Don - the exact same message is being added multiple times to a single channel #required
	if (!MessageHistory.Contains(NewMessage))
	{
		if (MessageHistory.Num() > 0)
		{
			NewMessage->SetPreviousMessage(MessageHistory.Last());
		}

		MessageHistory.Add(NewMessage);
		OnMessageReceived().Broadcast(NewMessage);
	}
}

USocialToolkit& USocialChatChannel::GetOwningToolkit() const
{
	USocialChatManager* OwningChatManager = GetOuterUSocialChatManager();
	check(OwningChatManager);
	return OwningChatManager->GetOwningToolkit();
}

void USocialChatChannel::ListenToChannel(USocialChatChannel& SourceChannel)
{
	if (ensure(this != &SourceChannel))
	{
		SourceChannel.OnMessageReceived().AddUObject(this, &USocialChatChannel::HandleListenedChannelMessageReceived, &SourceChannel);
	}
}

void USocialChatChannel::HandleListenedChannelMessageReceived(const FSocialChatMessageRef& Message, USocialChatChannel* SourceChannel)
{
	AddMessageInternal(Message);
}

void USocialChatChannel::SetChannelDisplayName(const FText& InDisplayName)
{
	ChannelDisplayName = InDisplayName;

	OnChannelDisplayNameChanged().Broadcast(InDisplayName);
}

void USocialChatChannel::SetIsHidden(bool InValue)
{
	if (InValue != bIsHidden)
	{
		bIsHidden = InValue;
		OnHiddenChanged().Broadcast(bIsHidden);
	}
}

void USocialChatChannel::AddMirroredMessage(FSocialChatMessageRef NewMessage)
{
	AddMessageInternal(NewMessage);
}

#undef LOCTEXT_NAMESPACE



