// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chat/SocialChatMessage.h"

FString FSocialUserChatMessage::GetSourceName() const
{
	if (FromUser.IsValid())
	{
		return FromUser->GetDisplayName();
	}
	else
	{
		return TEXT("(source name unavailable)");
	}
}