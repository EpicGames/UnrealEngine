// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chat/SocialChatRoom.h"
#include "Chat/SocialChatChannel.h"
#include "SocialToolkit.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SocialChatRoom)

#define LOCTEXT_NAMESPACE "SocialChatRoom"

void USocialChatRoom::Initialize(USocialUser* InSocialUser, const FChatRoomId& InChannelId, ESocialChannelType InSourceChannelType)
{
	SetRoomId(InChannelId);
	SetChannelType(InSourceChannelType);
	SetChannelDisplayName(DetermineChannelDisplayName(InSourceChannelType, InChannelId));
}

bool USocialChatRoom::SendMessage(const FString& Message)
{
	if (Message.Len() > 0)
	{
		IOnlineChatPtr ChatInterface = GetChatInterface();
		if (ChatInterface.IsValid())
		{
			USocialUser& LocalUser = GetOwningToolkit().GetLocalUser();
			FUniqueNetIdRepl LocalUserId = LocalUser.GetUserId(ESocialSubsystem::Primary);
			if (ensure(LocalUserId.IsValid()))
			{
				FString MessageToSend = Message;
				SanitizeMessage(MessageToSend);

				if (ChatInterface->SendRoomChat(*LocalUserId, RoomId, MessageToSend))
				{
					// don't echo message locally
					// AddMessageInternal(FSocialUserChatMessage::Create(LocalUser, MessageToSend, ChannelType));
					return true;
				}
			}
		}
	}
	return false;
}

const FText USocialChatRoom::DetermineChannelDisplayName(ESocialChannelType InSourceChannelType, const FChatRoomId& InRoomId )
{
	switch (InSourceChannelType)
	{
		case ESocialChannelType::Party:
			return LOCTEXT("SocialChatRoomPartyTab", "Party");
		case ESocialChannelType::Team:
			return LOCTEXT("SocialChatRoomTeamTab", "Team");
		case ESocialChannelType::System:
			return LOCTEXT("SocialChatRoomSystemTab", "System");
		case ESocialChannelType::General:
			return LOCTEXT("SocialChatRoomGeneralTab", "Global");
		case ESocialChannelType::Founder:
			return LOCTEXT("SocialChatRoomFoundersTab", "Founders");
	}
	return FText::FromString(InRoomId);
}

#undef LOCTEXT_NAMESPACE
