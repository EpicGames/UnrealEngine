// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chat/SocialGroupChannel.h"
#include "SocialToolkit.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SocialGroupChannel)

USocialGroupChannel::USocialGroupChannel()
{
}

void USocialGroupChannel::Initialize(IOnlineGroupsPtr InGroupInterface, USocialUser& InSocialUser, const FUniqueNetId& InGroupId)
{
	GroupId = InGroupId.AsShared();
	SocialUser = &InSocialUser;
	GroupInterfacePtr = InGroupInterface;

	const FUniqueNetIdRepl UserId = SocialUser->GetUserId(ESocialSubsystem::Primary);
	InGroupInterface->QueryGroupInfo(*UserId, *GroupId, FOnGroupsRequestCompleted::CreateUObject(this, &ThisClass::RefreshCompleted_GroupInfo));
	InGroupInterface->QueryGroupRoster(*UserId, *GroupId, FOnGroupsRequestCompleted::CreateUObject(this, &ThisClass::RefreshCompleted_Roster));
}

void USocialGroupChannel::RefreshCompleted_GroupInfo(FGroupsResult Result)
{
	IOnlineGroupsPtr GroupInterface = GroupInterfacePtr.Pin();

	if (GroupInterface.IsValid())
	{
		const FUniqueNetIdRepl UserId = SocialUser->GetUserId(ESocialSubsystem::Primary);
		TSharedPtr<const IGroupInfo> Info = GroupInterface->GetCachedGroupInfo(*UserId, *GroupId);
		if (Info.IsValid())
		{
			// TODO
		}
	}
}

void USocialGroupChannel::RefreshCompleted_Roster(FGroupsResult Result)
{
	IOnlineGroupsPtr GroupInterface = GroupInterfacePtr.Pin();

	if (GroupInterface.IsValid())
	{
		USocialToolkit& OwningToolkit = SocialUser->GetOwningToolkit();
		const FUniqueNetIdRepl UserId = SocialUser->GetUserId(ESocialSubsystem::Primary);

		TSharedPtr<const IGroupRoster> Roster = GroupInterface->GetCachedGroupRoster(*UserId, *GroupId);
		if (Roster.IsValid())
		{
			TArray<FGroupMember> GroupMembers;
			Roster->CopyEntries(GroupMembers);

			Members.Reset();
			for (const FGroupMember& GroupMember : GroupMembers)
			{
				// MERGE-REVIEW: Was changed from FindOrCreate
				USocialUser* NewUser = OwningToolkit.FindUser(GroupMember.GetId());
				Members.Add(NewUser);
			}
		}
	}
}
