// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chat/SocialPartyChatRoom.h"
#include "Chat/SocialChatRoom.h"
#include "SocialToolkit.h"
#include "Party/SocialParty.h"
#include "Party/PartyMember.h"
#include "SocialManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SocialPartyChatRoom)

#define LOCTEXT_NAMESPACE "SocialChatRoom"

void USocialPartyChatRoom::Initialize(USocialUser* InSocialUser, const FChatRoomId& InChannelId, ESocialChannelType InSourceChannelType)
{
	Super::Initialize(InSocialUser, InChannelId, InSourceChannelType);

	if (USocialParty* SocialParty = GetOwningToolkit().GetSocialManager().GetPersistentParty())
	{
		TArray<UPartyMember*> PartyMembers = SocialParty->GetPartyMembers();
		for (UPartyMember* PartyMember : PartyMembers)
		{
			PartyMember->OnLeftParty().AddUObject(this, &USocialPartyChatRoom::HandlePartyMemberLeft);
		}
		SocialParty->OnPartyMemberCreated().AddUObject(this, &USocialPartyChatRoom::HandlePartyMemberJoined);
	}
}

void USocialPartyChatRoom::HandlePartyMemberLeft(EMemberExitedReason Reason)
{
	if (USocialParty* SocialParty = GetOwningToolkit().GetSocialManager().GetPersistentParty())
	{
		if (SocialParty->GetNumPartyMembers() <= 1)
		{
			SetIsHidden(true);
		}
	}
}

void USocialPartyChatRoom::HandlePartyMemberJoined(UPartyMember& NewPartyMember)
{
	NewPartyMember.OnLeftParty().AddUObject(this, &USocialPartyChatRoom::HandlePartyMemberLeft);
	if (USocialParty* SocialParty = GetOwningToolkit().GetSocialManager().GetPersistentParty())
	{
		if (GetIsHidden() && SocialParty->GetNumPartyMembers() > 1)
		{
			SetIsHidden(false);
		}
	}
}

#undef LOCTEXT_NAMESPACE
