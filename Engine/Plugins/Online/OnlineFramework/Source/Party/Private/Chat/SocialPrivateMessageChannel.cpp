// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chat/SocialPrivateMessageChannel.h"

#include "Chat/SocialChatMessage.h"
#include "SocialToolkit.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SocialPrivateMessageChannel)

void USocialPrivateMessageChannel::Initialize(USocialUser* InSocialUser, const FChatRoomId& InChannelId, ESocialChannelType InSourceChannelType)
{
	check(InSocialUser);
	SetTargetUser(*InSocialUser);
	SetChannelType(ESocialChannelType::Private);
	SetChannelDisplayName(FText::FromString(InSocialUser->GetDisplayName()));
}

bool USocialPrivateMessageChannel::SendMessage(const FString& InMessage)
{
	if (InMessage.Len() > 0)
	{
		USocialUser& LocalUser = GetOwningToolkit().GetLocalUser();
		IOnlineChatPtr ChatInterface = GetChatInterface();
		if (ChatInterface.IsValid() &&
			TargetUser &&
			TargetUser != &LocalUser &&
			TargetUser->IsFriend(ESocialSubsystem::Primary))
		{
			FUniqueNetIdRepl LocalUserId = LocalUser.GetUserId(ESocialSubsystem::Primary);
			FUniqueNetIdRepl TargetUserId = TargetUser->GetUserId(ESocialSubsystem::Primary);
			if (LocalUserId.IsValid() && TargetUserId.IsValid() && ChatInterface->IsChatAllowed(*LocalUserId, *TargetUserId))
			{
				FString MessageToSend(InMessage);
				SanitizeMessage(MessageToSend);

				if (ChatInterface->SendPrivateChat(*LocalUserId, *TargetUserId, MessageToSend))
				{
					AddMessageInternal(FSocialUserChatMessage::Create(LocalUser, MessageToSend, ChannelType));
					return true;
				}
			}
		}
	}
	return false;
}

void USocialPrivateMessageChannel::SetTargetUser(USocialUser& InTargetUser)
{
	TargetUser = &InTargetUser;
}

USocialUser& USocialPrivateMessageChannel::GetTargetUser() const
{
	check(TargetUser);
	return *TargetUser;
}
