// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chat/SocialReadOnlyChatChannel.h"
#include "Interfaces/OnlinePartyInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SocialReadOnlyChatChannel)

void USocialReadOnlyChatChannel::Initialize(USocialUser* InSocialUser, const FChatRoomId& InChannelId, ESocialChannelType InSourceChannelType)
{
	SetChannelType(ESocialChannelType::General);
	SetChannelDisplayName(NSLOCTEXT("AllChatChannelNS","AllChatChannelKey","All"));
}

bool USocialReadOnlyChatChannel::SendMessage(const FString& InMessage)
{
	return false;
}
