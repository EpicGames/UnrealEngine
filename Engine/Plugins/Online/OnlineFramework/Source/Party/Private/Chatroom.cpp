// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chatroom.h"
#include "Engine/GameInstance.h"
#include "TimerManager.h"
#include "Engine/LocalPlayer.h"
#include "OnlineSubsystemUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(Chatroom)

inline FString GetLocalUserNickName(UWorld* World, const FUniqueNetId& LocalUserId)
{
	check(World);
	UGameInstance* GameInstance = World->GetGameInstance();
	check(GameInstance);
	ULocalPlayer* LP = GameInstance->FindLocalPlayerFromUniqueNetId(LocalUserId);
	if (LP)
	{
		return LP->GetNickname();
	}

	return TEXT("INVALID");
}

UChatroom::UChatroom()
	: MaxChatRoomRetries(5)
	, NumChatRoomRetries(0)
{
}

void UChatroom::UnregisterDelegates()
{
	IOnlineChatPtr ChatInt = Online::GetChatInterface(GetWorld());
	if (ChatInt.IsValid())
	{
		if (ChatRoomCreatedDelegateHandle.IsValid())
		{
            // Failsafe if chat CreateRoom never completes (xmpp is misbehaving)
			ChatInt->ClearOnChatRoomCreatedDelegate_Handle(ChatRoomCreatedDelegateHandle);
		}
	}
}

void UChatroom::CreateOrJoinChatRoom(FUniqueNetIdRepl LocalUserId, FChatRoomId ChatRoomId, FOnChatRoomCreatedOrJoined CompletionDelegate, FChatRoomConfig RoomConfig)
{
	bool bShouldFireDelegate = false;
	if (!ChatRoomId.IsEmpty())
	{
		if (LocalUserId.IsValid())
		{
			UWorld* World = GetWorld();
			IOnlineChatPtr ChatInt = Online::GetChatInterface(World);
			if (ChatInt.IsValid())
			{
				if (!IsAlreadyInChatRoom(LocalUserId, ChatRoomId))
				{
					if (CurrentChatRoomId.IsEmpty())
					{
						UE_LOGF(LogOnlineChat, Verbose, "[UChatroom::CreateOrJoinChatRoom] attempting to join %ls", *ChatRoomId);
						CurrentChatRoomId = ChatRoomId;

						FTimerManager& TM = GetTimerManager();
						if (TM.GetTimerRemaining(ChatRoomRetryTimerHandle) > 0)
						{
							TM.ClearTimer(ChatRoomRetryTimerHandle);
						}

						FOnChatRoomJoinPublicDelegate ChatRoomDelegate;
						ChatRoomDelegate.BindUObject(this, &ThisClass::OnChatRoomCreatedOrJoined, CompletionDelegate, RoomConfig);

						// Try to create the room first (it will create if it doesn't exist, or just join if it does)
						ChatRoomCreatedDelegateHandle = ChatInt->AddOnChatRoomCreatedDelegate_Handle(ChatRoomDelegate);
						ChatInt->CreateRoom(*LocalUserId, ChatRoomId, GetLocalUserNickName(World, *LocalUserId), RoomConfig);
					}
					else
					{
						if (ChatRoomId == CurrentChatRoomId)
						{
							UE_LOGF(LogOnlineChat, Verbose, "[UChatroom::CreateOrJoinChatRoom] already joining %ls", *ChatRoomId);
						}
						else
						{
							UE_LOGF(LogOnlineChat, Verbose, "[UChatroom::CreateOrJoinChatRoom] can't join %ls already joining %ls", *ChatRoomId, *CurrentChatRoomId);
							bShouldFireDelegate = true;
						}
					}
				}
				else
				{
					UE_LOGF(LogOnlineChat, Verbose, "[UChatroom::CreateOrJoinChatRoom] already joined %ls", *ChatRoomId);
				}
			}
		}
		else
		{
			UE_LOGF(LogOnlineChat, Verbose, "[UChatroom::CreateOrJoinChatRoom] invalid user id");
			bShouldFireDelegate = true;
		}
	}
	else
	{
		UE_LOGF(LogOnlineChat, Verbose, "[UChatroom::CreateOrJoinChatRoom] invalid chat room id");
		bShouldFireDelegate = true;
	}

	if (bShouldFireDelegate)
	{
		CompletionDelegate.ExecuteIfBound(ChatRoomId, false);
	}
}

void UChatroom::OnChatRoomCreatedOrJoined(const FUniqueNetId& LocalUserId, const FChatRoomId& RoomId, bool bWasSuccessful, const FString& Error, FOnChatRoomCreatedOrJoined CompletionDelegate, FChatRoomConfig RoomConfig)
{
	if (CurrentChatRoomId == RoomId)
	{
		UE_LOGF(LogOnlineChat, Verbose, "[UChatroom::OnChatRoomCreatedOrJoined] %ls joined room %ls Success: %d%ls", *LocalUserId.ToString(), *RoomId, bWasSuccessful, !Error.IsEmpty() ? *FString::Printf(TEXT(" %s"), *Error) : TEXT(""));

		// Don't clear if this delegate failed because of existing pending join
		const bool bAlreadyJoining = !Error.IsEmpty() && Error.Contains(TEXT("operation pending"));
		if (!bAlreadyJoining)
		{
			IOnlineChatPtr ChatInt = Online::GetChatInterface(GetWorld());
			if (ChatInt.IsValid())
			{
				ChatInt->ClearOnChatRoomCreatedDelegate_Handle(ChatRoomCreatedDelegateHandle);
			}

			if (bWasSuccessful)
			{
				FTimerDelegate Delegate = FTimerDelegate::CreateLambda([RoomId, CompletionDelegate, bWasSuccessful]()
				{
					// Announce that chat is available
					CompletionDelegate.ExecuteIfBound(RoomId, bWasSuccessful);
				});

				GetTimerManager().SetTimerForNextTick(Delegate);
			}
			else
			{
				if (NumChatRoomRetries < MaxChatRoomRetries)
				{
					NumChatRoomRetries++;
					UE_LOGF(LogOnlineChat, Verbose, "[UChatroom::OnChatRoomCreatedOrJoined] retry %d/%d", NumChatRoomRetries, MaxChatRoomRetries);

					FUniqueNetIdRepl StrongUserId(LocalUserId.AsShared());

					auto RetryDelegate = FTimerDelegate::CreateWeakLambda(this, [this, StrongUserId, RoomId, CompletionDelegate, RoomConfig] ()
					{
						// Attempt to rejoin room shortly
						CreateOrJoinChatRoom(StrongUserId, RoomId, CompletionDelegate, RoomConfig);
					});

					GetTimerManager().SetTimer(ChatRoomRetryTimerHandle, RetryDelegate, .3f, false);
				}
				else
				{
					UE_LOGF(LogOnlineChat, Verbose, "[UChatroom::OnChatRoomCreatedOrJoined] exceeded %d retries", MaxChatRoomRetries);
				}

				// Clear out the chat id since we failed to join
				CurrentChatRoomId.Empty();
			}
		}
		else
		{
			UE_LOGF(LogOnlineChat, Verbose, "[UChatroom::OnChatRoomCreatedForRejoin] already attempting to join %ls", *CurrentChatRoomId);
		}
	}
	else
	{
		UE_LOGF(LogOnlineChat, Verbose, "[UChatroom::OnChatRoomCreatedForRejoin] other chat room detected %ls, waiting for %ls", *RoomId, *CurrentChatRoomId);
	}
}

void UChatroom::LeaveChatRoom(const FUniqueNetIdRepl& LocalUserId, const FOnChatRoomLeft& CompletionDelegate)
{
	if (!CurrentChatRoomId.IsEmpty())
	{
		UE_LOGF(LogOnlineChat, Verbose, "[UChatroom::LeaveChatRoom] %ls leaving chat room %ls",
			*LocalUserId.ToString(),
			*CurrentChatRoomId);

		// Always reset these regardless of chat room
		GetTimerManager().ClearTimer(ChatRoomRetryTimerHandle);
		ChatRoomRetryTimerHandle.Invalidate();
		NumChatRoomRetries = 0;

		// ExitRoom callback could fire inline if it fails, make a copy so the ensure in ChatRoomLeftInternal does not hit
		FChatRoomId ChatRoomIdCpy = CurrentChatRoomId;
		CurrentChatRoomId.Empty();

		if (IsOnline())
		{
			// Leave chat
			IOnlineChatPtr ChatInt = Online::GetChatInterface(GetWorld());
			if (ensure(ChatInt.IsValid()))
			{
				FOnChatRoomCreatedDelegate ChatRoomDelegate;
				ChatRoomDelegate.BindUObject(this, &ThisClass::OnChatRoomLeft, ChatRoomIdCpy, CompletionDelegate);

				ChatRoomLeftDelegateHandle = ChatInt->AddOnChatRoomExitDelegate_Handle(ChatRoomDelegate);
				ChatInt->ExitRoom(*LocalUserId, ChatRoomIdCpy);
			}
		}
		else
		{
			UE_LOGF(LogOnlineChat, Display, "[UChatroom::LeaveChatRoom] Left chat while not logged in");
			// Clear out the chat room to avoid ensures (logged out, not in this room anymore)
			ChatRoomLeftInternal(ChatRoomIdCpy, CompletionDelegate);
		}
	}
	else
	{
		UE_LOGF(LogOnlineChat, Verbose, "[UChatroom::LeaveChatRoom] %ls no chat room to leave.", *LocalUserId.ToDebugString());
	}
}

void UChatroom::OnChatRoomLeft(const FUniqueNetId& LocalUserId, const FChatRoomId& RoomId, bool bWasSuccessful, const FString& Error, FChatRoomId ChatRoomIdCopy, FOnChatRoomLeft CompletionDelegate)
{
	if (ChatRoomIdCopy == RoomId)
	{
		// Not much to do failure or otherwise, just print logs here
		UE_LOGF(LogOnlineChat, Verbose, "[UChatroom::OnChatRoomLeft] %ls left chat room %ls Success: %d%ls", *LocalUserId.ToDebugString(), *RoomId, bWasSuccessful, !Error.IsEmpty() ? *FString::Printf(TEXT(" %s"), *Error) : TEXT(""));

		IOnlineChatPtr ChatInt = Online::GetChatInterface(GetWorld());
		if (ensure(ChatInt.IsValid()))
		{
			ChatInt->ClearOnChatRoomExitDelegate_Handle(ChatRoomLeftDelegateHandle);
		}

		ChatRoomLeftInternal(RoomId, CompletionDelegate);
	}
	else
	{
		UE_LOGF(LogOnlineChat, Verbose, "[UChatroom::OnChatRoomLeft] other chat room detected %ls, waiting for %ls", *RoomId, *ChatRoomIdCopy);
	}
}

void UChatroom::ChatRoomLeftInternal(const FChatRoomId& RoomId, const FOnChatRoomLeft& CompletionDelegate)
{
	GetTimerManager().SetTimerForNextTick(FTimerDelegate::CreateLambda([CompletionDelegate, RoomId]()
	{
		// Announce that chat is available
		CompletionDelegate.ExecuteIfBound(RoomId);
	}));
}

bool UChatroom::IsOnline() const
{
	return true;
}

bool UChatroom::IsAlreadyInChatRoom(const FUniqueNetIdRepl& LocalUserId, const FChatRoomId& ChatRoomId) const
{
	if (ensure(LocalUserId.IsValid()))
	{
		UWorld* World = GetWorld();
		IOnlineChatPtr ChatInt = Online::GetChatInterface(World);
		if (ensure(ChatInt.IsValid()))
		{
			TArray<FChatRoomId> JoinedRooms;
			ChatInt->GetJoinedRooms(*LocalUserId, JoinedRooms);

			auto Pred = [&ChatRoomId](const FChatRoomId& OtherChatRoomId)
			{
				return OtherChatRoomId == ChatRoomId;
			};

			return JoinedRooms.ContainsByPredicate(Pred);
		}
	}

	return false;
}

UWorld* UChatroom::GetWorld() const
{
	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		UE_LOGF(LogOnlineChat, Warning, "[UChatroom::GetWorld] Called GetWorld on the CDO");
		return nullptr;
	}
	else
	{
		UWorld* World = GetOuter() ? GetOuter()->GetWorld() : nullptr;
		checkf(World, TEXT("[UChatroom::GetWorld] Should have an outer that can access a world"));
		return World;
	}
}

FTimerManager& UChatroom::GetTimerManager() const
{
	UWorld* World = GetWorld();
	check(World);
	return World->GetTimerManager();
}


