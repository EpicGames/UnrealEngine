// Copyright Epic Games, Inc. All Rights Reserved.

#include "Interactions/CoreInteractions.h"

#include "SocialManager.h"
#include "Chat/SocialChatManager.h"
#include "HAL/IConsoleManager.h"


#define LOCTEXT_NAMESPACE "UserInteractions"

//////////////////////////////////////////////////////////////////////////
// AddFriend
//////////////////////////////////////////////////////////////////////////

DEFINE_SOCIAL_INTERACTION(AddFriend)

FText FSocialInteraction_AddFriend::GetDisplayName(const USocialUser& User)
{
	if (User.IsFriend(ESocialSubsystem::Platform))
	{
		return LOCTEXT("AddEpicFriend", "Add Epic friend");
	}
	return LOCTEXT("AddFriend", "Add friend");
}

FString FSocialInteraction_AddFriend::GetSlashCommandToken()
{
	return FString();
}

bool FSocialInteraction_AddFriend::CanExecute(const USocialUser& User)
{
	return User.CanSendFriendInvite(ESocialSubsystem::Primary);
}

void FSocialInteraction_AddFriend::ExecuteInteraction(USocialUser& User)
{
	User.SendFriendInvite(ESocialSubsystem::Primary);
}

//////////////////////////////////////////////////////////////////////////
// AddPlatformFriend
//////////////////////////////////////////////////////////////////////////

DEFINE_SOCIAL_INTERACTION(AddPlatformFriend)

FText FSocialInteraction_AddPlatformFriend::GetDisplayName(const USocialUser& User)
{
	const FName PlatformOssName = USocialManager::GetSocialOssName(ESocialSubsystem::Platform);
	if (PlatformOssName == PS4_SUBSYSTEM || USocialSettings::IsSonyOSS(PlatformOssName))
	{
		return LOCTEXT("AddPlatformFriend_PSN", "Add Playstation Network Friend");
	}
	else if (PlatformOssName == TENCENT_SUBSYSTEM)
	{
		return LOCTEXT("AddPlatformFriend_Tencent", "Add WeGame Friend");
	}
	return LOCTEXT("AddPlatformFriend_Unknown", "Add Platform Friend");
}

FString FSocialInteraction_AddPlatformFriend::GetSlashCommandToken()
{
	return TEXT("");
}

bool FSocialInteraction_AddPlatformFriend::CanExecute(const USocialUser& User)
{
	return User.CanSendFriendInvite(ESocialSubsystem::Platform);
}

void FSocialInteraction_AddPlatformFriend::ExecuteInteraction(USocialUser& User)
{
	User.SendFriendInvite(ESocialSubsystem::Platform);
}

//////////////////////////////////////////////////////////////////////////
// RemoveFriend
//////////////////////////////////////////////////////////////////////////

DEFINE_SOCIAL_INTERACTION(RemoveFriend)

FText FSocialInteraction_RemoveFriend::GetDisplayName(const USocialUser& User)
{
	return LOCTEXT("RemoveFriend", "Remove friend");
}

FString FSocialInteraction_RemoveFriend::GetSlashCommandToken()
{
	return LOCTEXT("SlashCommand_Unfriend", "unfriend").ToString();
}

bool FSocialInteraction_RemoveFriend::CanExecute(const USocialUser& User)
{
	return User.IsFriend(ESocialSubsystem::Primary);
}

void FSocialInteraction_RemoveFriend::ExecuteInteraction(USocialUser& User)
{
	//@todo DanH SocialInteractions: Same issue as parties - there can be N different named friends lists. Whatever we do for different party types, do here too. #future
	User.EndFriendship(ESocialSubsystem::Primary);
}

//////////////////////////////////////////////////////////////////////////
// AcceptFriendInvite
//////////////////////////////////////////////////////////////////////////

DEFINE_SOCIAL_INTERACTION(AcceptFriendInvite)

FText FSocialInteraction_AcceptFriendInvite::GetDisplayName(const USocialUser& User)
{
	return LOCTEXT("AcceptFriendInvite", "Accept");
}

FString FSocialInteraction_AcceptFriendInvite::GetSlashCommandToken()
{
	return FString();
}

bool FSocialInteraction_AcceptFriendInvite::CanExecute(const USocialUser& User)
{
	return User.GetFriendInviteStatus(ESocialSubsystem::Primary) == EInviteStatus::PendingInbound;
}

void FSocialInteraction_AcceptFriendInvite::ExecuteInteraction(USocialUser& User)
{
	User.AcceptFriendInvite(ESocialSubsystem::Primary);
}

//////////////////////////////////////////////////////////////////////////
// RejectFriendInvite
//////////////////////////////////////////////////////////////////////////

DEFINE_SOCIAL_INTERACTION(RejectFriendInvite)

FText FSocialInteraction_RejectFriendInvite::GetDisplayName(const USocialUser& User)
{
	return LOCTEXT("RejectFriendInvite", "Reject");
}

FString FSocialInteraction_RejectFriendInvite::GetSlashCommandToken()
{
	return FString();
}

bool FSocialInteraction_RejectFriendInvite::CanExecute(const USocialUser& User)
{
	return User.GetFriendInviteStatus(ESocialSubsystem::Primary) == EInviteStatus::PendingInbound;
}

void FSocialInteraction_RejectFriendInvite::ExecuteInteraction(USocialUser& User)
{
	User.RejectFriendInvite(ESocialSubsystem::Primary);
}

//////////////////////////////////////////////////////////////////////////
// Block
//////////////////////////////////////////////////////////////////////////

DEFINE_SOCIAL_INTERACTION(Block)

FText FSocialInteraction_Block::GetDisplayName(const USocialUser& User)
{
	return LOCTEXT("BlockUser", "Block");
}

FString FSocialInteraction_Block::GetSlashCommandToken()
{
	return LOCTEXT("SlashCommand_BlockUser", "block").ToString();
}

bool FSocialInteraction_Block::CanExecute(const USocialUser& User)
{
	return User.HasSubsystemInfo(ESocialSubsystem::Primary) && !User.IsBlocked(ESocialSubsystem::Primary) && !User.IsLocalUser();
}

void FSocialInteraction_Block::ExecuteInteraction(USocialUser& User)
{
	User.BlockUser(ESocialSubsystem::Primary);
}

//////////////////////////////////////////////////////////////////////////
// Unblock
//////////////////////////////////////////////////////////////////////////

DEFINE_SOCIAL_INTERACTION(Unblock)

FText FSocialInteraction_Unblock::GetDisplayName(const USocialUser& User)
{
	return LOCTEXT("UnblockUser", "Unblock");
}

FString FSocialInteraction_Unblock::GetSlashCommandToken()
{
	return LOCTEXT("SlashCommand_Unblock", "unblock").ToString();
}

bool FSocialInteraction_Unblock::CanExecute(const USocialUser& User)
{
	return User.HasSubsystemInfo(ESocialSubsystem::Primary) && User.IsBlocked(ESocialSubsystem::Primary);
}

void FSocialInteraction_Unblock::ExecuteInteraction(USocialUser& User)
{
	User.UnblockUser(ESocialSubsystem::Primary);
}

//////////////////////////////////////////////////////////////////////////
// PrivateMessage
//////////////////////////////////////////////////////////////////////////

DEFINE_SOCIAL_INTERACTION(PrivateMessage)

FText FSocialInteraction_PrivateMessage::GetDisplayName(const USocialUser& User)
{
	return LOCTEXT("SendPrivateMessage", "Whisper");
}

FString FSocialInteraction_PrivateMessage::GetSlashCommandToken()
{
	return LOCTEXT("SlashCommand_PrivateMessage", "whisper").ToString();
}

bool FSocialInteraction_PrivateMessage::CanExecute(const USocialUser& User)
{
	// Whispering only takes place on the primary subsystem, but is enabled for friends on any subsystem
	return !User.GetOwningToolkit().GetChatManager().IsChatRestricted() &&
		(USocialManager::ShouldReadPresenceFromOnlineServices() ? User.GetOnlineStatusV2() != UE::Online::EUserPresenceStatus::Offline : User.GetOnlineStatus() != EOnlinePresenceState::Offline) &&
		User.IsFriend();
}

void FSocialInteraction_PrivateMessage::ExecuteInteraction(USocialUser& User)
{
	USocialChatManager& ChatManager = User.GetOwningToolkit().GetChatManager();
	ChatManager.CreateChatChannel(User);
	ChatManager.FocusChatChannel(User);
}

//////////////////////////////////////////////////////////////////////////
// ShowPlatformProfile
//////////////////////////////////////////////////////////////////////////

DEFINE_SOCIAL_INTERACTION(ShowPlatformProfile)

FText FSocialInteraction_ShowPlatformProfile::GetDisplayName(const USocialUser& User)
{
	return LOCTEXT("ShowPlatformProfile", "View Profile");
}

FString FSocialInteraction_ShowPlatformProfile::GetSlashCommandToken()
{
	return TEXT("");
}

bool FSocialInteraction_ShowPlatformProfile::CanExecute(const USocialUser& User)
{
	return USocialManager::GetLocalUserPlatform().IsConsole() && User.GetUserId(ESocialSubsystem::Platform).IsValid();
}

void FSocialInteraction_ShowPlatformProfile::ExecuteInteraction(USocialUser& User)
{
	User.ShowPlatformProfile();
}

#undef LOCTEXT_NAMESPACE
