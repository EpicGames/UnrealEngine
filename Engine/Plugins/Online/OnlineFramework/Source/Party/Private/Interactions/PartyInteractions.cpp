// Copyright Epic Games, Inc. All Rights Reserved.

#include "Interactions/PartyInteractions.h"
#include "SocialToolkit.h"

#include "Party/SocialParty.h"
#include "Party/PartyMember.h" 
#include "Engine/LocalPlayer.h"

#define LOCTEXT_NAMESPACE "PartyInteractions"

//////////////////////////////////////////////////////////////////////////
// InviteToParty
//////////////////////////////////////////////////////////////////////////

DEFINE_SOCIAL_INTERACTION(InviteToParty)

FText FSocialInteraction_InviteToParty::GetDisplayName(const USocialUser& User)
{
	return LOCTEXT("InviteToParty", "Invite to party");
}

FString FSocialInteraction_InviteToParty::GetSlashCommandToken()
{
	return (LOCTEXT("SlashCommand_InviteToParty", "invite")).ToString();
}

bool FSocialInteraction_InviteToParty::CanExecute(const USocialUser& User)
{
	return User.CanInviteToParty(IOnlinePartySystem::GetPrimaryPartyTypeId());
}

void FSocialInteraction_InviteToParty::ExecuteInteraction(USocialUser& User)
{
	User.InviteToParty(IOnlinePartySystem::GetPrimaryPartyTypeId());
}

//////////////////////////////////////////////////////////////////////////
// JoinParty
//////////////////////////////////////////////////////////////////////////

DEFINE_SOCIAL_INTERACTION(JoinParty)

FText FSocialInteraction_JoinParty::GetDisplayName(const USocialUser& User)
{
	return LOCTEXT("JoinParty", "Join party");
}

FString FSocialInteraction_JoinParty::GetSlashCommandToken()
{
	return LOCTEXT("SlashCommand_JoinParty", "join").ToString();
}

bool FSocialInteraction_JoinParty::CanExecute(const USocialUser& User)
{
	FJoinPartyResult MockJoinResult = User.CheckPartyJoinability(IOnlinePartySystem::GetPrimaryPartyTypeId());
	return MockJoinResult.WasSuccessful() && User.IsFriend();
}

void FSocialInteraction_JoinParty::ExecuteInteraction(USocialUser& User)
{
	User.JoinParty(IOnlinePartySystem::GetPrimaryPartyTypeId(), PartyJoinMethod::Presence);
}

//////////////////////////////////////////////////////////////////////////
// RequestToJoinParty
//////////////////////////////////////////////////////////////////////////

DEFINE_SOCIAL_INTERACTION(RequestToJoinParty)

FText FSocialInteraction_RequestToJoinParty::GetDisplayName(const USocialUser& User)
{
	return LOCTEXT("RequestToJoinParty", "Request to join");
}

FString FSocialInteraction_RequestToJoinParty::GetSlashCommandToken()
{
	return LOCTEXT("SlashCommand_RequestToJoinParty", "requesttojoin").ToString();
}

bool FSocialInteraction_RequestToJoinParty::CanExecute(const USocialUser& User)
{
	return User.CanRequestToJoin() && User.IsFriend();
}

void FSocialInteraction_RequestToJoinParty::ExecuteInteraction(USocialUser& User)
{
	User.RequestToJoinParty(PartyJoinMethod::RequestToJoin);
}

//////////////////////////////////////////////////////////////////////////
// AcceptJoinRequest
//////////////////////////////////////////////////////////////////////////

DEFINE_SOCIAL_INTERACTION(AcceptJoinRequest)

FText FSocialInteraction_AcceptJoinRequest::GetDisplayName(const USocialUser& User)
{
	return LOCTEXT("AcceptJoinRequest", "Accept request");
}

FString FSocialInteraction_AcceptJoinRequest::GetSlashCommandToken()
{
	return LOCTEXT("SlashCommand_AcceptJoinRequest", "acceptjoinrequest").ToString();
}

bool FSocialInteraction_AcceptJoinRequest::CanExecute(const USocialUser& User)
{
	return User.HasRequestedToJoinUs();
}

void FSocialInteraction_AcceptJoinRequest::ExecuteInteraction(USocialUser& User)
{
	User.AcceptRequestToJoinParty();
}

//////////////////////////////////////////////////////////////////////////
// DismissJoinRequest
//////////////////////////////////////////////////////////////////////////

DEFINE_SOCIAL_INTERACTION(DismissJoinRequest)

FText FSocialInteraction_DismissJoinRequest::GetDisplayName(const USocialUser& User)
{
	return LOCTEXT("DismissJoinRequest", "Dismiss request");
}

FString FSocialInteraction_DismissJoinRequest::GetSlashCommandToken()
{
	return LOCTEXT("SlashCommand_DismissJoinRequest", "dismissjoinrequest").ToString();
}

bool FSocialInteraction_DismissJoinRequest::CanExecute(const USocialUser& User)
{
	return User.HasRequestedToJoinUs();
}

void FSocialInteraction_DismissJoinRequest::ExecuteInteraction(USocialUser& User)
{
	User.DismissRequestToJoinParty();
}

//////////////////////////////////////////////////////////////////////////
// AcceptPartyInvite
//////////////////////////////////////////////////////////////////////////

DEFINE_SOCIAL_INTERACTION(AcceptPartyInvite)

FText FSocialInteraction_AcceptPartyInvite::GetDisplayName(const USocialUser& User)
{
	return LOCTEXT("AcceptPartyInvite", "Accept invite");
}

FString FSocialInteraction_AcceptPartyInvite::GetSlashCommandToken()
{
	//join should be the preferred method of accepting a party invite
	return FString();
}

bool FSocialInteraction_AcceptPartyInvite::CanExecute(const USocialUser& User)
{
	return User.HasSentPartyInvite(IOnlinePartySystem::GetPrimaryPartyTypeId()) && User.CheckPartyJoinability(IOnlinePartySystem::GetPrimaryPartyTypeId()).WasSuccessful();
}

void FSocialInteraction_AcceptPartyInvite::ExecuteInteraction(USocialUser& User)
{
	User.JoinParty(IOnlinePartySystem::GetPrimaryPartyTypeId(), PartyJoinMethod::Invitation);
}

//////////////////////////////////////////////////////////////////////////
// RejectPartyInvite
//////////////////////////////////////////////////////////////////////////

DEFINE_SOCIAL_INTERACTION(RejectPartyInvite)

FText FSocialInteraction_RejectPartyInvite::GetDisplayName(const USocialUser& User)
{
	return LOCTEXT("RejectPartyInvite", "Reject invite");
}

FString FSocialInteraction_RejectPartyInvite::GetSlashCommandToken()
{
	return FString();
}

bool FSocialInteraction_RejectPartyInvite::CanExecute(const USocialUser& User)
{
	return User.HasSentPartyInvite(IOnlinePartySystem::GetPrimaryPartyTypeId());
}

void FSocialInteraction_RejectPartyInvite::ExecuteInteraction(USocialUser& User)
{
	User.RejectPartyInvite(IOnlinePartySystem::GetPrimaryPartyTypeId());
}

//////////////////////////////////////////////////////////////////////////
// LeaveParty
//////////////////////////////////////////////////////////////////////////

DEFINE_SOCIAL_INTERACTION(LeaveParty)

FText FSocialInteraction_LeaveParty::GetDisplayName(const USocialUser& User)
{
	return LOCTEXT("LeaveParty", "Leave party");
}

FString FSocialInteraction_LeaveParty::GetSlashCommandToken()
{
	return LOCTEXT("SlashCommand_LeaveParty", "leave").ToString();
}

bool FSocialInteraction_LeaveParty::CanExecute(const USocialUser& User)
{
	if (User.IsLocalUser())
	{
		const USocialToolkit& OwningToolkit = User.GetOwningToolkit();
		const ULocalPlayer* LocalPlayer = OwningToolkit.GetOwningLocalPlayerPtr();
		if (!LocalPlayer || !LocalPlayer->IsPrimaryPlayer())
		{
			return false;
		}

		const UPartyMember* LocalMember = User.GetPartyMember(IOnlinePartySystem::GetPrimaryPartyTypeId());
		return LocalMember && LocalMember->GetParty().GetNumPartyMembers() > 1;
	}
	return false;
}

void FSocialInteraction_LeaveParty::ExecuteInteraction(USocialUser& User)
{
	if (const UPartyMember* LocalMember = User.GetPartyMember(IOnlinePartySystem::GetPrimaryPartyTypeId()))
	{
		LocalMember->GetParty().LeaveParty();
	}
}

//////////////////////////////////////////////////////////////////////////
// KickPartyMember
//////////////////////////////////////////////////////////////////////////

DEFINE_SOCIAL_INTERACTION(KickPartyMember)

FText FSocialInteraction_KickPartyMember::GetDisplayName(const USocialUser& User)
{
	return LOCTEXT("KickPartyMember", "Kick");
}

FString FSocialInteraction_KickPartyMember::GetSlashCommandToken()
{
	return LOCTEXT("SlashCommand_KickMember", "kick").ToString();
}

bool FSocialInteraction_KickPartyMember::CanExecute(const USocialUser& User)
{
	const ULocalPlayer* LocalPlayer = User.GetOwningToolkit().GetOwningLocalPlayerPtr();
	const UPartyMember* PartyMember = User.GetPartyMember(IOnlinePartySystem::GetPrimaryPartyTypeId());
	return LocalPlayer && PartyMember && PartyMember->CanKickFromParty(*LocalPlayer);
}

void FSocialInteraction_KickPartyMember::ExecuteInteraction(USocialUser& User)
{
	const ULocalPlayer* LocalPlayer = User.GetOwningToolkit().GetOwningLocalPlayerPtr();
	UPartyMember* PartyMember = User.GetPartyMember(IOnlinePartySystem::GetPrimaryPartyTypeId());
	if (LocalPlayer && PartyMember)
	{
		PartyMember->KickFromParty(*LocalPlayer);
	}
}

//////////////////////////////////////////////////////////////////////////
// PromoteToPartyLeader
//////////////////////////////////////////////////////////////////////////

DEFINE_SOCIAL_INTERACTION(PromoteToPartyLeader)

FText FSocialInteraction_PromoteToPartyLeader::GetDisplayName(const USocialUser& User)
{
	return LOCTEXT("PromoteToPartyLeader", "Promote");
}

FString FSocialInteraction_PromoteToPartyLeader::GetSlashCommandToken()
{
	return LOCTEXT("SlashCommand_PromoteToLeader", "promote").ToString();
}

bool FSocialInteraction_PromoteToPartyLeader::CanExecute(const USocialUser& User)
{
	const ULocalPlayer* LocalPlayer = User.GetOwningToolkit().GetOwningLocalPlayerPtr();
	const UPartyMember* PartyMember = User.GetPartyMember(IOnlinePartySystem::GetPrimaryPartyTypeId());
	return LocalPlayer && PartyMember && PartyMember->CanPromoteToLeader(*LocalPlayer);
}

void FSocialInteraction_PromoteToPartyLeader::ExecuteInteraction(USocialUser& User)
{
	const ULocalPlayer* LocalPlayer = User.GetOwningToolkit().GetOwningLocalPlayerPtr();
	UPartyMember* PartyMember = User.GetPartyMember(IOnlinePartySystem::GetPrimaryPartyTypeId());
	if (LocalPlayer && PartyMember)
	{
		PartyMember->PromoteToPartyLeader(*LocalPlayer);
	}
}

#undef LOCTEXT_NAMESPACE
