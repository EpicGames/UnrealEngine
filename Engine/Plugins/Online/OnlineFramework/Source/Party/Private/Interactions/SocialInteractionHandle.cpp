// Copyright Epic Games, Inc. All Rights Reserved.

#include "Interactions/SocialInteractionHandle.h"
#include "Interactions/SocialInteractionMacros.h"

FSocialInteractionHandle::FSocialInteractionHandle(const ISocialInteractionWrapper& Wrapper)
	: InteractionWrapper(&Wrapper)
{}

bool FSocialInteractionHandle::IsValid() const
{
	return InteractionWrapper != nullptr;
}

bool FSocialInteractionHandle::operator==(const FSocialInteractionHandle& Other) const
{
	return InteractionWrapper == Other.InteractionWrapper;
}

FName FSocialInteractionHandle::GetInteractionName() const
{
	return InteractionWrapper ? InteractionWrapper->GetInteractionName() : NAME_None;
}

FText FSocialInteractionHandle::GetDisplayName(const USocialUser& User) const
{
	return InteractionWrapper ? InteractionWrapper->GetDisplayName(User) : FText::GetEmpty();
}

FString FSocialInteractionHandle::GetSlashCommandToken() const
{
	return InteractionWrapper ? InteractionWrapper->GetSlashCommandToken() : FString();
}

bool FSocialInteractionHandle::IsAvailable(const USocialUser& User) const
{
	return InteractionWrapper ? InteractionWrapper->IsAvailable(User) : false;
}

void FSocialInteractionHandle::ExecuteInteraction(USocialUser& User) const
{
	if (InteractionWrapper)
	{
		InteractionWrapper->ExecuteInteraction(User);
	}
}

void FSocialInteractionHandle::ExecuteInteractionWithContext(USocialUser& User, const TMap<FString, FString>& AnalyticsContext) const
{
	if (InteractionWrapper)
	{
		InteractionWrapper->ExecuteInteractionWithContext(User, AnalyticsContext);
	}
}