// Copyright Epic Games, Inc. All Rights Reserved.

#include "SocialSettingsXbl.h"

USocialSettingsXbl::USocialSettingsXbl()
{
}

bool USocialSettingsXbl::ShouldApplyPlatformInvitePermissionsLive()
{
	const USocialSettingsXbl& SettingsCDO = *GetDefault<USocialSettingsXbl>();
	return SettingsCDO.bApplyPlatformInvitePermissionsLive;
}

bool USocialSettingsXbl::ShouldApplyPlatformInvitePermissionsCrossplay()
{
	const USocialSettingsXbl& SettingsCDO = *GetDefault<USocialSettingsXbl>();
	return SettingsCDO.bApplyPlatformInvitePermissionsCrossplay;
}