// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"

#include "SocialSettingsXbl.generated.h"

/**
 * Config-driven settings object for the social framework.
 * Only the CDO is ever expected to be used, no instance is ever expected to be created.
 */
UCLASS(Config = Game)
class USocialSettingsXbl : public UObject
{
	GENERATED_BODY()

public:
	USocialSettingsXbl();

	/** Should GDK permissions be checked for invites received from other xbl users */
	static bool ShouldApplyPlatformInvitePermissionsLive();
	/** Should GDK permissions be checked for invites received from non-xbl users */
	static bool ShouldApplyPlatformInvitePermissionsCrossplay();

private:

	/** Should GDK permissions be checked for invites received from other xbl users */
	UPROPERTY(config)
	bool bApplyPlatformInvitePermissionsLive = true;

	/** Should GDK permissions be checked for invites received from non-xbl users */
	UPROPERTY(config)
	bool bApplyPlatformInvitePermissionsCrossplay = false;
};