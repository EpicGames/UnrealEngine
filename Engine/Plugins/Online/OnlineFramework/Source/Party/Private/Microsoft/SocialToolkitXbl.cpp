// Copyright Epic Games, Inc. All Rights Reserved.

#include "SocialToolkit.h"

#if PARTY_PLATFORM_XBL_INVITE_PERMISSIONS

#if !PARTY_PLATFORM_INVITE_PERMISSIONS
	#error expected PARTY_PLATFORM_INVITE_PERMISSIONS when PARTY_PLATFORM_XBL_INVITE_PERMISSIONS is set
#endif

#include "OnlineSubsystemGDK.h"
#include "OnlineSubsystemGDKTypes.h"
#include "OnlineUserInterfaceGDK.h"
#include "SocialSettingsXbl.h"

void USocialToolkit::CanReceiveInviteFrom(USocialUser& User, const IOnlinePartyJoinInfoConstRef& Invite, TFunction<void(const bool /*bResult*/)>&& CompletionFunc)
{	
	bool bQueryStarted = false;
	if (FOnlineSubsystemGDK* OssGDK = static_cast<FOnlineSubsystemGDK*>(IOnlineSubsystem::Get(GDK_SUBSYSTEM)))
	{
		if (FOnlineUserGDKPtr UserInterface = OssGDK->GetUsersGDK())
		{
			FUniqueNetIdRepl LocalUserPlatformId = GetLocalUserNetId(ESocialSubsystem::Platform);
			if (LocalUserPlatformId.IsValid())
			{
				// Need to work out if the invite is from the same platform, so we know which permissions API to call, the one for GDK users, or the one for cross platform
				const FString LocalExternalAccountType = FUserPlatform(IOnlineSubsystem::GetLocalPlatformName()).GetPlatformDescription().ExternalAccountType;
				const FString InviterExternalAccountType = FUserPlatform(Invite->GetSourcePlatform()).GetPlatformDescription().ExternalAccountType;

				if(LocalExternalAccountType == InviterExternalAccountType
					&& USocialSettingsXbl::ShouldApplyPlatformInvitePermissionsLive())
				{
					FUniqueNetIdRepl SocialUserPlatformId = User.GetUserId(ESocialSubsystem::Platform);
					if (ensure(SocialUserPlatformId.IsValid() && (SocialUserPlatformId.GetType() == GDK_SUBSYSTEM)))
					{
						UE_LOGF(LogParty, Log, "%ls Inviter=[%ls] InviterPlatformId=[%ls] checking XBL invite permissions", ANSI_TO_TCHAR(__FUNCTION__), *User.ToDebugString(), *SocialUserPlatformId->ToDebugString());

						// GDK user, so check with the "other GDK user" api
						const FUniqueNetIdRef& SocialUserPlatformNetIdRef = SocialUserPlatformId.GetUniqueNetId().ToSharedRef();
						UserInterface->QueryUserCommunicationPermissions(
							*LocalUserPlatformId.GetUniqueNetId(),
							{ SocialUserPlatformNetIdRef },
							// No explicit invite permission, apparently have to check for Text+Voice comms, see https://learn.microsoft.com/gaming/gdk/_content/gc/policies/XR/XR015
							{ XblPermission::CommunicateUsingText, XblPermission::CommunicateUsingVoice },
							FOnGDKCommunicationPermissionsQueryComplete::CreateWeakLambda(this,
								[this, SocialUserPlatformNetIdRef, CompletionFunc](const FOnlineError& RequestStatus, const FUniqueNetIdRef& /*RequestingUser*/, const FCommunicationPermissionResultsMap& Results)
						{
							bool bCanReceiveInvites = true;
							if (RequestStatus.WasSuccessful())
							{
								if (const bool* bCanCommunicate = Results.Find(SocialUserPlatformNetIdRef))
								{
									bCanReceiveInvites = *bCanCommunicate;
								}
							}
							CompletionFunc(bCanReceiveInvites);
						}));
						bQueryStarted = true;
					}
					else
					{
						UE_LOGF(LogParty, Warning, "%ls SocialUserPlatformId=[%ls] ExternalAccountTypes match, but NetId unexpected", ANSI_TO_TCHAR(__FUNCTION__), *SocialUserPlatformId.ToDebugString());
					}
				}
				else if (LocalExternalAccountType != InviterExternalAccountType
					&& USocialSettingsXbl::ShouldApplyPlatformInvitePermissionsCrossplay())
				{
					UE_LOGF(LogParty, Log, "%ls Inviter=[%ls] checking crossplay invite permissions", ANSI_TO_TCHAR(__FUNCTION__), *User.ToDebugString());

					// Crossplay user, so check with crossplay api
					const XblAnonymousUserType UserType = User.IsFriend(ESocialSubsystem::Primary) ? XblAnonymousUserType::CrossNetworkFriend : XblAnonymousUserType::CrossNetworkUser;
					UserInterface->QueryAnonymousUserCommunicationPermissions(
						*LocalUserPlatformId.GetUniqueNetId(),
						{ UserType },
						{ XblPermission::CommunicateUsingText, XblPermission::CommunicateUsingVoice },
						FOnGDKAnonymousUserCommunicationPermissionsQueryComplete::CreateWeakLambda(this,
							[this, UserType, CompletionFunc](const FOnlineError& RequestStatus, const FAnonymousUserCommunicationPermissionResultsMap& Results)
					{
						bool bCanReceiveInvites = true;
						if (RequestStatus.WasSuccessful())
						{
							if (const bool* bCanCommunicate = Results.Find(UserType))
							{
								bCanReceiveInvites = *bCanCommunicate;
							}
						}
						CompletionFunc(bCanReceiveInvites);
					}));
					bQueryStarted = true;
				}
			}
			else
			{
				UE_LOGF(LogParty, Warning, "%ls LocalUserPlatformId=[%ls] invalid", ANSI_TO_TCHAR(__FUNCTION__), *LocalUserPlatformId.ToDebugString());
			}
		}
		else
		{
			UE_LOGF(LogParty, Warning, "%ls UserInterface=nullptr", ANSI_TO_TCHAR(__FUNCTION__));
		}
	}
	else
	{
		UE_LOGF(LogParty, Warning, "%ls OssGDK=nullptr", ANSI_TO_TCHAR(__FUNCTION__));
	}

	if (!bQueryStarted)
	{
		CompletionFunc(true);
	}
}

#endif // PARTY_PLATFORM_XBL_INVITE_PERMISSIONS

