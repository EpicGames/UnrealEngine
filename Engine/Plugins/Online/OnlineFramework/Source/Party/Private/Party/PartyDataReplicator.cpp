// Copyright Epic Games, Inc. All Rights Reserved.

#include "Party/PartyDataReplicator.h"
#include "Party/SocialParty.h"
#include "Party/PartyMember.h"
#include "OnlineSubsystemUtils.h"

namespace PartyDataCvars
{
	static bool bAsyncLoadingEnabled = true;
	FAutoConsoleVariableRef CVarAsyncLoadingEnabled(
		TEXT("Party.ReplicationAsyncLoadingEnabled"),
		bAsyncLoadingEnabled,
		TEXT("When enabled, party data will be given a chance to async load and required assets upon replication.")
	);
}

namespace UE::Party
{
	bool IsRepDataAsyncLoadingEnabled()
	{
		return PartyDataCvars::bAsyncLoadingEnabled;
	}
}

void FPartyDataReplicatorHelper::ReplicateDataToMembers(const FOnlinePartyRepDataBase& RepDataInstance, const UScriptStruct& RepDataType, const FOnlinePartyData& ReplicationPayload)
{
	if (const USocialParty* OwnerParty = static_cast<const FOnlinePartyRepDataBase*>(&RepDataInstance)->GetOwnerParty())
	{
		FUniqueNetIdRepl LocalUserId = OwnerParty->GetOwningLocalUserId();
		IOnlinePartyPtr PartyInterface = Online::GetPartyInterface(OwnerParty->GetWorld());
		if (LocalUserId.IsValid() && PartyInterface.IsValid())
		{
			const FOnlinePartyId& PartyId = OwnerParty->GetPartyId();
			if (RepDataType.IsChildOf(FPartyRepData::StaticStruct()))
			{
				UE_LOGF(LogParty, VeryVerbose, "Sending rep data update for party [%ls].", *OwnerParty->ToDebugString());
				PartyInterface->UpdatePartyData(*LocalUserId, OwnerParty->GetPartyId(), DefaultPartyDataNamespace, ReplicationPayload);
			}
			else if (RepDataType.IsChildOf(FPartyMemberRepData::StaticStruct()))
			{
				if (const UPartyMember* Owner = static_cast<const FOnlinePartyRepDataBase*>(&RepDataInstance)->GetOwningMember())
				{
					LocalUserId = Owner->GetPrimaryNetId();
					UE_LOGF(LogParty, VeryVerbose, "Sending rep data update for member within party [%ls].", *OwnerParty->ToDebugString());
					PartyInterface->UpdatePartyMemberData(*LocalUserId, OwnerParty->GetPartyId(), DefaultPartyDataNamespace, ReplicationPayload);
				}
				else
				{
					UE_LOGF(LogParty, Warning, "Sending rep data update for member within party [%ls] Could not identidy owner.", *OwnerParty->ToDebugString());
					PartyInterface->UpdatePartyMemberData(*LocalUserId, OwnerParty->GetPartyId(), DefaultPartyDataNamespace, ReplicationPayload);
				}				
			}
		}
	}
}