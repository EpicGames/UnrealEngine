// Copyright Epic Games, Inc. All Rights Reserved.

#include "Party/PartyPlatformSessionMonitor.h"

#include "Interfaces/OnlineSessionDelegates.h"
#include "Party/SocialParty.h"
#include "Online/OnlineSessionNames.h"
#include "Party/PartyMember.h"
#include "SocialToolkit.h"
#include "SocialManager.h"

#include "OnlineSessionSettings.h"
#include "OnlineSubsystemUtils.h"

#if PARTY_PLATFORM_SESSIONS_XBL
#include "Misc/Base64.h"
#endif

static bool IsTencentPlatform()
{
	return USocialManager::GetSocialOssName(ESocialSubsystem::Platform) == TENCENT_SUBSYSTEM;
}

//////////////////////////////////////////////////////////////////////////
// CVars
//////////////////////////////////////////////////////////////////////////

#if !UE_BUILD_SHIPPING

static int32 ForcePlatformSessionFindFailure = 0;
static FAutoConsoleVariableRef CVar_ForcePlatformSessionFindFailure(
	TEXT("Party.PlatformSession.Find.ForceFail"),
	ForcePlatformSessionFindFailure,
	TEXT("Always fail to find platform sessions.\n")
	TEXT("0: Do not force fail platform session finds (default).\n")
	TEXT("1: Fail the find without attempting it.\n"),
	ECVF_Cheat
);

static float PlatformSessionFindDelay = 0.f;
static FAutoConsoleVariableRef CVar_PlatformSessionFindDelay(
	TEXT("Party.PlatformSession.Find.Delay"),
	PlatformSessionFindDelay,
	TEXT("Simulated delay (in seconds) between beginning an attempt to find a platform session and actually making the call the OSS."),
	ECVF_Cheat
);

static int32 ForcePlatformSessionCreationFailure = 0;
static FAutoConsoleVariableRef CVar_ForcePlatformSessionCreationFailure(
	TEXT("Party.PlatformSession.Create.ForceFail"),
	ForcePlatformSessionCreationFailure,
	TEXT("Always fail to create platform sessions.\n")
	TEXT("0: Do not force fail platform session creates (default).\n")
	TEXT("1: Fail the create without attempting it.\n"),
	ECVF_Cheat
);

static float PlatformSessionCreationDelay = 0.f;
static FAutoConsoleVariableRef CVar_PlatformSessionCreationDelay(
	TEXT("Party.PlatformSession.Create.Delay"),
	PlatformSessionCreationDelay,
	TEXT("Simulated delay (in seconds) between beginning an attempt to create a platform session and actually making the call the OSS."),
	ECVF_Cheat
);

static int32 ForcePlatformSessionJoinFailure = 0;
static FAutoConsoleVariableRef CVar_ForcePlatformSessionJoinFailure(
	TEXT("Party.PlatformSession.Join.ForceFail"),
	ForcePlatformSessionJoinFailure,
	TEXT("Always fail to join platform sessions.\n")
	TEXT("0: Do not force fail platform session joins (default).\n")
	TEXT("1: Force fail the join without attempting it.\n"),
	ECVF_Cheat
);

static float PlatformSessionJoinDelay = 0.f;
static FAutoConsoleVariableRef CVar_PlatformSessionJoinDelay(
	TEXT("Party.PlatformSession.Join.Delay"),
	PlatformSessionJoinDelay,
	TEXT("Simulated delay (in seconds) between beginning an attempt to join a platform session and actually making the call to the OSS."),
	ECVF_Cheat
);

#endif

static int32 AllowCreateSessionFailure = 1;
static FAutoConsoleVariableRef CVar_AllowCreateSessionFailure(
	TEXT("Party.PlatformSession.Create.AllowFailure"),
	AllowCreateSessionFailure,
	TEXT("Are we ok with allowing party session creation to fail? If not, we'll continuously retry until we succeed or leave the party."),
	ECVF_Default
);

static float EstablishSessionRetryDelay = 30.f;
static FAutoConsoleVariableRef CVar_EstablishSessionRetryDelay(
	TEXT("Party.PlatformSession.RetryDelay"),
	EstablishSessionRetryDelay,
	TEXT("Time in seconds to wait between reattempts to create or join a party platform session."),
	ECVF_Default
);

#if PARTY_PLATFORM_SESSIONS_XBL
extern TAutoConsoleVariable<bool> CVarXboxMpaEnabled;
#endif

//////////////////////////////////////////////////////////////////////////
// FPartySessionManager
//////////////////////////////////////////////////////////////////////////

static const FSocialPlatformDescription* FindPlatformDescriptionByOss(FName OssName)
{
	return USocialSettings::GetSocialPlatformDescriptionForOnlineSubsystem(OssName);
}

bool FPartyPlatformSessionManager::DoesOssNeedPartySession(FName OssName)
{
	const FSocialPlatformDescription* const PlatformDescription = FindPlatformDescriptionByOss(OssName);
	return PlatformDescription && !PlatformDescription->SessionType.IsEmpty();
}

TOptional<FString> FPartyPlatformSessionManager::GetOssPartySessionType(FName OssName)
{
	TOptional<FString> SessionType;
	const FSocialPlatformDescription* const PlatformDescription = FindPlatformDescriptionByOss(OssName);
	if (PlatformDescription && !PlatformDescription->SessionType.IsEmpty())
	{
		SessionType.Emplace(PlatformDescription->SessionType);
	}
	return SessionType;
}

TSharedRef<FPartyPlatformSessionManager> FPartyPlatformSessionManager::Create(USocialManager& InSocialManager)
{
	TSharedRef<FPartyPlatformSessionManager> NewManager = MakeShareable(new FPartyPlatformSessionManager(InSocialManager));
	NewManager->InitSessionManager();
	return NewManager;
}

FPartyPlatformSessionManager::FPartyPlatformSessionManager(USocialManager& InSocialManager)
	: SocialManager(InSocialManager)
	, PlatformOssName(USocialManager::GetSocialOssName(ESocialSubsystem::Platform))
{
	check(FPartyPlatformSessionManager::DoesOssNeedPartySession(PlatformOssName));
}

void FPartyPlatformSessionManager::InitSessionManager()
{
	SocialManager.OnPartyJoined().AddSP(this, &FPartyPlatformSessionManager::HandlePartyJoined);
}

void FPartyPlatformSessionManager::CreateMonitor(USocialParty& Party)
{
	TSharedRef<FPartyPlatformSessionMonitor> NewMonitor = FPartyPlatformSessionMonitor::Create(AsShared(), Party);
	ActiveMonitors.Add(NewMonitor);
	NewMonitor->OnShutdownComplete.BindSP(this, &FPartyPlatformSessionManager::HandleMonitorShutdownComplete);
}

bool FPartyPlatformSessionManager::FindSession(const USocialUser& User, const FOnFindSessionAttemptComplete& OnAttemptComplete)
{
	UE_LOGF(LogParty, Log, "FPartyPlatformSessionManager finding party platform session of user [%ls]", *User.GetDisplayName());

	FSessionId SessionId;
	FUniqueNetIdRepl UserPlatformId;
	if (const FOnlineUserPresence* PlatformPresence = User.GetFriendPresenceInfo(ESocialSubsystem::Platform))
	{
		if (PlatformPresence->SessionId)
		{
			SessionId = PlatformPresence->SessionId->ToString();
		}

		UserPlatformId = User.GetUserId(ESocialSubsystem::Platform);
	}
	return FindSessionInternal(SessionId, UserPlatformId, OnAttemptComplete);
}

bool FPartyPlatformSessionManager::FindSession(const FPartyPlatformSessionInfo& SessionInfo, const FOnFindSessionAttemptComplete& OnAttemptComplete)
{
	UE_LOGF(LogParty, Log, "FPartyPlatformSessionManager finding party platform session [%ls]", *SessionInfo.ToDebugString());

	const USocialToolkit* SocialToolkit = SocialManager.GetFirstLocalUserToolkit();
	check(SocialToolkit);

	// Look up the platform Id of the session owner from their corresponding SocialUser 
	const USocialUser* SessionOwnerUser = SocialToolkit->FindUser(SessionInfo.OwnerPrimaryId);
	if (ensure(SessionOwnerUser))
	{
		return FindSessionInternal(SessionInfo.SessionId, SessionOwnerUser->GetUserId(ESocialSubsystem::Platform), OnAttemptComplete);
	}
	return false;
}

IOnlineSessionPtr FPartyPlatformSessionManager::GetSessionInterface()
{
	return Online::GetSessionInterfaceChecked(SocialManager.GetWorld(), PlatformOssName);
}

IOnlineFriendsPtr FPartyPlatformSessionManager::GetFriendsInterface()
{
	return Online::GetFriendsInterfaceChecked(SocialManager.GetWorld(), PlatformOssName);
}

FUniqueNetIdRepl FPartyPlatformSessionManager::GetLocalUserPlatformId() const
{
	return SocialManager.GetFirstLocalUserId(ESocialSubsystem::Platform);
}

bool FPartyPlatformSessionManager::FindSessionInternal(const FSessionId& SessionIdString, const FUniqueNetIdRepl& SessionOwnerId, const FOnFindSessionAttemptComplete& OnAttemptComplete)
{
	if (!SessionIdString.IsEmpty() && SessionOwnerId.IsValid())
	{
		const FUniqueNetIdRepl& LocalUserPlatformId = GetLocalUserPlatformId();
		if (ensure(LocalUserPlatformId.IsValid()))
		{
			const IOnlineSessionPtr& SessionInterface = GetSessionInterface();
			FUniqueNetIdPtr SessionId = SessionInterface->CreateSessionIdFromString(SessionIdString);
			if (SessionId.IsValid())
			{
#if !UE_BUILD_SHIPPING
				float DelaySeconds = FMath::Max(0.f, PlatformSessionFindDelay);
				if (DelaySeconds > 0.f || ForcePlatformSessionFindFailure != 0)
				{
					UE_LOGF(LogParty, Warning, "PartyPlatformSessionMonitor adding artificial delay of %0.2fs to session find attempt", DelaySeconds);

					TWeakPtr<FPartyPlatformSessionManager> AsWeakPtr = SharedThis(this);
					FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda(
						[AsWeakPtr, SessionId, SessionIdString, SessionOwnerId, LocalUserPlatformId, OnAttemptComplete, this](float)
						{
							QUICK_SCOPE_CYCLE_COUNTER(STAT_FPartyPlatformSessionManager_FindSessionAttempt);
							if (AsWeakPtr.IsValid())
							{
								if (ForcePlatformSessionFindFailure != 0)
								{
									UE_LOGF(LogParty, Warning, "Forcing session find failure");
									ProcessCompletedSessionSearch(SocialManager.GetFirstLocalUserNum(), false, FOnlineSessionSearchResult(), SessionIdString, SessionOwnerId, OnAttemptComplete);
								}
								else
								{
									GetSessionInterface()->FindSessionById(*LocalUserPlatformId, *SessionId, *SessionOwnerId.GetV1(), FOnSingleSessionResultCompleteDelegate::CreateSP(this, &FPartyPlatformSessionManager::HandleFindSessionByIdComplete, SessionIdString, SessionOwnerId, OnAttemptComplete));
								}
							}
							return false; // Don't retick
						}), DelaySeconds);
					return ForcePlatformSessionFindFailure != 0;
				}
#endif
				// Always start by trying to find the session directly by ID
				return SessionInterface->FindSessionById(*LocalUserPlatformId, *SessionId, *SessionOwnerId.GetV1(), FOnSingleSessionResultCompleteDelegate::CreateSP(this, &FPartyPlatformSessionManager::HandleFindSessionByIdComplete, SessionIdString, SessionOwnerId, OnAttemptComplete));
			}
			else
			{
				UE_LOGF(LogParty, Warning, "PartyPlatformSessionMonitor could not create a session id from string [%ls]", *SessionIdString);
			}
		}
	}
	return false;
}

void FPartyPlatformSessionManager::HandleFindSessionByIdComplete(int32 LocalUserNum, bool bWasSuccessful, const FOnlineSessionSearchResult& FoundSession, FSessionId SessionId, FUniqueNetIdRepl SessionOwnerId, FOnFindSessionAttemptComplete OnAttemptComplete)
{
	UE_LOGF(LogParty, Log, "FPartyPlatformSessionManager completed attempt to find platform session [%ls] of user [%ls] by SessionId with result [%d]", *SessionId, *SessionOwnerId.ToDebugString(), (int32)bWasSuccessful);

	if (bWasSuccessful && FoundSession.IsSessionInfoValid())
	{
		ProcessCompletedSessionSearch(LocalUserNum, true, FoundSession, SessionId, SessionOwnerId, OnAttemptComplete);
	}
	else
	{
#if PARTY_PLATFORM_SESSIONS_PSN
		//@todo DanH: Obviously remove all of this asap - we need the PSN OSS to be able to get updated presence info for a single user without querying the whole frigging list
		//		Also, querying the list shouldn't wipe the existing friend infos, it should update them surgically, in which case we wouldn't need this notify regardless of additional queries

		// Only attempt again if we haven't already tried getting an updated friends list (to prevent looping)
		if (!bHasAlreadyRequeriedPSNFriends)
		{
			UE_LOGF(LogParty, Log, "PartyPlatformSessionManager failed to find PSN party session, requerying PSN friends list now.");

			bHasAlreadyRequeriedPSNFriends = true;
			Online::GetFriendsInterfaceChecked(SocialManager.GetWorld(), USocialManager::GetSocialOssName(ESocialSubsystem::Platform))->ReadFriendsList(LocalUserNum, EFriendsLists::ToString(EFriendsLists::Default), FOnReadFriendsListComplete::CreateSP(this, &FPartyPlatformSessionManager::HandleReadPSNFriendsListComplete, SessionId, SessionOwnerId, OnAttemptComplete));
			return;
		}
#endif
		if (IsTencentPlatform())
		{
			// Fallback to FindFriendSession on Tencent
			const IOnlineSessionPtr& SessionInterface = GetSessionInterface();
			SessionInterface->AddOnFindFriendSessionCompleteDelegate_Handle(LocalUserNum, FOnFindFriendSessionCompleteDelegate::CreateSP(this, &FPartyPlatformSessionManager::HandleFindFriendSessionsComplete, SessionId, SessionOwnerId, OnAttemptComplete));
			SessionInterface->FindFriendSession(LocalUserNum, *SessionOwnerId);
		}
		else
		{
			OnAttemptComplete.ExecuteIfBound(false, FoundSession);
		}
	}
}

void FPartyPlatformSessionManager::ProcessCompletedSessionSearch(int32 LocalUserNum, bool bWasSuccessful, const FOnlineSessionSearchResult& FoundSession, const FSessionId& SessionId, const FUniqueNetIdRepl& SessionOwnerId, const FOnFindSessionAttemptComplete& OnAttemptComplete)
{
#if PARTY_PLATFORM_SESSIONS_PSN
	bHasAlreadyRequeriedPSNFriends = false;
#endif

	UE_LOGF(LogParty, Log, "PartyPlatformSessionManager has fully completed its search for session [%ls] associated with user [%ls] with result [%d]", *SessionId, *SessionOwnerId.ToDebugString(), (int32)bWasSuccessful);
	OnAttemptComplete.ExecuteIfBound(bWasSuccessful, FoundSession);
}

#if PARTY_PLATFORM_SESSIONS_PSN
void FPartyPlatformSessionManager::HandleReadPSNFriendsListComplete(int32 LocalUserNum, bool bWasSuccessful, const FString& ListName, const FString& ErrorStr, FSessionId OriginalSessionId, FUniqueNetIdRepl SessionOwnerId, FOnFindSessionAttemptComplete OnAttemptComplete)
{
	UE_LOGF(LogParty, Log, "PartyPlatformSessionManager completed requery of the PSN friends list for user [%ls] with result [%d] and Error [%ls]", *SessionOwnerId.ToDebugString(), (int32)bWasSuccessful, *ErrorStr);

	if (bWasSuccessful)
	{
		if (USocialToolkit* SocialToolkit = SocialManager.GetSocialToolkit(LocalUserNum))
		{
			SocialToolkit->NotifyPSNFriendsListRebuilt();
		}

		// We've successfully re-queried the friends list on PSN, so run the whole thing again from the top using the session ID from the updated friend info
		USocialUser* TargetUser = SocialManager.GetSocialToolkit(LocalUserNum)->FindUser(SessionOwnerId);
		const FOnlineUserPresence* PSNPresence = ensure(TargetUser) ? TargetUser->GetFriendPresenceInfo(ESocialSubsystem::Platform) : nullptr;
		if (PSNPresence && PSNPresence->SessionId.IsValid())
		{
			FindSessionInternal(PSNPresence->SessionId->ToString(), SessionOwnerId, OnAttemptComplete);
			return;
		}
	}

	// Either the read failed or the target user doesn't exist/isn't a friend anymore, so there's no point in trying to find the session again
	ProcessCompletedSessionSearch(LocalUserNum, false, FOnlineSessionSearchResult(), OriginalSessionId, SessionOwnerId, OnAttemptComplete);
}
#endif

void FPartyPlatformSessionManager::HandleFindFriendSessionsComplete(int32 LocalUserNum, bool bWasSuccessful, const TArray<FOnlineSessionSearchResult>& FoundSessions, FSessionId SessionId, FUniqueNetIdRepl SessionOwnerId, FOnFindSessionAttemptComplete OnAttemptComplete)
{
	UE_LOGF(LogParty, Log, "PartyPlatformSessionManager found [%d] sessions searching by friend ID [%ls] (bWasSuccessful=%d)", FoundSessions.Num(), *SessionOwnerId.ToDebugString(), (int32)bWasSuccessful);

	GetSessionInterface()->ClearOnFindFriendSessionCompleteDelegates(LocalUserNum, this);

	//@todo DanH: This assumes the session we're after is always the first one in the array. Quite the assumption...
	const FOnlineSessionSearchResult& SearchResult = bWasSuccessful && FoundSessions.Num() > 0 ? FoundSessions[0] : FOnlineSessionSearchResult();
	ProcessCompletedSessionSearch(LocalUserNum, bWasSuccessful, SearchResult, SessionId, SessionOwnerId, OnAttemptComplete);
}

void FPartyPlatformSessionManager::HandlePartyJoined(USocialParty& NewParty)
{
	for (const TSharedRef<FPartyPlatformSessionMonitor>& Monitor : ActiveMonitors)
	{
		if (Monitor->GetMonitoredPartyTypeId() == NewParty.GetPartyTypeId())
		{
			return;
		}
	}
	CreateMonitor(NewParty);
}

void FPartyPlatformSessionManager::HandleMonitorShutdownComplete(TSharedRef<FPartyPlatformSessionMonitor> Monitor)
{
	ActiveMonitors.Remove(Monitor);

	USocialParty* CurrentParty = SocialManager.GetParty(Monitor->GetMonitoredPartyTypeId());
	if (CurrentParty && !CurrentParty->IsCurrentlyLeaving())
	{
		CreateMonitor(*CurrentParty);
	}
}

//////////////////////////////////////////////////////////////////////////
// FPlatformSessionMonitor
//////////////////////////////////////////////////////////////////////////


const FName FPartyPlatformSessionMonitor::Step_FindSession = TEXT("FindSession");
const FName FPartyPlatformSessionMonitor::Step_JoinSession = TEXT("JoinSesson");
const FName FPartyPlatformSessionMonitor::Step_CreateSession = TEXT("CreateSession");

TSharedRef<FPartyPlatformSessionMonitor> FPartyPlatformSessionMonitor::Create(const TSharedRef<FPartyPlatformSessionManager>& InSessionManager, USocialParty& PartyToMonitor)
{
	TSharedRef<FPartyPlatformSessionMonitor> SessionMonitor = MakeShareable(new FPartyPlatformSessionMonitor(InSessionManager, PartyToMonitor));
	SessionMonitor->Initialize();
	return SessionMonitor;
}

FPartyPlatformSessionMonitor::FPartyPlatformSessionMonitor(const TSharedRef<FPartyPlatformSessionManager>& InSessionManager, USocialParty& PartyToMonitor)
	: SessionManager(InSessionManager)
	, MonitoredParty(&PartyToMonitor)
	, PartyTypeId(PartyToMonitor.GetPartyTypeId())
{
}

void FPartyPlatformSessionMonitor::ShutdownMonitor()
{
	if (ShutdownState == EMonitorShutdownState::None)
	{
		ShutdownState = EMonitorShutdownState::Requested;
		const EOnlineSessionState::Type CurrentState = GetOssSessionState();
		if (CurrentState >= EOnlineSessionState::Pending && CurrentState <= EOnlineSessionState::Ended)
		{
			LeaveSession();
		}
		else if (CurrentState == EOnlineSessionState::NoSession || !ensureMsgf(!RetryTickerHandle.IsValid(), TEXT("We should never be registered for a retry at establishing the session if we aren't in the NoSession state.")))
		{
			ShutdownInternal();
		}
	}
}

FPartyPlatformSessionMonitor::~FPartyPlatformSessionMonitor()
{
	if (ShutdownState != EMonitorShutdownState::Complete)
	{
		UE_LOGF(LogParty, Error, "PartyPlatformSessionMonitor instance is being destructed without properly shutting down. Undesired and inaccurate session membership will result!");

		if (ShutdownState == EMonitorShutdownState::None)
		{
			// Try to leave the session - we won't hear about how it goes
			LeaveSession();
		}
	}
}

const FOnlinePartyTypeId& FPartyPlatformSessionMonitor::GetMonitoredPartyTypeId() const
{
	return PartyTypeId;
}

EOnlineSessionState::Type FPartyPlatformSessionMonitor::GetOssSessionState() const
{
	FNamedOnlineSession* PlatformSession = SessionManager->GetSessionInterface()->GetNamedSession(NAME_PartySession);
	return PlatformSession ? PlatformSession->SessionState : EOnlineSessionState::NoSession;
}

void FPartyPlatformSessionMonitor::EvaluateCurrentSession()
{
	// Bail if we've somehow lost track of the party or are already looking for a session
	if (!ensure(MonitoredParty.IsValid()) || !TargetSessionId.IsEmpty())
	{
		return;
	}

	const IOnlineSessionPtr& SessionInterface = SessionManager->GetSessionInterface();
	if (const FNamedOnlineSession* Session = SessionInterface->GetNamedSession(NAME_PartySession))
	{
		// TODO: We need to check which of the local players are in the session and process them accordingly 
		// We already have a platform session session, so we should be all set. Just check the session ids to make sure we have the correct session.
		UPartyMember& LocalUserMember = MonitoredParty->GetOwningLocalMember();
		const FSessionId& ReplicatedSessionId = LocalUserMember.GetRepData().GetPlatformDataSessionId();
		const FSessionId TrueSessionId = Session->GetSessionIdStr();
		if (ReplicatedSessionId != TrueSessionId && ensure(DoesLocalUserOwnPlatformSession()))
		{
			UE_CLOGF(!ReplicatedSessionId.IsEmpty(), LogParty, Warning, "PartyPlatformSessionMonitor: Local player's session [%ls] does not match replicated session [%ls]", *TrueSessionId, *ReplicatedSessionId);
			LocalUserMember.GetMutableRepData().SetPlatformDataSessionId(TrueSessionId);
		}
	}
	else if (const FPartyPlatformSessionInfo* ExistingSessionInfo = FindLocalPlatformSessionInfo())
	{
		if (!ExistingSessionInfo->SessionId.IsEmpty())
		{
			// Verify that there's actually someone in the party in this session
			// Potentially saves a bit on traffic in edge cases where we're joining just after the former sole session owner has left
			bool bAnyMemberInParty = MonitoredParty->GetPartyMembers().ContainsByPredicate([this, ExistingSessionInfo](const UPartyMember* Member)
			{
				if (Member->GetRepData().GetPlatformDataSessionId() == ExistingSessionInfo->SessionId)
				{
					if (LastAttemptedFindSessionId.IsSet() && LastAttemptedFindSessionId.GetValue() == ExistingSessionInfo->SessionId)
					{
						return false;
					}
					// Someone else is claiming to be in the session already, so go find it now
					return true;
				}
				return false;
			});

			if (bAnyMemberInParty || MonitoredParty->ShouldAlwaysJoinPlatformSession(ExistingSessionInfo->SessionId))
			{
				FindSession(*ExistingSessionInfo);
			}
		}
		else
		{
			if (ShouldAlwaysCreateLocalPlatformSession())
			{
				CreateSession(SessionManager->GetLocalUserPlatformId());
			}
			else
			{
				for (UPartyMember* Member : MonitoredParty->GetPartyMembers())
				{
					if (Member->IsLocalPlayer())
					{
						if (ExistingSessionInfo->IsSessionOwner(*Member))
						{
							// There is no session ID yet, but the session owner is a local player, so it's on us to create it now
							CreateSession(Member->GetRepData().GetPlatformDataUniqueId());
							break;
						}
					}
				}
			}
		}
	}
	else
	{
		// No session yet for this platform at all - that means we're the first user to be on this platform and the leader doesn't know about us yet
		// Wait until the leader has updated the party data to decide on a session owner (since we could be joining along with someone else on this platform at the same time)
	}
}

void FPartyPlatformSessionMonitor::Initialize()
{
	check(MonitoredParty.IsValid());

	USocialParty& Party = *MonitoredParty;
	UE_LOGF(LogParty, Log, "Initializing PartyPlatformSessionMonitor for party [%ls]", *Party.GetPartyId().ToDebugString());

	Party.OnPartyMemberCreated().AddSP(this, &FPartyPlatformSessionMonitor::HandlePartyMemberCreated);
	Party.OnPartyConfigurationChanged().AddSP(this, &FPartyPlatformSessionMonitor::HandlePartyConfigurationChanged);
	Party.GetRepData().OnPlatformSessionsChanged().AddSP(this, &FPartyPlatformSessionMonitor::HandlePlatformSessionsChanged);
	Party.OnPartyLeft().AddSP(this, &FPartyPlatformSessionMonitor::HandlePartyLeft);
	Party.OnPartyMemberLeft().AddSP(this, &FPartyPlatformSessionMonitor::HandlePartyMemberLeft);
	Party.OnPlatformSessionUpdateRequested().AddSP(this, &FPartyPlatformSessionMonitor::QueuePlatformSessionUpdate);

	EvaluateCurrentSession();

#if PARTY_PLATFORM_SESSIONS_XBL
	if (CVarXboxMpaEnabled.GetValueOnAnyThread())
	{
		UpdateRecentPlayersOfLocalMembers(MonitoredParty->GetPartyMembers());
	}
#endif
}

void FPartyPlatformSessionMonitor::ShutdownInternal()
{
	UE_LOGF(LogParty, Log, "Finalizing shutdown of PartyPlatformSessionMonitor for party of type [%d]", PartyTypeId.GetValue());

	if (RetryTickerHandle.IsValid())
	{
		FTSTicker::RemoveTicker(RetryTickerHandle);
		RetryTickerHandle.Reset();
	}

	ShutdownState = EMonitorShutdownState::Complete;
	OnShutdownComplete.ExecuteIfBound(AsShared());
}

void FPartyPlatformSessionMonitor::CreateSession(const FUniqueNetIdRepl& LocalUserPlatformId)
{
	if (ShutdownState != EMonitorShutdownState::None || !ensure(MonitoredParty.IsValid()))
	{
		return;
	}

	FOnlineSessionSettings SessionSettings;
	if (ensure(LocalUserPlatformId.IsValid()) && ensure(ConfigurePlatformSessionSettings(SessionSettings)))
	{
		SessionInitTracker.BeginStep(Step_CreateSession);
		
		const IOnlineSessionPtr& SessionInterface = SessionManager->GetSessionInterface();
		SessionInterface->AddOnCreateSessionCompleteDelegate_Handle(FOnCreateSessionCompleteDelegate::CreateSP(this, &FPartyPlatformSessionMonitor::HandleCreateSessionComplete));
		
#if !UE_BUILD_SHIPPING
		float DelaySeconds = FMath::Max(0.f, PlatformSessionCreationDelay);
		if (DelaySeconds > 0.f || ForcePlatformSessionCreationFailure != 0)
		{
			UE_LOGF(LogParty, Warning, "PartyPlatformSessionMonitor adding artificial delay of %0.2fs to session creation attempt", DelaySeconds);

			TWeakPtr<FPartyPlatformSessionMonitor> AsWeakPtr = SharedThis(this);
			FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda(
				[AsWeakPtr, SessionSettings, LocalUserPlatformId, this] (float)
				{
					QUICK_SCOPE_CYCLE_COUNTER(STAT_FPartyPlatformSessionManager_CreateSessionAttempt);
					if (AsWeakPtr.IsValid())
					{
						if (ForcePlatformSessionCreationFailure != 0)
						{
							UE_LOGF(LogParty, Warning, "Forcing session creation failure");
							HandleCreateSessionComplete(NAME_PartySession, false);
						}
						else
						{
							SessionManager->GetSessionInterface()->CreateSession(*LocalUserPlatformId, NAME_PartySession, SessionSettings);
						}
					}
					return false; // Don't retick
				}), DelaySeconds);
			return;
		}
#endif
		SessionInterface->CreateSession(*LocalUserPlatformId, NAME_PartySession, SessionSettings);
		UE_LOGF(LogParty, Log, "PartyPlatformSessionMonitor creating session with the following parameters: ");
		DumpSessionSettings(&SessionSettings);
	}
}

void FPartyPlatformSessionMonitor::AddLocalPlayerToSession(UPartyMember* PartyMember)
{
	if (ensure(MonitoredParty.IsValid()) &&
		!MonitoredParty->IsMissingPlatformSession())
	{
		const IOnlineSessionPtr& SessionInterface = SessionManager->GetSessionInterface();
		const FNamedOnlineSession* Session = SessionInterface->GetNamedSession(NAME_PartySession);
		if (ensure(Session && Session->SessionInfo.IsValid()))
		{
			const FSessionId SessionId = Session->GetSessionIdStr();

			// TODO: Move SetPlatformSessionId to after AddLocalPlayerToSession completes
			PartyMember->GetMutableRepData().SetPlatformDataSessionId(SessionId);

			const FUniqueNetIdRepl PartyMemberPlatformUniqueId = PartyMember->GetRepData().GetPlatformDataUniqueId();
			if (PartyMemberPlatformUniqueId.IsValid())
			{
				UE_LOGF(LogParty, Verbose, "AddLocalPlayerToSession: Registering player, PartyMember=%ls PPUID=%ls", *PartyMember->ToDebugString(true), *PartyMemberPlatformUniqueId->ToDebugString());
				SessionInterface->RegisterPlayer(Session->SessionName, *PartyMemberPlatformUniqueId, false);

				SessionInterface->RegisterLocalPlayer(*PartyMemberPlatformUniqueId, Session->SessionName,
					FOnRegisterLocalPlayerCompleteDelegate::CreateLambda([](const FUniqueNetId& UserId, EOnJoinSessionCompleteResult::Type Result)
				{
					UE_LOGF(LogParty, Log, "AddLocalPlayerToSession: Complete User=%ls Result=%ls", *UserId.ToDebugString(), LexToString(Result));
				}));
			}
		}
	}
}

void FPartyPlatformSessionMonitor::RemoveLocalPlayerFromSession(UPartyMember* PartyMember)
{
	if (ensure(MonitoredParty.IsValid()) &&
		!MonitoredParty->IsMissingPlatformSession())
	{
		const IOnlineSessionPtr& SessionInterface = SessionManager->GetSessionInterface();
		const FNamedOnlineSession* Session = SessionInterface->GetNamedSession(NAME_PartySession);

		if (ensure(Session))
		{
			const FUniqueNetIdRepl PartyMemberPlatformUniqueId = PartyMember->GetRepData().GetPlatformDataUniqueId();
			if (PartyMemberPlatformUniqueId.IsValid())
			{
				UE_LOGF(LogParty, Verbose, "RemoveLocalPlayerFromSession: Unregistering player, PartyMember=%ls PPUID=%ls", *PartyMember->ToDebugString(true), *PartyMemberPlatformUniqueId->ToDebugString());
				SessionInterface->UnregisterPlayer(Session->SessionName, *PartyMemberPlatformUniqueId);

				SessionInterface->UnregisterLocalPlayer(*PartyMemberPlatformUniqueId, Session->SessionName,
					FOnUnregisterLocalPlayerCompleteDelegate::CreateLambda([](const FUniqueNetId& UserId, const bool bWasSuccessful)
				{
					UE_LOGF(LogParty, Log, "RemoveLocalPlayerFromSession: Complete User=%ls bWasSuccessful=%ls", *UserId.ToDebugString(), *LexToString(bWasSuccessful));
				}));
			}
		}
	}
}

void FPartyPlatformSessionMonitor::FindSession(const FPartyPlatformSessionInfo& SessionInfo)
{
	if (ShutdownState == EMonitorShutdownState::None)
	{
		check(TargetSessionId.IsEmpty());

		SessionInitTracker.BeginStep(Step_FindSession);
		TargetSessionId = SessionInfo.SessionId;

		// Don't attempt to find this session again if this find fails.  This is cleared if the find is successful.
		LastAttemptedFindSessionId.Emplace(SessionInfo.SessionId);

		SessionManager->FindSession(SessionInfo, FPartyPlatformSessionManager::FOnFindSessionAttemptComplete::CreateSP(this, &FPartyPlatformSessionMonitor::HandleFindSessionComplete));
	}
}

void FPartyPlatformSessionMonitor::JoinSession(const FOnlineSessionSearchResult& SessionSearchResult)
{
	UE_LOGF(LogParty, Log, "PartyPlatformSessionMonitor joining platform session [%ls]", *SessionSearchResult.GetSessionIdStr());

	const IOnlineSessionPtr& SessionInterface = SessionManager->GetSessionInterface();
	const FUniqueNetIdRepl LocalUserPlatformId = SessionManager->GetLocalUserPlatformId();
	check(LocalUserPlatformId.IsValid());
	
	SessionInitTracker.BeginStep(Step_JoinSession);
	SessionInterface->AddOnJoinSessionCompleteDelegate_Handle(FOnJoinSessionCompleteDelegate::CreateSP(this, &FPartyPlatformSessionMonitor::HandleJoinSessionComplete));

	FOnlineSessionSearchResult SearchResultCopy = SessionSearchResult;
#if PARTY_PLATFORM_SESSIONS_XBL
	// Set session to be dedicated as we are not using peer to peer features
	SearchResultCopy.Session.SessionSettings.bIsDedicated = true;
#endif
		
#if !UE_BUILD_SHIPPING
	float DelaySeconds = FMath::Max(0.f, PlatformSessionJoinDelay);
	if (DelaySeconds > 0.f || ForcePlatformSessionJoinFailure != 0)
	{
		UE_LOGF(LogParty, Warning, "Adding artificial delay of %0.2fs to session join attempt", DelaySeconds);

		TWeakPtr<FPartyPlatformSessionMonitor> AsWeakPtr = SharedThis(this);
		FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda(
			[AsWeakPtr, SearchResultCopy, LocalUserPlatformId, this] (float)
			{
				QUICK_SCOPE_CYCLE_COUNTER(STAT_FPartyPlatformSessionManager_JoinSessionAttempt);
				if (AsWeakPtr.IsValid())
				{
					if (ForcePlatformSessionCreationFailure != 0)
					{
						UE_LOGF(LogParty, Warning, "Forcing session join failure");
						HandleJoinSessionComplete(NAME_PartySession, EOnJoinSessionCompleteResult::UnknownError);
					}
					else
					{
						SessionManager->GetSessionInterface()->JoinSession(*LocalUserPlatformId, NAME_PartySession, SearchResultCopy);
					}
				}
				return false; // Don't retick
			}), DelaySeconds);
		return;
	}
#endif
		
	if (!SessionInterface->JoinSession(*LocalUserPlatformId, NAME_PartySession, SearchResultCopy))
	{
		UE_LOGF(LogParty, Warning, "JoinSession call failed for session [%ls].", *SessionSearchResult.GetSessionIdStr());
		TargetSessionId = FSessionId();

		if (ensure(MonitoredParty.IsValid()))
		{
			MonitoredParty->SetIsMissingPlatformSession(true);
		}
	}
}

void FPartyPlatformSessionMonitor::LeaveSession()
{
	UE_LOGF(LogParty, Log, "PartyPlatformSessionMonitor destroying platform party session now.");
	ShutdownState = EMonitorShutdownState::InProgress;

	const IOnlineSessionPtr& SessionInterface = SessionManager->GetSessionInterface();

	SessionInterface->ClearOnSessionFailureDelegates(this);
	SessionInterface->DestroySession(NAME_PartySession, FOnDestroySessionCompleteDelegate::CreateSP(this, &FPartyPlatformSessionMonitor::HandleDestroySessionComplete));
}

void FPartyPlatformSessionMonitor::QueuePlatformSessionUpdate()
{
	if (DoesLocalUserOwnPlatformSession() && !bHasQueuedSessionUpdate && ShutdownState == EMonitorShutdownState::None)
	{
		UE_LOGF(LogParty, Verbose, "PartyPlatformSessionMonitor queuing session update for party [%ls]", *MonitoredParty->ToDebugString())
		bHasQueuedSessionUpdate = true;
		FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateSP(this, &FPartyPlatformSessionMonitor::HandleQueuedSessionUpdate));
	}
}

const FPartyPlatformSessionInfo* FPartyPlatformSessionMonitor::FindLocalPlatformSessionInfo() const
{
	if (MonitoredParty.IsValid())
	{
		TOptional<FString> SessionType = FPartyPlatformSessionManager::GetOssPartySessionType(USocialManager::GetSocialOssName(ESocialSubsystem::Platform));
		if (SessionType)
		{
			for (const FPartyPlatformSessionInfo& SessionInfo : MonitoredParty->GetRepData().GetPlatformSessions())
			{
				if (SessionType.GetValue() == SessionInfo.SessionType)
				{
					return &SessionInfo;
				}
			}
		}
	}
	return nullptr;
}

bool FPartyPlatformSessionMonitor::DoesLocalUserOwnPlatformSession()
{
	if (MonitoredParty.IsValid())
	{
		if (IsTencentPlatform())
		{
			// Tencent platform sessions are all locally managed - everyone is responsible for updating their local version of it
			return true;
		}
		else
		{
			for (UPartyMember* Member : MonitoredParty->GetPartyMembers())
			{
				if (Member->IsLocalPlayer())
				{
					for (const FPartyPlatformSessionInfo& SessionInfo : MonitoredParty->GetRepData().GetPlatformSessions())
					{
						if (SessionInfo.IsSessionOwner(*Member))
						{
							return true;
						}
					}
				}
			}
		}
	}
	
	return false;
}

void FPartyPlatformSessionMonitor::HandlePlatformSessionsChanged()
{
	const bool bWasSessionOwner = DoesLocalUserOwnPlatformSession();
	
	EvaluateCurrentSession();

	if (!bWasSessionOwner && DoesLocalUserOwnPlatformSession())
	{
		UE_LOGF(LogParty, Log, "Local user just became owner of their party platform session within party [%ls]", *MonitoredParty->ToDebugString());

		// We just took over ownership of the session on this platform
		QueuePlatformSessionUpdate();
	}
}

void FPartyPlatformSessionMonitor::HandlePartyConfigurationChanged(const FPartyConfiguration& NewConfig)
{
	QueuePlatformSessionUpdate();
}

void FPartyPlatformSessionMonitor::HandlePartyLeft(EMemberExitedReason Reason)
{
	// When the user leaves the monitored party, shut down and leave the session
	if (ensure(MonitoredParty.IsValid()))
	{
		UE_LOGF(LogParty, Log, "Party [%ls] left - shutting down PartyPlatformSessionMonitor", *MonitoredParty->GetPartyId().ToDebugString());
		MonitoredParty.Reset();
	}

	ShutdownMonitor();
}

void FPartyPlatformSessionMonitor::HandlePartyMemberCreated(UPartyMember& NewMember)
{
	if (NewMember.IsInitialized())
	{
		HandlePartyMemberInitialized(&NewMember);
	}
	else
	{
		NewMember.OnInitializationComplete().AddSP(this, &FPartyPlatformSessionMonitor::HandlePartyMemberInitialized, &NewMember);
	}
}

bool FPartyPlatformSessionMonitor::ShouldAlwaysCreateLocalPlatformSession() const
{
#if PARTY_PLATFORM_SESSIONS_XBL
	if (CVarXboxMpaEnabled.GetValueOnAnyThread())
	{
		return true;
	}
#endif

	return false;
}

bool FPartyPlatformSessionMonitor::ShouldRecordAsRecentPlayer(const FUniqueNetId& LocalUserId, const UPartyMember* PartyMember)
{
	if (PartyMember->GetPlatformOssName() != LocalUserId.GetType())
	{
		return false;
	}

	const FUniqueNetIdRepl PartyMemberPlatformUniqueId = PartyMember->GetRepData().GetPlatformDataUniqueId();
	if (!PartyMemberPlatformUniqueId.IsValid())
	{
		return false;
	}

	if (*PartyMember->GetRepData().GetPlatformDataUniqueId() == LocalUserId)
	{
		return false;
	}

	return true;
}

void FPartyPlatformSessionMonitor::UpdateRecentPlayersOfLocalMembers(const TArray<UPartyMember*>& RecentPlayers)
{
	for (UPartyMember* Member : MonitoredParty->GetPartyMembers())
	{
		if (Member->IsLocalPlayer())
		{
			const FUniqueNetIdRepl PartyMemberPlatformUniqueId = Member->GetRepData().GetPlatformDataUniqueId();
			if (PartyMemberPlatformUniqueId.IsValid())
			{
				UpdateRecentPlayersOfLocalUser(*PartyMemberPlatformUniqueId, RecentPlayers);
			}
		}
	}
}

void FPartyPlatformSessionMonitor::UpdateRecentPlayersOfLocalUser(const FUniqueNetId& LocalUserId, const TArray<UPartyMember*>& Members)
{
	TArray<FReportPlayedWithUser> RecentPlayers;
	for (UPartyMember* Member : Members)
	{
		if (ShouldRecordAsRecentPlayer(LocalUserId, Member))
		{
			RecentPlayers.Emplace(Member->GetRepData().GetPlatformDataUniqueId().GetUniqueNetId().ToSharedRef(), "", ERecentPlayerEncounterType::Teammate);
		}
	}

	if (!RecentPlayers.IsEmpty())
	{
		SessionManager->GetFriendsInterface()->AddRecentPlayers(LocalUserId, RecentPlayers, TEXT(""), FOnAddRecentPlayersComplete());
	}
}

void FPartyPlatformSessionMonitor::HandlePartyMemberInitialized(UPartyMember* InitializedMember)
{
	if (IsTencentPlatform() && InitializedMember->GetPlatformOssName() == TENCENT_SUBSYSTEM)
	{
		SessionManager->GetSessionInterface()->RegisterPlayer(NAME_PartySession, *InitializedMember->GetRepData().GetPlatformDataUniqueId(), false);
	}

	// If a local player joined the party (split screen) we add them to the platform session
	if (InitializedMember->IsLocalPlayer())
	{
		AddLocalPlayerToSession(InitializedMember);
	}

#if PARTY_PLATFORM_SESSIONS_XBL
	if (CVarXboxMpaEnabled.GetValueOnAnyThread())
	{
		TArray<UPartyMember*> RecentPlayers{ InitializedMember };
		UpdateRecentPlayersOfLocalMembers(RecentPlayers);

		if (InitializedMember->IsLocalPlayer())
		{
			const FUniqueNetIdRepl PartyMemberPlatformUniqueId = InitializedMember->GetRepData().GetPlatformDataUniqueId();
			if (PartyMemberPlatformUniqueId.IsValid())
			{
				UpdateRecentPlayersOfLocalUser(*PartyMemberPlatformUniqueId, MonitoredParty->GetPartyMembers());
			}
		}	
	}
#endif
}

void FPartyPlatformSessionMonitor::HandlePartyMemberLeft(UPartyMember* OldMember, const EMemberExitedReason ExitReason)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FPartyPlatformSessionMonitor_HandlePartyMemberLeft);
	const FUniqueNetIdRepl PartyMemberPlatformUniqueId = OldMember->GetRepData().GetPlatformDataUniqueId();

	if (PartyMemberPlatformUniqueId.IsValid())
	{
		UE_LOGF(LogParty, Verbose, "HandlePartyMemberLeft: PartyMember=%ls User=%ls ExitReason=%ls",
			*OldMember->ToDebugString(true), *PartyMemberPlatformUniqueId.ToDebugString(), ToString(ExitReason));
	}
	else
	{
		UE_LOGF(LogParty, Warning, "HandlePartyMemberLeft: PartyMember=Unknown ExitReason=%ls", ToString(ExitReason));
	}

	if (IsTencentPlatform() && OldMember->GetPlatformOssName() == TENCENT_SUBSYSTEM)
	{
		SessionManager->GetSessionInterface()->UnregisterPlayer(NAME_PartySession, *OldMember->GetRepData().GetPlatformDataUniqueId());
	}

	if (OldMember->IsLocalPlayer())
	{
		RemoveLocalPlayerFromSession(OldMember);
	}
}

void FPartyPlatformSessionMonitor::HandleCreateSessionComplete(FName SessionName, bool bWasSuccessful)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FPartyPlatformSessionMonitor_HandleCreateSessionComplete);
	UE_LOGF(LogParty, Log, "PartyPlatformSessionMonitor created platform session SessionName=[%ls], bWasSuccessful=[%d]", *SessionName.ToString(), bWasSuccessful);

	const IOnlineSessionPtr& SessionInterface = SessionManager->GetSessionInterface();
	SessionInterface->ClearOnCreateSessionCompleteDelegates(this);

	if (bWasSuccessful)
	{
		SessionInitTracker.CompleteStep(Step_CreateSession);

		SessionInterface->AddOnSessionFailureDelegate_Handle(FOnSessionFailureDelegate::CreateSP(this, &FPartyPlatformSessionMonitor::HandleSessionFailure));

		if (MonitoredParty.IsValid())
		{
			MonitoredParty->SetIsMissingPlatformSession(false);

			// For all local players
			for (UPartyMember* PartyMember : MonitoredParty->GetPartyMembers())
			{
				if (PartyMember->IsLocalPlayer())
				{
					AddLocalPlayerToSession(PartyMember);
				}
			}

			bool bQueuePlatformSessionUpdate = bMissedSessionUpdateDuringCreate || PARTY_PLATFORM_SESSIONS_PSN;
			if (bQueuePlatformSessionUpdate)
			{
				bMissedSessionUpdateDuringCreate = false;
				QueuePlatformSessionUpdate();
			}
		}
		
		OnSessionEstablished.ExecuteIfBound();

		if (ShutdownState == EMonitorShutdownState::Requested)
		{
			// Leave the session we just created
			LeaveSession();
		}
	}
	else if (ShutdownState == EMonitorShutdownState::Requested)
	{
		// If we're supposed to leave, it doesn't matter if we failed to create, so just announce that we "left" and be done with it
		ShutdownInternal();
	}
	else
	{
		if (ensure(MonitoredParty.IsValid()))
		{
			MonitoredParty->SetIsMissingPlatformSession(true);
		}
		if (AllowCreateSessionFailure == 0 && ensure(!RetryTickerHandle.IsValid()))
		{
			// Unsuccessful and we aren't trying to leave, so we'll try again here in a moment
			RetryTickerHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateSP(this, &FPartyPlatformSessionMonitor::HandleRetryEstablishingSession), EstablishSessionRetryDelay);
		}
	}
}

void FPartyPlatformSessionMonitor::HandleFindSessionComplete(bool bWasSuccessful, const FOnlineSessionSearchResult& FoundSession)
{
	check(!TargetSessionId.IsEmpty());

	//@todo DanH Sessions: Not necessarily complete time-wise here - decide if we want this to be additive or what. Like are we cool with completing and starting it again? And if so, how to we accumulate all the time? Ignore the new start time and wipe out the old completion time? #suggested
	SessionInitTracker.CompleteStep(Step_FindSession);

	if (ShutdownState == EMonitorShutdownState::Requested)
	{
		// Doesn't matter if we found the session successfully or not, we're shutting down
		ShutdownInternal();
	}
	else if (bWasSuccessful)
	{
		LastAttemptedFindSessionId.Reset();
		JoinSession(FoundSession);
	}
	else
	{
		UE_LOGF(LogParty, Log, "PartyPlatformSessionMonitor failed to find platform session [%ls]", *TargetSessionId);
		ProcessJoinFailure();
	}
}

void FPartyPlatformSessionMonitor::HandleJoinSessionComplete(FName SessionName, EOnJoinSessionCompleteResult::Type JoinSessionResult)
{
	UE_LOGF(LogParty, Log, "PartyPlatformSessionMonitor attempt to join session [%ls] completed with result [%ls]", *TargetSessionId, LexToString(JoinSessionResult));

	const IOnlineSessionPtr& SessionInterface = SessionManager->GetSessionInterface();

	SessionInitTracker.CompleteStep(Step_JoinSession);
	SessionInterface->ClearOnJoinSessionCompleteDelegates(this);

	const bool bWasSuccessful = JoinSessionResult == EOnJoinSessionCompleteResult::Success || JoinSessionResult == EOnJoinSessionCompleteResult::AlreadyInSession;
	if (ShutdownState == EMonitorShutdownState::Requested)
	{
		TargetSessionId = FSessionId();
		if (bWasSuccessful)
		{
			LeaveSession();
		}
		else
		{
			ShutdownInternal();
		}
	}
	else if (bWasSuccessful && ensure(MonitoredParty.IsValid()))
	{
		TargetSessionId = FSessionId();
		MonitoredParty->SetIsMissingPlatformSession(false);
		SessionInterface->AddOnSessionFailureDelegate_Handle(FOnSessionFailureDelegate::CreateSP(this, &FPartyPlatformSessionMonitor::HandleSessionFailure));

		// For all local players
		for (UPartyMember* PartyMember : MonitoredParty->GetPartyMembers())
		{
			if (PartyMember->IsLocalPlayer())
			{
				AddLocalPlayerToSession(PartyMember);
			}
		}
		if (DoesLocalUserOwnPlatformSession())
		{
			QueuePlatformSessionUpdate();
		}

		// For all players only for Tencent
		if (IsTencentPlatform())
		{
			TArray<FUniqueNetIdRef> MemberIdsOnPlatform;
			for (UPartyMember* PartyMember : MonitoredParty->GetPartyMembers())
			{
				if (PartyMember->GetPlatformOssName() == TENCENT_SUBSYSTEM)
				{
					MemberIdsOnPlatform.Add(PartyMember->GetRepData().GetPlatformDataUniqueId().GetUniqueNetId().ToSharedRef());
				}
			}
			SessionInterface->RegisterPlayers(NAME_PartySession, MemberIdsOnPlatform);
		}
	}
	else
	{
		ProcessJoinFailure();
	}

	if (MonitoredParty.IsValid())
	{
		MonitoredParty->JoinSessionCompleteAnalytics(*TargetSessionId, LexToString(JoinSessionResult));
	}
}

void FPartyPlatformSessionMonitor::HandleDestroySessionComplete(FName SessionName, bool bWasSuccessful)
{
	UE_LOGF(LogParty, Log, "PartyPlatformSessionMonitor finished destroying party session with result [%d].", (int32)bWasSuccessful);

	//@todo DanH Sessions: What does it mean to fail at leaving a session? Does that mean we need to try again? Or just that we weren't in one to begin with? #suggested
	ShutdownInternal();
}

bool FPartyPlatformSessionMonitor::ConfigurePlatformSessionSettings(FOnlineSessionSettings& SessionSettings) const
{
	bool bEstablishedPartySettings = false;

	IOnlinePartyPtr PartyInterface = MonitoredParty.IsValid() ? Online::GetPartyInterface(MonitoredParty->GetWorld()) : nullptr;
	if (PartyInterface.IsValid())
	{
#if PARTY_PLATFORM_SESSIONS_XBL
		if (CVarXboxMpaEnabled.GetValueOnAnyThread())
		{
			if (IOnlinePartyJoinInfoConstPtr JoinInfo = PartyInterface->MakeJoinInfo(*MonitoredParty->GetOwningLocalUserId(), MonitoredParty->GetPartyId()))
			{
				SessionSettings.Set(SETTING_CUSTOM_JOIN_INFO, PartyInterface->MakeTokenFromJoinInfo(*JoinInfo), EOnlineDataAdvertisementType::ViaOnlineService);
				bEstablishedPartySettings = true;
			}
		}
		else
#endif
		{
			const FString JoinInfoJson = PartyInterface->MakeJoinInfoJson(*MonitoredParty->GetOwningLocalUserId(), MonitoredParty->GetPartyId());
			if (ensure(!JoinInfoJson.IsEmpty()))
			{
				bEstablishedPartySettings = true;

#if PARTY_PLATFORM_SESSIONS_PSN
				SessionSettings.Set(SETTING_HOST_MIGRATION, true, EOnlineDataAdvertisementType::DontAdvertise);
				SessionSettings.Set(SETTING_CUSTOM, JoinInfoJson, EOnlineDataAdvertisementType::ViaOnlineService);
#elif PARTY_PLATFORM_SESSIONS_XBL
				// This needs to match our value on the XDP service configuration
				SessionSettings.Set(SETTING_SESSION_TEMPLATE_NAME, FString(TEXT("MultiplayerGameSession")), EOnlineDataAdvertisementType::DontAdvertise);

				// XBOX has their own value for this as SETTING_CUSTOM is hard-coded to constant data in the OSS, and is the actual originator of
				// SETTING_CUSTOM.  Everyone else co-opted it and made it dynamic, so we need to use something else just here so other OSS' still
				// work out of the box for this functionality
				// Encode our JoinInfo into Base64 to prevent XboxLive from parsing our json
				SessionSettings.Set(SETTING_CUSTOM_JOIN_INFO, FBase64::Encode(JoinInfoJson), EOnlineDataAdvertisementType::ViaOnlineService);
#elif PLATFORM_DESKTOP
				// PC (Tencent)
				SessionSettings.Set(SETTING_CUSTOM, JoinInfoJson, EOnlineDataAdvertisementType::ViaOnlineService);
#endif
			}
		}
	}

	if (bEstablishedPartySettings)
	{
		int32 NumMembersInSession = 0;
		int32 NumMembersOnPlatform = 0;

		const FString PlatformName = IOnlineSubsystem::GetLocalPlatformName();
		for (const UPartyMember* PartyMember : MonitoredParty->GetPartyMembers())
		{
			const FPartyMemberRepData& MemberData = PartyMember->GetRepData();
			if (MemberData.GetPlatformDataPlatform() == PlatformName)
			{
				// Even if they end up joining a different session than ours, keep our session open so they could join ours if they have issues with the session they are in
				++NumMembersOnPlatform;
				if (!MemberData.GetPlatformDataSessionId().IsEmpty())
				{
					++NumMembersInSession;
				}
			}
			else if (!MemberData.GetPlatformDataPlatform().IsValid())
			{
				// We don't yet know what platform this player is on, so assume that they are the local platform to keep session open.
				++NumMembersOnPlatform;
			}
		}

		const bool bAreAllMembersInLocalPlatformSession = (NumMembersInSession == NumMembersOnPlatform);
		const FPartyJoinDenialReason PublicJoinDenialReason = MonitoredParty->GetPublicJoinability();

		const EPartyType PartyType = (PublicJoinDenialReason.HasAnyReason() && bAreAllMembersInLocalPlatformSession) ? EPartyType::Private : MonitoredParty->GetRepData().GetPrivacySettings().PartyType;
		switch (PartyType)
		{
		case EPartyType::Private:
#if PARTY_PLATFORM_SESSIONS_XBL
			// Xbox needs this false for privacy of session on dashboard
			SessionSettings.bUsesPresence = false;
#else
			SessionSettings.bUsesPresence = true;
#endif
			SessionSettings.NumPublicConnections = 0;
			SessionSettings.NumPrivateConnections = MonitoredParty->GetPartyMaxSize();
			SessionSettings.bShouldAdvertise = false;
			SessionSettings.bAllowJoinViaPresence = false;
			break;

		case EPartyType::FriendsOnly:
			SessionSettings.bUsesPresence = true;
			SessionSettings.NumPublicConnections = 0;
			SessionSettings.NumPrivateConnections = MonitoredParty->GetPartyMaxSize();
#if PARTY_PLATFORM_SESSIONS_PSN
			SessionSettings.bShouldAdvertise = false;
#else
			SessionSettings.bShouldAdvertise = true;
#endif
			SessionSettings.bAllowJoinViaPresence = true;
			break;

		case EPartyType::Public:
			SessionSettings.bUsesPresence = true;
			SessionSettings.NumPublicConnections = MonitoredParty->GetPartyMaxSize();
			SessionSettings.NumPrivateConnections = 0;
			SessionSettings.bShouldAdvertise = true;
			SessionSettings.bAllowJoinViaPresence = true;
			break;
		}

		const bool bIsAcceptingMembers = !PublicJoinDenialReason.HasAnyReason() || PublicJoinDenialReason.GetReason() == EPartyJoinDenialReason::PartyPrivate;
		SessionSettings.bAllowInvites = bIsAcceptingMembers;
		SessionSettings.bAllowJoinInProgress = bIsAcceptingMembers;

		MonitoredParty->AugmentPlatformSessionSettings(SessionSettings);

		return true;
	}

	return false;
}

bool FPartyPlatformSessionMonitor::HandleRetryEstablishingSession(float)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FPartyPlatformSessionMonitor_HandleRetryEstablishingSession);

	RetryTickerHandle.Reset();
	
	// Do a full re-evaluation of our target session, since things may have changed substantially since the last attempt
	EvaluateCurrentSession();
	return false;
}

void FPartyPlatformSessionMonitor::ProcessJoinFailure()
{
	const FPartyPlatformSessionInfo* ExistingSessionInfo = FindLocalPlatformSessionInfo();
	if (ExistingSessionInfo && ExistingSessionInfo->SessionId != TargetSessionId)
	{
		UE_LOGF(LogParty, Log, "PartyPlatformSessionMonitor targeted platform session [%ls] out of date - retrying with updated id [%ls]", *TargetSessionId, *ExistingSessionInfo->SessionId);
		TargetSessionId = FSessionId();

		// The ID of our platform session changed during the find attempt, so try again right away
		FindSession(*ExistingSessionInfo);
	}
	else
	{
		TargetSessionId = FSessionId();
		RetryTickerHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateSP(this, &FPartyPlatformSessionMonitor::HandleRetryEstablishingSession), EstablishSessionRetryDelay);
	}
}

bool FPartyPlatformSessionMonitor::HandleQueuedSessionUpdate(float)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FPartyPlatformSessionMonitor_HandleQueuedSessionUpdate);
	bHasQueuedSessionUpdate = false;

	if (ShutdownState == EMonitorShutdownState::None && DoesLocalUserOwnPlatformSession())
	{
		const IOnlineSessionPtr& SessionInterface = SessionManager->GetSessionInterface();

		// Make sure the party session is in a fully created state and is not destroying
		FNamedOnlineSession* PlatformSession = SessionInterface->GetNamedSession(NAME_PartySession);
		if (PlatformSession)
		{
			if (PlatformSession->SessionState >= EOnlineSessionState::Pending && PlatformSession->SessionState <= EOnlineSessionState::Ended)
			{
				if (ConfigurePlatformSessionSettings(PlatformSession->SessionSettings))
				{
					if (!SessionInterface->UpdateSession(NAME_PartySession, PlatformSession->SessionSettings, true))
					{
						UE_LOGF(LogParty, Warning, "PartyPlatformSessionMonitor call to UpdateSession failed");
					}
				}
			}
			else if (PlatformSession->SessionState == EOnlineSessionState::Creating)
			{
				// Try again when the session is finished creating
				bMissedSessionUpdateDuringCreate = true;
			}
		}
	}

	// Only fire once - never retick
	return false;
}

void FPartyPlatformSessionMonitor::HandleSessionFailure(const FUniqueNetId& LocalUserId, ESessionFailure::Type FailureType)
{
	UE_LOGF(LogParty, Warning, "PartyPlatformSessionMonitor HandleSessionFailure LocalUserId=%ls FailureType=%ls", *LocalUserId.ToDebugString(), LexToString(FailureType));

	if (ensure(MonitoredParty.IsValid()))
	{
		MonitoredParty->SetIsMissingPlatformSession(true);

		RetryTickerHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateSP(this, &FPartyPlatformSessionMonitor::HandleRetryEstablishingSession), EstablishSessionRetryDelay);
	}
}
