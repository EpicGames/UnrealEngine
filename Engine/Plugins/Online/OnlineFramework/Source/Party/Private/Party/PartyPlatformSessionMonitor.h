// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Party/PartyTypes.h"
#include "OnlineSubsystem.h"
#include "Containers/Ticker.h"
#include "Interfaces/OnlineSessionInterface.h"
#include "SocialTypes.h"

struct FUniqueNetIdRepl;

class USocialManager;
class USocialUser;
class FOnlineSessionSearchResult;
class FOnlineSessionSettings;
class FPartyPlatformSessionMonitor;

struct FPartyRepData;
struct FPartyConfiguration;
enum class EMemberExitedReason : uint8;

class FPartyPlatformSessionManager : public TSharedFromThis<FPartyPlatformSessionManager>
{
public:
	static bool DoesOssNeedPartySession(FName OssName);
	static TOptional<FString> GetOssPartySessionType(FName OssName);
	static TSharedRef<FPartyPlatformSessionManager> Create(USocialManager& InSocialManager);

	DECLARE_DELEGATE_TwoParams(FOnFindSessionAttemptComplete, bool, const FOnlineSessionSearchResult&);
	bool FindSession(const USocialUser& User, const FOnFindSessionAttemptComplete& OnAttemptComplete);
	bool FindSession(const FPartyPlatformSessionInfo& SessionInfo, const FOnFindSessionAttemptComplete& OnAttemptComplete);

	IOnlineSessionPtr GetSessionInterface();
	IOnlineFriendsPtr GetFriendsInterface();
	FUniqueNetIdRepl GetLocalUserPlatformId() const;

private:
	FPartyPlatformSessionManager(USocialManager& InSocialManager);
	void InitSessionManager();
	void CreateMonitor(USocialParty& Party);

	bool FindSessionInternal(const FSessionId& SessionId, const FUniqueNetIdRepl& SessionOwnerId, const FOnFindSessionAttemptComplete& OnAttemptComplete);
	void ProcessCompletedSessionSearch(int32 LocalUserNum, bool bWasSuccessful, const FOnlineSessionSearchResult& FoundSession, const FSessionId& SessionId, const FUniqueNetIdRepl& SessionOwnerId, const FOnFindSessionAttemptComplete& OnAttemptComplete);

	void HandleFindSessionByIdComplete(int32 LocalUserNum, bool bWasSuccessful, const FOnlineSessionSearchResult& FoundSession, FSessionId SessionId, FUniqueNetIdRepl SessionOwnerId, FOnFindSessionAttemptComplete OnAttemptComplete);
	void HandleFindFriendSessionsComplete(int32 LocalUserNum, bool bWasSuccessful, const TArray<FOnlineSessionSearchResult>& FoundSessions, FSessionId SessionId, FUniqueNetIdRepl SessionOwnerId, FOnFindSessionAttemptComplete OnAttemptComplete);
	
	void HandlePartyJoined(USocialParty& NewParty);
	void HandleMonitorShutdownComplete(TSharedRef<FPartyPlatformSessionMonitor> Monitor);

#if PARTY_PLATFORM_SESSIONS_PSN
	bool bHasAlreadyRequeriedPSNFriends = false;
	void HandleReadPSNFriendsListComplete(int32 LocalUserNum, bool bWasSuccessful, const FString& ListName, const FString& ErrorStr, FSessionId OriginalSessionId, FUniqueNetIdRepl SessionOwnerId, FOnFindSessionAttemptComplete OnAttemptComplete);
#endif

	USocialManager& SocialManager;
	const FName PlatformOssName;
	TArray<TSharedRef<FPartyPlatformSessionMonitor>> ActiveMonitors;
};

/** Util class to maintain ideal platform session membership during the lifetime of the owning party */
class FPartyPlatformSessionMonitor : public TSharedFromThis<FPartyPlatformSessionMonitor>
{
public:
	static TSharedRef<FPartyPlatformSessionMonitor> Create(const TSharedRef<FPartyPlatformSessionManager>& InSessionManager, USocialParty& PartyToMonitor);
	
	void ShutdownMonitor();
	~FPartyPlatformSessionMonitor();

	const FOnlinePartyTypeId& GetMonitoredPartyTypeId() const;
	EOnlineSessionState::Type GetOssSessionState() const;
	FSimpleDelegate OnSessionEstablished;

	DECLARE_DELEGATE_OneParam(FOnShutdownComplete, TSharedRef<FPartyPlatformSessionMonitor>);
	FOnShutdownComplete OnShutdownComplete;

private:
	FPartyPlatformSessionMonitor(const TSharedRef<FPartyPlatformSessionManager>& InSessionManager, USocialParty& PartyToMonitor);
	void Initialize();
	void ShutdownInternal();

	void CreateSession(const FUniqueNetIdRepl& LocalUserPlatformId);
	void AddLocalPlayerToSession(UPartyMember* InitializedMember);
	void RemoveLocalPlayerFromSession(UPartyMember* PartyMember);
	void FindSession(const FPartyPlatformSessionInfo& SessionInfo);
	void JoinSession(const FOnlineSessionSearchResult& SessionSearchResult);
	void LeaveSession();

	void EvaluateCurrentSession();
	void QueuePlatformSessionUpdate();
	bool ConfigurePlatformSessionSettings(FOnlineSessionSettings& SessionSettings) const;

	const FPartyPlatformSessionInfo* FindLocalPlatformSessionInfo() const;
	bool DoesLocalUserOwnPlatformSession();
	void SetIsSessionMissing(bool bIsMissing);

	void ProcessJoinFailure();

	bool ShouldAlwaysCreateLocalPlatformSession() const;
	bool ShouldRecordAsRecentPlayer(const FUniqueNetId& LocalUserId, const UPartyMember* PartyMember);
	void UpdateRecentPlayersOfLocalMembers(const TArray<UPartyMember*>& RecentPlayers);
	void UpdateRecentPlayersOfLocalUser(const FUniqueNetId& LocalUserId, const TArray<UPartyMember*>& Members);

private:
	void HandlePlatformSessionsChanged();
	void HandlePartyConfigurationChanged(const FPartyConfiguration& NewConfig);
	void HandlePartyLeft(EMemberExitedReason Reason);
	void HandlePartyMemberCreated(UPartyMember& NewMember);
	void HandlePartyMemberInitialized(UPartyMember* InitializedMember);
	void HandlePartyMemberLeft(UPartyMember* OldMember, const EMemberExitedReason Reason);

	void HandleCreateSessionComplete(const FName SessionName, bool bWasSuccessful);
	void HandleFindSessionComplete(bool bWasSuccessful, const FOnlineSessionSearchResult& FoundSession);
	void HandleJoinSessionComplete(FName SessionName, EOnJoinSessionCompleteResult::Type JoinSessionResult);
	void HandleDestroySessionComplete(FName SessionName, bool bWasSuccessful);
	bool HandleRetryEstablishingSession(float);
	void HandleSessionFailure(const FUniqueNetId& LocalUserId, ESessionFailure::Type FailureType);

	bool HandleQueuedSessionUpdate(float);

private:
	TSharedRef<FPartyPlatformSessionManager> SessionManager;
	TWeakObjectPtr<USocialParty> MonitoredParty;
	FOnlinePartyTypeId PartyTypeId;

	FSessionId TargetSessionId;

	/** Last session id we attempted to find to prevent repeated failures to find the same session */
	TOptional<FSessionId> LastAttemptedFindSessionId;

	/** Do we have a console session update queued? */
	bool bHasQueuedSessionUpdate = false;

	/** Did we miss a session update while we were creating? */
	bool bMissedSessionUpdateDuringCreate = false;

	static const FName Step_FindSession;
	static const FName Step_JoinSession;
	static const FName Step_CreateSession;
	FSocialActionTimeTracker SessionInitTracker;

	enum class EMonitorShutdownState : uint8
	{
		None,
		Requested,
		InProgress,
		Complete
	} ShutdownState = EMonitorShutdownState::None;
	
	FTSTicker::FDelegateHandle RetryTickerHandle;
	FDelegateHandle UpdateOssSessionHandle;
};
