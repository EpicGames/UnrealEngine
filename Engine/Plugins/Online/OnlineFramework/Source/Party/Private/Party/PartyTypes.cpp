// Copyright Epic Games, Inc. All Rights Reserved.

#include "Party/PartyTypes.h"
#include "Party/SocialParty.h"
#include "Party/PartyMember.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PartyTypes)

//////////////////////////////////////////////////////////////////////////
// FPartyPlatformSessionInfo
//////////////////////////////////////////////////////////////////////////

bool FPartyPlatformSessionInfo::operator==(const FPartyPlatformSessionInfo& Other) const
{
	return SessionType == Other.SessionType
		&& SessionId == Other.SessionId
		&& OwnerPrimaryId == Other.OwnerPrimaryId;
}

bool FPartyPlatformSessionInfo::operator==(const FString& InSessionType) const
{
	return SessionType == InSessionType;
}

FString FPartyPlatformSessionInfo::ToDebugString() const
{
	return FString::Printf(TEXT("SessionType=[%s], SessionId=[%s], OwnerPrimaryId=[%s]"), *SessionType, *SessionId, *OwnerPrimaryId.ToDebugString());
}

bool FPartyPlatformSessionInfo::IsSessionOwner(const UPartyMember& PartyMember) const
{
	return PartyMember.GetPrimaryNetId() == OwnerPrimaryId;
}

bool FPartyPlatformSessionInfo::IsInSession(const UPartyMember& PartyMember) const
{
	return PartyMember.GetRepData().GetPlatformDataSessionId() == SessionId;
}

//////////////////////////////////////////////////////////////////////////
// FPartyPrivacySettings
//////////////////////////////////////////////////////////////////////////

bool FPartyPrivacySettings::operator==(const FPartyPrivacySettings& Other) const
{
	return PartyType == Other.PartyType
		&& PartyInviteRestriction == Other.PartyInviteRestriction
		&& bOnlyLeaderFriendsCanJoin == Other.bOnlyLeaderFriendsCanJoin;
}

//////////////////////////////////////////////////////////////////////////
// FJoinPartyResult
//////////////////////////////////////////////////////////////////////////

FJoinPartyResult::FJoinPartyResult()
	: Result(EJoinPartyCompletionResult::Succeeded)
{
}

FJoinPartyResult::FJoinPartyResult(FPartyJoinDenialReason InDenialReason)
{
	SetDenialReason(InDenialReason);
}

FJoinPartyResult::FJoinPartyResult(EJoinPartyCompletionResult InResult)
	: Result(InResult)
{
}

FJoinPartyResult::FJoinPartyResult(EJoinPartyCompletionResult InResult, FPartyJoinDenialReason InDenialReason)
{
	SetResult(InResult);
	if (InResult == EJoinPartyCompletionResult::NotApproved)
	{
		SetDenialReason(InDenialReason);
	}
}

FJoinPartyResult::FJoinPartyResult(EJoinPartyCompletionResult InResult, int32 InResultSubCode)
{
	SetResult(InResult);
	if (InResult == EJoinPartyCompletionResult::NotApproved)
	{
		SetDenialReason(InResultSubCode);
	}
	else
	{
		ResultSubCode = InResultSubCode;
	}
}

void FJoinPartyResult::SetDenialReason(FPartyJoinDenialReason InDenialReason)
{
	DenialReason = InDenialReason;
	if (InDenialReason.HasAnyReason())
	{
		Result = EJoinPartyCompletionResult::NotApproved;
	}
}

void FJoinPartyResult::SetResult(EJoinPartyCompletionResult InResult)
{
	Result = InResult;
	if (InResult != EJoinPartyCompletionResult::NotApproved)
	{
		DenialReason = EPartyJoinDenialReason::NoReason;
	}
}

bool FJoinPartyResult::WasSuccessful() const
{
	return Result == EJoinPartyCompletionResult::Succeeded;
}

//////////////////////////////////////////////////////////////////////////
// FOnlinePartyRepDataBase
//////////////////////////////////////////////////////////////////////////

void FOnlinePartyRepDataBase::LogSetPropertyFailure(const TCHAR* OwningStructTypeName, const TCHAR* PropertyName) const
{
	const USocialParty* OwningParty = GetOwnerParty();
	UE_LOGF(LogParty, Warning, "Failed to modify RepData property [%ls::%ls] in party [%ls] - local member [%ls] does not have authority.",
		OwningStructTypeName,
		PropertyName,
		OwningParty ? *OwningParty->ToDebugString() : TEXT("unknown"),
		OwningParty ? *OwningParty->GetOwningLocalMember().ToDebugString(false) : TEXT("unknown"));
}

void FOnlinePartyRepDataBase::LogPropertyChanged(const TCHAR* OwningStructTypeName, const TCHAR* PropertyName, bool bFromReplication) const
{
	const USocialParty* OwningParty = GetOwnerParty();
	
	// Only thing this lacks is the id of the party member for member rep data changes
	UE_LOGF(LogParty, VeryVerbose, "RepData property [%ls::%ls] changed %ls in party [%ls]",
		OwningStructTypeName,
		PropertyName,
		bFromReplication ? TEXT("remotely") : TEXT("locally"),
		OwningParty ? *OwningParty->ToDebugString() : TEXT("unknown"));
}

const TCHAR* ToString(EPartyJoinDenialReason Type)
{
	switch (Type)
	{
	case EPartyJoinDenialReason::NoReason:
		return TEXT("NoReason");
	case EPartyJoinDenialReason::JoinAttemptAborted:
		return TEXT("JoinAttemptAborted");
	case EPartyJoinDenialReason::Busy:
		return TEXT("Busy");
	case EPartyJoinDenialReason::OssUnavailable:
		return TEXT("OssUnavailable");
	case EPartyJoinDenialReason::PartyFull:
		return TEXT("PartyFull");
	case EPartyJoinDenialReason::GameFull:
		return TEXT("GameFull");
	case EPartyJoinDenialReason::NotPartyLeader:
		return TEXT("NotPartyLeader");
	case EPartyJoinDenialReason::PartyPrivate:
		return TEXT("PartyPrivate");
	case EPartyJoinDenialReason::JoinerCrossplayRestricted:
		return TEXT("JoinerCrossplayRestricted");
	case EPartyJoinDenialReason::MemberCrossplayRestricted:
		return TEXT("MemberCrossplayRestricted");
	case EPartyJoinDenialReason::GameModeRestricted:
		return TEXT("GameModeRestricted");
	case EPartyJoinDenialReason::Banned:
		return TEXT("Banned");
	case EPartyJoinDenialReason::NotLoggedIn:
		return TEXT("NotLoggedIn");
	case EPartyJoinDenialReason::CheckingForRejoin:
		return TEXT("CheckingForRejoin");
	case EPartyJoinDenialReason::TargetUserMissingPresence:
		return TEXT("TargetUserMissingPresence");
	case EPartyJoinDenialReason::TargetUserUnjoinable:
		return TEXT("TargetUserUnjoinable");
	case EPartyJoinDenialReason::TargetUserAway:
		return TEXT("TargetUserAway");
	case EPartyJoinDenialReason::AlreadyLeaderInPlatformSession:
		return TEXT("AlreadyLeaderInPlatformSession");
	case EPartyJoinDenialReason::TargetUserPlayingDifferentGame:
		return TEXT("TargetUserPlayingDifferentGame");
	case EPartyJoinDenialReason::TargetUserMissingPlatformSession:
		return TEXT("TargetUserMissingPlatformSession");
	case EPartyJoinDenialReason::PlatformSessionMissingJoinInfo:
		return TEXT("PlatformSessionMissingJoinInfo");
	case EPartyJoinDenialReason::FailedToStartFindConsoleSession:
		return TEXT("FailedToStartFindConsoleSession");
	case EPartyJoinDenialReason::MissingPartyClassForTypeId:
		return TEXT("MissingPartyClassForTypeId");
	case EPartyJoinDenialReason::TargetUserBlocked:
		return TEXT("TargetUserBlocked");
	case EPartyJoinDenialReason::InvalidJoinInfo:
		return TEXT("InvalidJoinInfo");
	case EPartyJoinDenialReason::NotFriends:
		return TEXT("NotFriends");
	default:
		return TEXT("CustomReason");
	}
}

const TCHAR* LexToString(ESocialPartyInviteFailureReason Type)
{
	switch (Type)
	{
	case ESocialPartyInviteFailureReason::Success: return TEXT("Success");
	case ESocialPartyInviteFailureReason::NotOnline: return TEXT("NotOnline");
	case ESocialPartyInviteFailureReason::NotAcceptingMembers: return TEXT("NotAcceptingMembers");
	case ESocialPartyInviteFailureReason::NotFriends: return TEXT("NotFriends");
	case ESocialPartyInviteFailureReason::AlreadyInParty: return TEXT("AlreadyInParty");
	case ESocialPartyInviteFailureReason::OssValidationFailed: return TEXT("OssValidationFailed");
	case ESocialPartyInviteFailureReason::PlatformInviteFailed: return TEXT("PlatformInviteFailed");
	case ESocialPartyInviteFailureReason::PartyInviteFailed: return TEXT("PartyInviteFailed");
	case ESocialPartyInviteFailureReason::InviteRateLimitExceeded: return TEXT("InviteRateLimitExceeded");
	default:
		checkNoEntry();
		return TEXT("Unknown");
	}
}

const TCHAR* ToString(EPartyType Type)
{
	switch (Type)
	{
	case EPartyType::Public:
	{
		return TEXT("Public");
	}
	case EPartyType::FriendsOnly:
	{
		return TEXT("FriendsOnly");
	}
	case EPartyType::Private:
	{
		return TEXT("Private");
	}
	default:
	{
		return TEXT("Unknown");
	}
	}
}

const TCHAR* ToString(EApprovalAction Type)
{
	switch (Type)
	{
	case EApprovalAction::Approve:
	{
		return TEXT("Approve");
	}
	case EApprovalAction::Enqueue:
	{
		return TEXT("Enqueue");
	}
	case EApprovalAction::EnqueueAndStartBeacon:
	{
		return TEXT("EnqueueAndStartBeacon");
	}
	case EApprovalAction::Deny:
	{
		return TEXT("Deny");
	}
	default:
	{
		return TEXT("Unknown");
	}
	}
}

namespace PartyJoinMethod
{
	// User has created the party
	const FName Creation(TEXT("Creation"));
	// User has joined the party via invitation
	const FName Invitation(TEXT("Invitation"));
	// user has joined after requesting access
	const FName RequestToJoin(TEXT("RequestToJoin"));
	// User has joined the party via presence
	const FName Presence(TEXT("Presence"));
	// User has joined the party using a platform option
	const FName PlatformSession(TEXT("PlatformSession"));
	// User has joined the party via command line
	const FName CommandLineJoin(TEXT("CommandLineJoin"));
	// User has joined via unknown/undocumented process
	const FName Unspecified(TEXT("Unspecified"));
}
