// Copyright Epic Games, Inc. All Rights Reserved.

#include "PartyModule.h"

#include "HAL/IConsoleManager.h"

IMPLEMENT_MODULE(FPartyModule, Party);

DEFINE_LOG_CATEGORY(LogParty);

#if STATS
DEFINE_STAT(STAT_PartyStat1);
#endif

void FPartyModule::StartupModule()
{	
}

void FPartyModule::ShutdownModule()
{
}

/**
* CVar to disable writing information to Online Subsystem Presence
*/
static TAutoConsoleVariable<int32> CVar_OnlineFramework_Party_ShouldWritePresenceToOnlineSubsystem(
	TEXT("onlineframework.party.shouldwritepresencetoonlinesubsystem"),
	1,
	TEXT("Whether relevant systems should write Presence information to OnlineSubsystem"));

/**
* CVar to enable writing information to Online Services Presence
*/
static TAutoConsoleVariable<int32> CVar_OnlineFramework_Party_ShouldWritePresenceToOnlineServices(
	TEXT("onlineframework.party.shouldwritepresencetoonlineservices"),
	0,
	TEXT("Whether relevant systems should write Presence information to OnlineServices"));

/**
* CVar to enable reading information from Online Services Presence
*/
static TAutoConsoleVariable<int32> CVar_OnlineFramework_Party_ShouldReadPresenceFromOnlineServices(
	TEXT("onlineframework.party.shouldreadpresencefromonlineservices"),
	0,
	TEXT("Whether relevant systems should read Presence information from OnlineServices"));

/**
* CVar to enable reading information from Online Services User Info
*/
static TAutoConsoleVariable<int32> CVar_OnlineFramework_Party_ShouldReadUserInfoFromOnlineServices(
	TEXT("onlineframework.party.shouldreaduserinfofromonlineservices"),
	0,
	TEXT("Whether relevant systems should read User Information from OnlineServices"));
