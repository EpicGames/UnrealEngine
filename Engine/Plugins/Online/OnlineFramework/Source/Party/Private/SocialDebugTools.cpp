// Copyright Epic Games, Inc. All Rights Reserved.

#include "SocialDebugTools.h"

#include "Interfaces/OnlineUserInterface.h"
#include "Interfaces/OnlinePresenceInterface.h"
#include "Misc/ScopeExit.h"
#include "Online/Presence.h"
#include "OnlineFramework/CommonConfig.h"
#include "OnlineFramework/CommonAccount.h"
#include "OnlineFramework/CommonAccountUtils.h"
#include "OnlineSubsystemUtils.h"
#include "Party/PartyMember.h"
#include "SocialManager.h"
#include "TimerManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SocialDebugTools)

USocialDebugTools::USocialDebugTools()
	: bAutoAcceptFriendInvites(true)
	, bAutoAcceptPartyInvites(true)
{
	
}

void USocialDebugTools::Shutdown()
{
	// shutdown OSS instances that were created	
	for (auto& Entry : Contexts)
	{
		FInstanceContext& Context = Entry.Value;
		GetContext(Context.Name).Shutdown();
	}
	Contexts.Empty();
}

void USocialDebugTools::PrintExecUsage() const
{
	UE_LOGF(LogParty, Log, "Usage SOCIAL DEBUG CONTEXT=<Name> <COMMAND> <PARAMS>");
}

void USocialDebugTools::PrintExecCommands() const
{
	UE_LOGF(LogParty, Log, "LOGIN <Id> <Auth> <optional CONTEXT=<Name2> <Id2> <Auth2> CONTEXT=<Name3> <Id3> <Auth3> ...>");
	UE_LOGF(LogParty, Log, "LOGOUT");
	UE_LOGF(LogParty, Log, "JOINPARTY <Id> <Auth> <optional FriendName>");
	UE_LOGF(LogParty, Log, "LEAVEPARTY");
	UE_LOGF(LogParty, Log, "AUTOACCEPTFRIENDINVITES");
	UE_LOGF(LogParty, Log, "AUTOACCEPTPARTYINVITES");
	UE_LOGF(LogParty, Log, "JIP");
	UE_LOGF(LogParty, Log, "HELP");
}

#if UE_ALLOW_EXEC_COMMANDS
bool USocialDebugTools::Exec(class UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Out)
{
	if (FParse::Command(&Cmd, TEXT("DEBUG")))
	{
		FString Instance;
		FParse::Value(Cmd, TEXT("CONTEXT="), Instance);
		if (FParse::Command(&Cmd, TEXT("HELP")))
		{
			PrintExecUsage();
			PrintExecCommands();
		}
		else if (Instance.IsEmpty())
		{
			PrintExecUsage();
		}
		else
		{
			bool bAllInstancesRequested = false;
			TArray<FString> TargetInstances;

			// strip out context to parse next entry
			FParse::Command(&Cmd, *FString(TEXT("CONTEXT=") + Instance));

			if(Instance.Equals(TEXT("all")))
			{
				bAllInstancesRequested = true;
				GetContextNames(TargetInstances);
				if(TargetInstances.Num() == 0)
				{
					UE_LOGF(LogParty, Log, "CONTEXT=ALL used, but no OSS contexts found!");
				}
			}
			else
			{
				TargetInstances.Add(Instance);
			}

			return RunCommand(Cmd, TargetInstances);
		}

		return true;
	}
	
	return false;
}
#endif // UE_ALLOW_EXEC_COMMANDS

void USocialDebugTools::Login(const FString& Instance, const FOnlineAccountCredentials& Credentials, const FLoginComplete& OnComplete)
{
	bool bResult = false;
	FInstanceContext& Context = GetContext(Instance);
	IOnlineSubsystem* OnlineSub = Context.GetOSS();
	if (OnlineSub)
	{
		IOnlineIdentityPtr OnlineIdentity = OnlineSub->GetIdentityInterface();
		if (OnlineIdentity.IsValid())
		{
			if (OnlineIdentity->GetLoginStatus(LocalUserNum) == ELoginStatus::LoggedIn)
			{
				bResult = true;
			}
			else
			{
				if (!Context.LoginCompleteDelegateHandle.IsValid())
				{
					auto PresenceDelegateV2 = [this, Instance, OnComplete](const UE::Online::FPresenceUpdated& Event)
					{
						FInstanceContext& ContextTmp = GetContext(Instance);
						OnComplete.ExecuteIfBound(true);

						ContextTmp.OnPresenceUpdatedHandle.Unbind();
					};

					auto PresenceDelegate = FOnPresenceReceivedDelegate::CreateLambda([this, Instance, OnComplete](const class FUniqueNetId& LocalUserId, const TSharedRef<FOnlineUserPresence>& Presence)
					{
						FInstanceContext& ContextTmp = GetContext(Instance);
						OnComplete.ExecuteIfBound(true);

						if (ContextTmp.PresenceReceivedDelegateHandle.IsValid())
						{
							ContextTmp.GetOSS()->GetPresenceInterface()->ClearOnPresenceReceivedDelegate_Handle(ContextTmp.PresenceReceivedDelegateHandle);
							ContextTmp.PresenceReceivedDelegateHandle.Reset();
						}
					});

					auto LoginDelegate = FOnLoginCompleteDelegate::CreateLambda([this, Instance, PresenceDelegate, PresenceDelegateV2, OnComplete](int32 InLocalUserNum, bool bWasSuccessful, const FUniqueNetId& UserId, const FString& Error)
					{
						FInstanceContext& ContextTmp = GetContext(Instance);
						if (bWasSuccessful)
						{		
							if (USocialManager::ShouldReadPresenceFromOnlineServices())
							{
								using namespace UE::Online;
								using namespace UE::OnlineFramework;

								const FName& PresenceFrameworkInstance = USocialManager::GetPresenceFrameworkInstance();

								FCommonConfig CommonConfig(this);

								FAccountId LocalUserAccountId, PresenceUserAccountId;

								if (IOnlineIdentityPtr OnlineIdentity = ContextTmp.GetOSS()->GetIdentityInterface())
								{
									if (FUniqueNetIdPtr LocalUserNetIdPtr = OnlineIdentity->GetUniquePlayerId(InLocalUserNum))
									{
										if (const FCommonAccountPtr LocalUserCommonAccountPtr = GetCommonAccountFromV1(CommonConfig, LocalUserNetIdPtr, GetFirstFrameworkInstanceName(LocalUserNetIdPtr)))
										{
											LocalUserAccountId = LocalUserCommonAccountPtr->GetId(PresenceFrameworkInstance);
										}
									}
								}

								if (const FCommonAccountPtr PresenceUserCommonAccountPtr = GetCommonAccountFromV1(CommonConfig, UserId.AsShared(), GetFirstFrameworkInstanceName(UserId)))
								{
									PresenceUserAccountId = PresenceUserCommonAccountPtr->GetId(PresenceFrameworkInstance);
								}

								if (LocalUserAccountId.IsValid() && PresenceUserAccountId.IsValid())
								{
									if (IOnlineServicesPtr OnlineServices = CommonConfig.GetServices(PresenceFrameworkInstance))
									{
										if (IPresencePtr Presence = OnlineServices->GetPresenceInterface())
										{
											ContextTmp.OnPresenceUpdatedHandle = Presence->OnPresenceUpdated().Add(this, PresenceDelegateV2);

											TSharedRef<FUserPresence> UpdatedPresence = MakeShared<FUserPresence>();
											UpdatedPresence->AccountId = PresenceUserAccountId;
											UpdatedPresence->Status = EUserPresenceStatus::Online;
											UpdatedPresence->StatusString = FString(TEXT("Golem:")) + Instance;

											FUpdatePresence::Params Params = { LocalUserAccountId, UpdatedPresence };
											Presence->UpdatePresence(MoveTemp(Params));
										}
										else
										{
											OnComplete.ExecuteIfBound(false);
										}
									}
									else
									{
										OnComplete.ExecuteIfBound(false);
									}
								}
								else
								{
									OnComplete.ExecuteIfBound(false);
								}								
							}
							else
							{
								IOnlinePresencePtr OnlinePresence = ContextTmp.GetOSS()->GetPresenceInterface();
								if (OnlinePresence.IsValid())
								{
									ContextTmp.PresenceReceivedDelegateHandle = OnlinePresence->AddOnPresenceReceivedDelegate_Handle(PresenceDelegate);

									FOnlinePresenceSetPresenceParameters Status;
									Status.State = EOnlinePresenceState::Online;
									Status.StatusStr = FString(TEXT("Golem:")) + Instance;
									Status.Properties.Emplace(); // Intentionally empty to clear out existing fields
									OnlinePresence->SetPresence(UserId, MoveTemp(Status));
								}
								else
								{
									OnComplete.ExecuteIfBound(false);
								}
							}
						}
						else
						{
							OnComplete.ExecuteIfBound(false);
						}
						
						if (ContextTmp.LoginCompleteDelegateHandle.IsValid())
						{
							ContextTmp.GetOSS()->GetIdentityInterface()->ClearOnLoginCompleteDelegate_Handle(InLocalUserNum, ContextTmp.LoginCompleteDelegateHandle);
							ContextTmp.LoginCompleteDelegateHandle.Reset();
						}
					});

					Context.LoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(LocalUserNum, LoginDelegate);
					OnlineIdentity->Login(LocalUserNum, Credentials);
					return;
				}
			}
		}
	}
	OnComplete.ExecuteIfBound(bResult);
}

void USocialDebugTools::Logout(const FString& Instance, const FLogoutComplete& OnComplete)
{
	bool bResult = false;
	FInstanceContext& Context = GetContext(Instance);
	IOnlineSubsystem* OnlineSub = Context.GetOSS();
	if (OnlineSub)
	{
		IOnlineIdentityPtr OnlineIdentity = OnlineSub->GetIdentityInterface();
		if (OnlineIdentity.IsValid())
		{
			if (OnlineIdentity->GetLoginStatus(LocalUserNum) == ELoginStatus::NotLoggedIn)
			{
				bResult = true;
			}
			else
			{
				if (!Context.LogoutCompleteDelegateHandle.IsValid())
				{
					auto Delegate = FOnLogoutCompleteDelegate::CreateLambda([this, Instance, OnComplete](int32 InLocalUserNum, bool bWasSuccessful)
					{
						OnComplete.ExecuteIfBound(bWasSuccessful);

						FInstanceContext& ContextTmp = GetContext(Instance);
						if (ContextTmp.LogoutCompleteDelegateHandle.IsValid())
						{
							GetContext(Instance).GetOSS()->GetIdentityInterface()->ClearOnLogoutCompleteDelegate_Handle(InLocalUserNum, ContextTmp.LogoutCompleteDelegateHandle);
							ContextTmp.LogoutCompleteDelegateHandle.Reset();
						}
					});

					Context.LogoutCompleteDelegateHandle = OnlineIdentity->AddOnLogoutCompleteDelegate_Handle(LocalUserNum, Delegate);
					OnlineIdentity->Logout(LocalUserNum);
					return;
				}
			}
		}
	}
	OnComplete.ExecuteIfBound(bResult);
}

void USocialDebugTools::JoinInProgress(const FString& Instance, const FJoinInProgressComplete& OnComplete)
{
	// Assume an error, and trigger delegate on scope exit
	TOptional<FJoinInProgressComplete> OnError = OnComplete;
	ON_SCOPE_EXIT
	{
		// Execute error 
		if (OnError)
		{
			// Error reason doesn't matter, but we're very unlikely to receive this from the JIP target
			OnError->ExecuteIfBound(EPartyJoinDenialReason::OssUnavailable);
		}
	};

	FInstanceContext& Context = GetContext(Instance);
	IOnlineSubsystem* OnlineSub = Context.GetOSS();
	if (!OnlineSub)
	{
		return;
	}
	FUniqueNetIdPtr LocalUserId = Context.GetLocalUserId();
	if (!LocalUserId)
	{
		return;
	}

	// Find the party we're in
	IOnlinePartyPtr PartySystem = OnlineSub->GetPartyInterface();
	if (!PartySystem)
	{
		return;
	}
	FOnlinePartyConstPtr Party = PartySystem->GetParty(*LocalUserId, IOnlinePartySystem::GetPrimaryPartyTypeId());
	if (!Party)
	{
		return;
	}

	// JIP on the party leader
	FUniqueNetIdPtr JipTargetId = Party->LeaderId;

	if (!JipTargetId.IsValid())
	{
		return;
	}

	FPartyMemberJoinInProgressRequest Request;
	Request.Target = JipTargetId;
	Request.Time = FDateTime::UtcNow().ToUnixTimestamp();

	if (Context.SetJIPRequest(Request))
	{
		OnError.Reset(); // No longer assuming an error
		
		// Wait for player to respond 
		TSharedRef<FDelegateHandle> DelegateHandleRef = MakeShared<FDelegateHandle>();
		TSharedRef<FTimerHandle> TimerHandleRef = MakeShared<FTimerHandle>();
		
		auto CleanupJipRequest = [this, Instance, PartySystem = PartySystem.ToWeakPtr(), OnComplete, DelegateHandleRef, TimerHandleRef](EPartyJoinDenialReason Result)
		{
			// Clear our JIP request
			FInstanceContext& LocalContext = GetContext(Instance);
			LocalContext.SetJIPRequest(FPartyMemberJoinInProgressRequest());

			OnComplete.ExecuteIfBound(Result);

			// Copy lambda captures as they will be deleted through the unbinds here
			TSharedRef<FDelegateHandle> DelegateHandleCopy = DelegateHandleRef;
			TSharedRef<FTimerHandle> TimerHandleCopy = TimerHandleRef;
			IOnlinePartyPtr StrongPartySystem = PartySystem.Pin();
			UWorld* World = GetWorld();
			if (StrongPartySystem)
			{
				StrongPartySystem->ClearOnPartyMemberDataReceivedDelegate_Handle(*DelegateHandleCopy);
			}
			if (World)
			{
				World->GetTimerManager().ClearTimer(*TimerHandleCopy);
			}
		};

		auto OnPartyMemberDataReceivedLambda = [this, LocalUserId, JipTargetId, Request, CleanupJipRequest](const FUniqueNetId& InLocalUserId, const FOnlinePartyId& InPartyId, const FUniqueNetId& InMemberId, const FName& InNamespace, const FOnlinePartyData& InPartyMemberData)
		{
			if (InMemberId == *JipTargetId)
			{
				FPartyMemberRepData ReceivedPartyMemberRepData;
				if (FVariantDataConverter::VariantMapToUStruct(InPartyMemberData.GetKeyValAttrs(), FPartyMemberRepData::StaticStruct(), &ReceivedPartyMemberRepData, 0, CPF_Transient | CPF_RepSkip))
				{
					for (const FPartyMemberJoinInProgressResponse& Response : ReceivedPartyMemberRepData.GetJoinInProgressDataResponses())
					{
						if (Response.RequestTime != Request.Time ||
							Response.Requester != LocalUserId)
						{
							// Response was not for us.
							continue;
						}

						CleanupJipRequest(static_cast<EPartyJoinDenialReason>(Response.DenialReason));
						return;
					}
				}
			}
		};

		*DelegateHandleRef = PartySystem->AddOnPartyMemberDataReceivedDelegate_Handle(FOnPartyMemberDataReceivedDelegate::CreateWeakLambda(this, OnPartyMemberDataReceivedLambda));
		// wait 60 seconds for a response
		if (UWorld* World = GetWorld())
		{
			World->GetTimerManager().SetTimer(*TimerHandleRef, FTimerDelegate::CreateWeakLambda(this, CleanupJipRequest, static_cast<EPartyJoinDenialReason>(EPartyJoinDenialReason::OssUnavailable)), 60, false);
		}
	}
}

void USocialDebugTools::JoinParty(const FString& Instance, const FString& FriendName, const FJoinPartyComplete& OnComplete)
{
	FInstanceContext& Context = GetContext(Instance);
	IOnlineSubsystem* OnlineSub = Context.GetOSS();
	if (OnlineSub)
	{
		if (FUniqueNetIdPtr LocalUserId = Context.GetLocalUserId())
		{
			IOnlinePartyPtr OnlineParty = OnlineSub->GetPartyInterface();
			if (OnlineParty.IsValid())
			{
				// query user id by name
				if (!FriendName.IsEmpty())
				{
					IOnlineUserPtr OnlineUser = OnlineSub->GetUserInterface();
					if (OnlineUser.IsValid())
					{
						OnlineUser->QueryUserIdMapping(*LocalUserId, FriendName, IOnlineUser::FOnQueryUserMappingComplete::CreateLambda([this, Instance, OnlineParty, OnComplete](bool bWasSuccessful, const FUniqueNetId& UserId, const FString& DisplayNameOrEmail, const FUniqueNetId& FoundUserId, const FString& Error)
						{
							if (bWasSuccessful)
							{
								IOnlinePartyJoinInfoConstPtr JoinInfo = OnlineParty->GetAdvertisedParty(UserId, FoundUserId, IOnlinePartySystem::GetPrimaryPartyTypeId());
								if (JoinInfo.IsValid())
								{
									OnlineParty->JoinParty(UserId, *JoinInfo, FOnJoinPartyComplete::CreateLambda([this, Instance, OnlineParty, OnComplete](const FUniqueNetId& UserIdTmp, const FOnlinePartyId& PartyId, const EJoinPartyCompletionResult Result, const int32 NotApprovedReason)
									{
										bool bSuccess = Result == EJoinPartyCompletionResult::Succeeded;
										if (bSuccess)
										{
											FOnlinePartyDataConstPtr PartyMemberData = GetContext(Instance).GetPartyMemberData();
											if (PartyMemberData.IsValid())
											{
												OnlineParty->UpdatePartyMemberData(UserIdTmp, PartyId, DefaultPartyDataNamespace, *PartyMemberData);
											}
										}
										else
										{
											UE_LOGF(LogParty, Warning, "Party context[%ls] join attempt denied for reason [%d]", *Instance, (int32)Result);
										}
										OnComplete.ExecuteIfBound(bSuccess);
									}));
									return;
								}
							}
							OnComplete.ExecuteIfBound(false);
						}));
						return;
					}
				}					
				else
				{	
					IOnlinePartyJoinInfoConstPtr JoinInfo = GetDefaultPartyJoinInfo();
					if (JoinInfo.IsValid())
					{
						OnlineParty->JoinParty(*LocalUserId, *JoinInfo, FOnJoinPartyComplete::CreateLambda([this, Instance, OnlineParty, OnComplete](const FUniqueNetId& UserId, const FOnlinePartyId& PartyId, const EJoinPartyCompletionResult Result, const int32 NotApprovedReason)
						{
							bool bSuccess = Result == EJoinPartyCompletionResult::Succeeded;
							if (bSuccess)
							{
								FOnlinePartyDataConstPtr PartyMemberData = GetContext(Instance).GetPartyMemberData();
								if (PartyMemberData.IsValid())
								{
									OnlineParty->UpdatePartyMemberData(UserId, PartyId, DefaultPartyDataNamespace, *PartyMemberData);
								}
							}
							else
							{
								UE_LOGF(LogParty, Warning, "Party context[%ls] join attempt denied for reason [%d]", *Instance, (int32)Result);
							}
							OnComplete.ExecuteIfBound(bSuccess);
						}));
						return;
					}
				}
			}
		}
	}
	OnComplete.ExecuteIfBound(false);
}

void USocialDebugTools::LeaveParty(const FString& Instance, const FLeavePartyComplete& OnComplete)
{
	FInstanceContext& Context = GetContext(Instance);
	IOnlineSubsystem* OnlineSub = GetContext(Instance).GetOSS();
	if (OnlineSub)
	{
		if (FUniqueNetIdPtr UserId = Context.GetLocalUserId())
		{
			IOnlinePartyPtr OnlineParty = OnlineSub->GetPartyInterface();
			if (OnlineParty.IsValid())
			{
				TSharedPtr<const FOnlineParty> ExistingParty = OnlineParty->GetParty(*UserId, IOnlinePartySystem::GetPrimaryPartyTypeId());
				if (ExistingParty.IsValid())
				{
					OnlineParty->LeaveParty(*UserId, *ExistingParty->PartyId, true, FOnLeavePartyComplete::CreateLambda([this, Instance, OnComplete](const FUniqueNetId& LocalUserId, const FOnlinePartyId& PartyId, const ELeavePartyCompletionResult Result)
					{
						OnComplete.ExecuteIfBound(Result == ELeavePartyCompletionResult::Succeeded);
					}));
					return;
				}
			}
		}
	}
	OnComplete.ExecuteIfBound(false);
}

void USocialDebugTools::CleanupParties(const FString& Instance, const FCleanupPartiesComplete& OnComplete)
{
	FInstanceContext& Context = GetContext(Instance);
	IOnlineSubsystem* OnlineSub = Context.GetOSS();
	if (OnlineSub)
	{
		if (FUniqueNetIdPtr UserId = Context.GetLocalUserId())
		{
			IOnlinePartyPtr OnlineParty = OnlineSub->GetPartyInterface();
			if (OnlineParty.IsValid())
			{
				OnlineParty->CleanupParties(*UserId, FOnCleanupPartiesComplete::CreateLambda([this, Instance, OnComplete](const FUniqueNetId& LocalUserId, const FOnlineError& Result)
				{
					OnComplete.ExecuteIfBound(Result.WasSuccessful());
				}));
				return;
			}
		}
	}
	OnComplete.ExecuteIfBound(false);
}

void USocialDebugTools::SetPartyMemberData(const FString& Instance, const UStruct* StructType, const void* StructData, const FSetPartyMemberDataComplete& OnComplete)
{
	check(StructType);
	check(StructData);

	bool bResult = false;
	FInstanceContext& Context = GetContext(Instance);
	// parse from struct to party data
	FOnlinePartyData OnlinePartyData;
	if (FVariantDataConverter::UStructToVariantMap(StructType, StructData, OnlinePartyData.GetKeyValAttrs(), 0, CPF_Transient | CPF_RepSkip))
	{
		if (!Context.PartyMemberData.IsValid())
		{
			Context.PartyMemberData = MakeShared<FOnlinePartyData>();
		}
		// cache party data on the context so it can be sent whenever a party is joined
		*Context.PartyMemberData = OnlinePartyData;
		// send the party data if connected to a party
		IOnlineSubsystem* OnlineSub = Context.GetOSS();
		if (OnlineSub)
		{
			if (FUniqueNetIdPtr UserId = Context.GetLocalUserId())
			{
				IOnlinePartyPtr OnlineParty = OnlineSub->GetPartyInterface();
				if (OnlineParty.IsValid())
				{
					TSharedPtr<const FOnlineParty> ExistingParty = OnlineParty->GetParty(*UserId, IOnlinePartySystem::GetPrimaryPartyTypeId());
					if (ExistingParty.IsValid())
					{
						const FOnlinePartyId& PartyId = *ExistingParty->PartyId;
						if (StructType->IsChildOf(FPartyMemberRepData::StaticStruct()))
						{
							UE_LOGF(LogParty, Display, "Sending rep data update for member within party [%ls].", *PartyId.ToDebugString());
							bResult = OnlineParty->UpdatePartyMemberData(*UserId, PartyId, DefaultPartyDataNamespace, *Context.PartyMemberData);
						}
					}
				}
			}
		}
	}
	OnComplete.ExecuteIfBound(bResult);
}

void USocialDebugTools::SetPartyMemberDataJson(const FString& Instance, const FString& JsonStr, const FSetPartyMemberDataComplete& OnComplete)
{
	bool bResult = false;
	FInstanceContext& Context = GetContext(Instance);
	// parse from struct to party data
	FOnlinePartyData OnlinePartyData;
	OnlinePartyData.FromJson(JsonStr);
	if (OnlinePartyData.GetKeyValAttrs().Num() > 0)
	{
		if (!Context.PartyMemberData.IsValid())
		{
			Context.PartyMemberData = MakeShared<FOnlinePartyData>();
		}
		// cache party data on the context so it can be sent whenever a party is joined
		*Context.PartyMemberData = OnlinePartyData;
		// send the party data if connected to a party
		IOnlineSubsystem* OnlineSub = Context.GetOSS();
		if (OnlineSub)
		{
			if (FUniqueNetIdPtr UserId = Context.GetLocalUserId())
			{
				IOnlinePartyPtr OnlineParty = OnlineSub->GetPartyInterface();
				if (OnlineParty.IsValid())
				{
					TSharedPtr<const FOnlineParty> ExistingParty = OnlineParty->GetParty(*UserId, IOnlinePartySystem::GetPrimaryPartyTypeId());
					if (ExistingParty.IsValid())
					{
						const FOnlinePartyId& PartyId = *ExistingParty->PartyId;
						UE_LOGF(LogParty, Display, "Sending rep data update for member within party [%ls].", *PartyId.ToDebugString());
						bResult = OnlineParty->UpdatePartyMemberData(*UserId, PartyId, DefaultPartyDataNamespace, *Context.PartyMemberData);
					}
				}
			}
		}
	}
	else
	{
		UE_LOGF(LogParty, Warning, "SetPartyMemberDataJson didnt parse data from JsonStr=[%ls].", *JsonStr);
	}
	OnComplete.ExecuteIfBound(bResult);
}

IOnlineSubsystem* USocialDebugTools::GetDefaultOSS() const
{
	if (UWorld* World = GetWorld())
	{
		return Online::GetSubsystem(World, MCP_SUBSYSTEM);
	}
	else
	{
		return IOnlineSubsystem::Get(MCP_SUBSYSTEM);
	}
}

USocialManager& USocialDebugTools::GetSocialManager() const
{
	USocialManager* OuterSocialManager = GetTypedOuter<USocialManager>();
	check(OuterSocialManager);
	return *OuterSocialManager;
}

IOnlinePartyJoinInfoConstPtr USocialDebugTools::GetDefaultPartyJoinInfo() const
{
	IOnlinePartyJoinInfoConstPtr Result;
	IOnlineSubsystem* OnlineSub = GetDefaultOSS();
	if (OnlineSub)
	{
		IOnlineIdentityPtr OnlineIdentity = OnlineSub->GetIdentityInterface();
		if (OnlineIdentity.IsValid())
		{
			FUniqueNetIdPtr UserId = OnlineIdentity->GetUniquePlayerId(LocalUserNum);
			if (UserId.IsValid())
			{
				IOnlinePartyPtr OnlineParty = OnlineSub->GetPartyInterface();
				if (OnlineParty.IsValid())
				{
					Result = OnlineParty->GetAdvertisedParty(*UserId, *UserId, IOnlinePartySystem::GetPrimaryPartyTypeId());
					if (!Result.IsValid())
					{
						TSharedPtr<const FOnlineParty> ExistingParty = OnlineParty->GetParty(*UserId, IOnlinePartySystem::GetPrimaryPartyTypeId());
						if (ExistingParty.IsValid())
						{
							FString JsonJoinInfo = OnlineParty->MakeJoinInfoJson(*UserId, *ExistingParty->PartyId);
							Result = OnlineParty->MakeJoinInfoFromJson(JsonJoinInfo);
						}
					}
				}
			}
		}
	}
	return Result;
}

USocialDebugTools::FInstanceContext& USocialDebugTools::GetContext(const FString& Instance)
{
	FInstanceContext* Context = Contexts.Find(Instance);
	if (!Context)
	{
		Context = &Contexts.Add(Instance, FInstanceContext(Instance, *this));
		Context->Init();
		NotifyContextInitialized(*Context);
	}
	return *Context;
}

USocialDebugTools::FInstanceContext* USocialDebugTools::GetContextForUser(const FUniqueNetId& UserId)
{
	FInstanceContext* Result = nullptr;
	for (TPair<FString, FInstanceContext>& Entry : Contexts)
	{
		FInstanceContext& Context = Entry.Value;
		if (Context.GetOSS())
		{
			if (FUniqueNetIdPtr LocalUserId = Context.GetLocalUserId())
			{
				if (*LocalUserId == UserId)
				{
					Result = &Context;
					break;
				}
			}
		}
	}
	return Result;
}

bool USocialDebugTools::RunCommand(const TCHAR* Cmd, const TArray<FString>& TargetInstances)
{
	if (FParse::Command(&Cmd, TEXT("LOGIN")))
	{
		if (TargetInstances.Num() > 1)
		{
			UE_LOGF(LogParty, Log, "CONTEXT=ALL cannot be used for the LOGIN command!");
		}
		else
		{
			FString InstanceName = TargetInstances[0];
			FString Id = FParse::Token(Cmd, false);
			FString Auth = FParse::Token(Cmd, false);
			
			int MaxConcurrentLogins = 512; // Reasonable limit to prevent infinite loop
			while (!Id.IsEmpty() && !Auth.IsEmpty() && MaxConcurrentLogins >= 0)
			{
				Login(InstanceName, FOnlineAccountCredentials(TEXT("epic"), Id, Auth), FLoginComplete::CreateLambda([this, InstanceName](bool bSuccess)
				{
					UE_LOGF(LogParty, Display, "Login OSS context[%ls] %ls", *InstanceName, *LexToString(bSuccess));
				}));
				
				// Allow for more logins by concatenating more context=name auth pass
				if (FParse::Value(Cmd, TEXT("CONTEXT="), InstanceName))
				{
					// strip out context to parse next entry
					FParse::Command(&Cmd, *FString(TEXT("CONTEXT=") + InstanceName));
					Id = FParse::Token(Cmd, false);
					Auth = FParse::Token(Cmd, false);
					--MaxConcurrentLogins;
				}
				else
				{
					break;
				}
			}
		}
		return true;
	}
	else if (FParse::Command(&Cmd, TEXT("LOGOUT")))
	{
		for (const FString& TargetInstance : TargetInstances)
		{
			Logout(TargetInstance, FLogoutComplete::CreateLambda([this, TargetInstance](bool bSuccess)
			{
				UE_LOGF(LogParty, Display, "Logout OSS context[%ls] %ls", *TargetInstance, *LexToString(bSuccess));
			}));
		}
		return true;
	}
	else if (FParse::Command(&Cmd, TEXT("JOINPARTY")))
	{
		const FString Id = FParse::Token(Cmd, false);
		const FString Auth = FParse::Token(Cmd, false);
		const FString FriendName = FParse::Token(Cmd, false);

		for (const FString& TargetInstance : TargetInstances)
		{
			Login(TargetInstance, FOnlineAccountCredentials(TEXT("epic"), Id, Auth), FLoginComplete::CreateLambda([this, TargetInstance, FriendName](bool bSuccess)
			{
				UE_LOGF(LogParty, Display, "Login OSS context[%ls] %ls", *TargetInstance, *LexToString(bSuccess));

				if (bSuccess)
				{
					CleanupParties(TargetInstance, FCleanupPartiesComplete::CreateLambda([this, TargetInstance, FriendName](bool bCleanupPartiesSuccess)
					{
						LeaveParty(TargetInstance, FLeavePartyComplete::CreateLambda([this, TargetInstance, FriendName](bool bLeavePartySuccess)
						{
							UE_LOGF(LogParty, Display, "Leave party OSS context[%ls] %ls", *TargetInstance, *LexToString(bLeavePartySuccess));

							JoinParty(TargetInstance, FriendName, FJoinPartyComplete::CreateLambda([this, TargetInstance](bool bJoinPartySuccess)
							{
								UE_LOGF(LogParty, Display, "Join party OSS context[%ls] %ls", *TargetInstance, *LexToString(bJoinPartySuccess));
							}));
						}));
					}));
				}
			}));
		}
		return true;
	}
	else if (FParse::Command(&Cmd, TEXT("JIP")))
	{
		for (const FString& TargetInstance : TargetInstances)
		{
			JoinInProgress(TargetInstance, FJoinInProgressComplete::CreateLambda([this, TargetInstance](EPartyJoinDenialReason Result)
			{
				UE_LOGF(LogParty, Display, "JIP OSS context[%ls] %ls", *TargetInstance, ToString(Result));
			}));
		}
		return true;
	}
	else if (FParse::Command(&Cmd, TEXT("LEAVEPARTY")))
	{
		for (const FString& TargetInstance : TargetInstances)
		{
			LeaveParty(TargetInstance, FLeavePartyComplete::CreateLambda([this, TargetInstance](bool bSuccess)
			{
				UE_LOGF(LogParty, Display, "Leave party OSS context[%ls] %ls", *TargetInstance, *LexToString(bSuccess));
			}));
		}
		return true;
	}
	else if (FParse::Command(&Cmd, TEXT("AUTOACCEPTFRIENDINVITES")))
	{
		const FString Enabled = FParse::Token(Cmd, false);
		if (Enabled == TEXT("true") || Enabled == TEXT("1"))
		{
			bAutoAcceptFriendInvites = true;
		}
		else if (Enabled == TEXT("false") || Enabled == TEXT("0"))
		{
			bAutoAcceptFriendInvites = false;
		}
		return true;
	}
	else if (FParse::Command(&Cmd, TEXT("AUTOACCEPTPARTYINVITES")))
	{
		const FString Enabled = FParse::Token(Cmd, false);
		if (Enabled == TEXT("true") || Enabled == TEXT("1"))
		{
			bAutoAcceptPartyInvites = true;
		}
		else if (Enabled == TEXT("false") || Enabled == TEXT("0"))
		{
			bAutoAcceptPartyInvites = false;
		}
		return true;
	}
	else if (FParse::Command(&Cmd, TEXT("DUMPPARTY")))
	{
		for (const FString& TargetInstance : TargetInstances)
		{
			UE_LOGF(LogParty, Display, "---DUMPPARTY - party OSS context[%ls]", *TargetInstance);
			IOnlineSubsystem* OnlineSub = GetContext(TargetInstance).GetOSS();
			if (OnlineSub &&
				OnlineSub->GetPartyInterface().IsValid())
			{
				OnlineSub->GetPartyInterface()->DumpPartyState();
			}
		}
		return true;
	}

	return false;
}

void USocialDebugTools::FInstanceContext::Init()
{
	const FString OSSName = FName(MCP_SUBSYSTEM).ToString() + TEXT(":") + Name;
	OnlineSub = IOnlineSubsystem::Get(*OSSName);
	if (OnlineSub)
	{
		// register delegates
		IOnlineFriendsPtr OnlineFriends = OnlineSub->GetFriendsInterface();
		if (OnlineFriends.IsValid())
		{
			FriendInviteReceivedDelegateHandle = OnlineFriends->AddOnInviteReceivedDelegate_Handle(FOnInviteReceivedDelegate::CreateUObject(&Owner, &USocialDebugTools::HandleFriendInviteReceived));
		}
		IOnlinePartyPtr OnlineParty = OnlineSub->GetPartyInterface();
		if (OnlineParty.IsValid())
		{
			PartyInviteReceivedDelegateHandle = OnlineParty->AddOnPartyInviteReceivedExDelegate_Handle(FOnPartyInviteReceivedExDelegate::CreateUObject(&Owner, &USocialDebugTools::HandlePartyInviteReceived));
			PartyJoinRequestReceivedDelegateHandle = OnlineParty->AddOnPartyJoinRequestReceivedDelegate_Handle(FOnPartyJoinRequestReceivedDelegate::CreateUObject(&Owner, &USocialDebugTools::HandlePartyJoinRequestReceived));
		}
	}
}

void USocialDebugTools::FInstanceContext::Shutdown()
{
	if (OnlineSub)
	{
		// unregister delegates
		IOnlineFriendsPtr OnlineFriends = OnlineSub->GetFriendsInterface();
		if (OnlineFriends.IsValid())
		{
			OnlineFriends->ClearOnInviteReceivedDelegate_Handle(FriendInviteReceivedDelegateHandle);
			OnlineFriends.Reset();
		}
		IOnlinePartyPtr OnlineParty = OnlineSub->GetPartyInterface();
		if (OnlineParty.IsValid())
		{
			OnlineParty->ClearOnPartyInviteReceivedExDelegate_Handle(PartyInviteReceivedDelegateHandle);
			OnlineParty->ClearOnPartyJoinRequestReceivedDelegate_Handle(PartyJoinRequestReceivedDelegateHandle);
			OnlineParty.Reset();
		}
		OnlineSub->Shutdown();

		const FString OSSName = FName(MCP_SUBSYSTEM).ToString() + TEXT(":") + Name;
		IOnlineSubsystem::Destroy(*OSSName);
	}
}

FUniqueNetIdPtr USocialDebugTools::FInstanceContext::GetLocalUserId() const
{
	FUniqueNetIdPtr LocalUserId;
	if (OnlineSub)
	{
		IOnlineIdentityPtr OnlineIdentity = OnlineSub->GetIdentityInterface();
		if (OnlineIdentity.IsValid())
		{
			LocalUserId = OnlineIdentity->GetUniquePlayerId(LocalUserNum);
		}
	}
	return LocalUserId;
}

void USocialDebugTools::FInstanceContext::ModifyPartyField(const FString& FieldName, const FVariantData& FieldValue)
{
	if (!OnlineSub)
	{
		return;
	}
	IOnlinePartyPtr PartySystem = OnlineSub->GetPartyInterface();
	if (!PartySystem)
	{
		return;
	}
	FUniqueNetIdPtr LocalUserId = GetLocalUserId();
	if (!LocalUserId)
	{
		return;
	}
	TSharedPtr<const FOnlineParty> Party = PartySystem->GetParty(*LocalUserId, IOnlinePartySystem::GetPrimaryPartyTypeId());
	if (!Party)
	{
		return;
	}

	// Get current party data and update the JIP field
	if (!PartyMemberData)
	{
		if (FOnlinePartyDataConstPtr ExistingPartyMemberData = PartySystem->GetPartyMemberData(*LocalUserId, *Party->PartyId, *LocalUserId, DefaultPartyDataNamespace))
		{
			PartyMemberData = MakeShared<FOnlinePartyData>(*ExistingPartyMemberData);
		}
	}
	if (!PartyMemberData)
	{
		PartyMemberData = MakeShared<FOnlinePartyData>();
	}
	PartyMemberData->SetAttribute(FieldName, FieldValue);
	PartySystem->UpdatePartyMemberData(*LocalUserId, *Party->PartyId, DefaultPartyDataNamespace , *PartyMemberData);
}

bool USocialDebugTools::FInstanceContext::SetJIPRequest(const FPartyMemberJoinInProgressRequest& InRequest)
{
	FPartyMemberRepData PartyMemberRepData;
	PartyMemberRepData.MarkOwnerless();
	PartyMemberRepData.SetJoinInProgressDataRequest(InRequest);

	FOnlinePartyData OnlinePartyData;
	const FString JipFieldName(TEXT("JoinInProgressData"));
	if (FVariantDataConverter::UStructToVariantMap(FPartyMemberRepData::StaticStruct(), &PartyMemberRepData, OnlinePartyData.GetKeyValAttrs(), 0, CPF_Transient | CPF_RepSkip) &&
		ensure(OnlinePartyData.GetKeyValAttrs().Contains(JipFieldName)))
	{
		ModifyPartyField(JipFieldName, OnlinePartyData.GetKeyValAttrs().FindChecked(JipFieldName));
		return true;
	}
	return false;
}

void USocialDebugTools::HandleFriendInviteReceived(const FUniqueNetId& LocalUserId, const FUniqueNetId& FriendId)
{
	if (bAutoAcceptFriendInvites)
	{
		FInstanceContext* Context = GetContextForUser(LocalUserId);
		if (Context)
		{
			IOnlineFriendsPtr OnlineFriends = Context->GetOSS()->GetFriendsInterface();
			if (OnlineFriends.IsValid())
			{
				OnlineFriends->AcceptInvite(LocalUserNum, FriendId, EFriendsLists::ToString(EFriendsLists::Default));
			}
		}
	}
}

void USocialDebugTools::HandlePartyInviteReceived(const FUniqueNetId& LocalUserId, const IOnlinePartyJoinInfo& Invitation)
{
	if (bAutoAcceptPartyInvites)
	{
		FInstanceContext* Context = GetContextForUser(LocalUserId);
		if (Context)
		{
			IOnlinePartyPtr OnlineParty = Context->GetOSS()->GetPartyInterface();
			if (OnlineParty.IsValid())
			{
				const FString Instance = Context->Name;
				OnlineParty->JoinParty(LocalUserId, Invitation, FOnJoinPartyComplete::CreateLambda([this, Instance, OnlineParty](const FUniqueNetId& UserId, const FOnlinePartyId& PartyIdTmp, const EJoinPartyCompletionResult Result, const int32 NotApprovedReason)
				{
					bool bSuccess = Result == EJoinPartyCompletionResult::Succeeded;
					if (bSuccess)
					{
						FOnlinePartyDataConstPtr PartyMemberData = GetContext(Instance).GetPartyMemberData();
						if (PartyMemberData.IsValid())
						{
							OnlineParty->UpdatePartyMemberData(UserId, PartyIdTmp, DefaultPartyDataNamespace, *PartyMemberData);
						}
					}
					else
					{
						UE_LOGF(LogParty, Warning, "Party context[%ls] invite join attempt denied for reason [%d]", *Instance, (int32)Result);
					}
				}));
			}
		}
	}
}

void USocialDebugTools::HandlePartyJoinRequestReceived(const FUniqueNetId& LocalUserId, const FOnlinePartyId& PartyId, const IOnlinePartyPendingJoinRequestInfo& JoinRequestInfo)
{
	FInstanceContext* Context = GetContextForUser(LocalUserId);
	if (Context)
	{
		IOnlinePartyPtr OnlineParty = Context->GetOSS()->GetPartyInterface();
		if (OnlineParty.IsValid())
		{
			TArray<IOnlinePartyUserPendingJoinRequestInfoConstRef> JoiningUsers;
			JoinRequestInfo.GetUsers(JoiningUsers);
			check(JoiningUsers.IsValidIndex(0));
			OnlineParty->ApproveJoinRequest(LocalUserId, PartyId, *JoiningUsers[0]->GetUserId(), true);
			return;
		}
	}
}