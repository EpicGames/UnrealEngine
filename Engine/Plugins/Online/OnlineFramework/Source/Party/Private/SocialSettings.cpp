// Copyright Epic Games, Inc. All Rights Reserved.

#include "SocialSettings.h"
#include "HAL/IConsoleManager.h"
#include "OnlineSubsystem.h"
#include "SocialManager.h"
#include "Misc/CommandLine.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SocialSettings)

#if !UE_BUILD_SHIPPING
int32 MaxPartySizeOverride = INDEX_NONE;
FAutoConsoleVariableRef CVarMaxPartySize(
	TEXT("SocialSettings.MaxPartySize"),
	MaxPartySizeOverride,
	TEXT("Override the maximum persistent party size allowed by the social system"));
#endif

USocialSettings::USocialSettings()
{
	// Switch is the only default supported OSS that does not itself support multiple environments
	OssNamesWithEnvironmentIdPrefix.Add(NINTENDO_SUBSYSTEM);
}

FString USocialSettings::GetUniqueIdEnvironmentPrefix(ESocialSubsystem SubsystemType)
{
	const USocialSettings& SettingsCDO = *GetDefault<USocialSettings>();

	// We don't need to worry about world specificity here for the OSS (both because there is no platform PIE and because we aren't accessing data that could differ if there was)
	IOnlineSubsystem* OSS = USocialManager::GetSocialOss(nullptr, SubsystemType);
	if (OSS && SettingsCDO.OssNamesWithEnvironmentIdPrefix.Contains(OSS->GetSubsystemName()))
	{
		return OSS->GetOnlineEnvironmentName() + TEXT("_");
	}
	return FString();
}

bool USocialSettings::ShouldPreferPlatformInvites()
{
	const USocialSettings& SettingsCDO = *GetDefault<USocialSettings>();
	return SettingsCDO.bPreferPlatformInvites;
}

bool USocialSettings::MustSendPrimaryInvites()
{
	const USocialSettings& SettingsCDO = *GetDefault<USocialSettings>();
	return SettingsCDO.bMustSendPrimaryInvites;
}

bool USocialSettings::MustAlwaysSendPlatformSessionInvites()
{
	const USocialSettings& SettingsCDO = *GetDefault<USocialSettings>();
	return SettingsCDO.bMustAlwaysSendPlatformInvites;
}

bool USocialSettings::ShouldLeavePartyOnDisconnect()
{
	const USocialSettings& SettingsCDO = *GetDefault<USocialSettings>();
	return SettingsCDO.bLeavePartyOnDisconnect;
}

bool USocialSettings::ShouldSetDesiredPrivacyOnLocalPlayerBecomesLeader()
{
	const USocialSettings& SettingsCDO = *GetDefault<USocialSettings>();
	return SettingsCDO.bSetDesiredPrivacyOnLocalPlayerBecomesLeader;
}

int32 USocialSettings::GetDefaultMaxPartySize()
{
#if !UE_BUILD_SHIPPING
	if (MaxPartySizeOverride > 0)
	{
		return MaxPartySizeOverride;
	}

	static FString CommandLineOverridePartySize;
	if (FParse::Value(FCommandLine::Get(), TEXT("MaxPartySize="), CommandLineOverridePartySize))
	{
		return FCString::Atoi(*CommandLineOverridePartySize);
	}
#endif

	const USocialSettings& SettingsCDO = *GetDefault<USocialSettings>();
	return SettingsCDO.DefaultMaxPartySize;
}

float USocialSettings::GetUserListAutoUpdateRate()
{
	const USocialSettings& SettingsCDO = *GetDefault<USocialSettings>();
	return SettingsCDO.UserListAutoUpdateRate;
}

int32 USocialSettings::GetMinNicknameLength()
{
	const USocialSettings& SettingsCDO = *GetDefault<USocialSettings>();
	return SettingsCDO.MinNicknameLength;
}

int32 USocialSettings::GetMaxNicknameLength()
{
	const USocialSettings& SettingsCDO = *GetDefault<USocialSettings>();
	return SettingsCDO.MaxNicknameLength;
}

const TArray<FSocialPlatformDescription>& USocialSettings::GetSocialPlatformDescriptions()
{
	const USocialSettings& SettingsCDO = *GetDefault<USocialSettings>();
	return SettingsCDO.SocialPlatformDescriptions;
}

const FSocialPlatformDescription* USocialSettings::GetSocialPlatformDescriptionForOnlineSubsystem(const FName& OnlineSubsystemName)
{
	return GetSocialPlatformDescriptions().FindByPredicate([&OnlineSubsystemName](const FSocialPlatformDescription& Candidate)
	{
		return Candidate.OnlineSubsystem == OnlineSubsystemName;
	});
}

// Static
bool USocialSettings::IsSonyOSS(const FName& OSSName)
{
	const USocialSettings& SettingsCDO = *GetDefault<USocialSettings>();
	for (const FName& SonyOSS : SettingsCDO.SonyOSSNames)
	{
		if (SonyOSS == OSSName)
		{
			return true;
		}
	}
	return false;
}
