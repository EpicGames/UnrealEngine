// Copyright Epic Games, Inc. All Rights Reserved.

#include "SocialToolkit.h"
#include "Interfaces/OnlineUserInterface.h"
#include "SocialManager.h"
#include "SocialQuery.h"
#include "User/SocialUserList.h"
#include "Chat/SocialChatManager.h"
#include "Party/SocialParty.h"
#include "Party/PartyMember.h"

#include "OnlineSubsystemUtils.h"
#include "Engine/LocalPlayer.h"

#include "Online/OnlineServices.h"
#include "Online/PresenceOSSAdapter.h"
#include "OnlineFramework/CommonAccount.h"
#include "OnlineFramework/CommonAccountUtils.h"
#include "OnlineFramework/CommonConfig.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SocialToolkit)

namespace FriendInviteFailureReason 
{
	const FString InviteFailReason_NotFound = TEXT("NotFound");
	const FString InviteFailReason_AddingSelfFail = TEXT("AddingSelfFail");
	const FString InviteFailReason_AddingBlockedFail = TEXT("AddingBlockedFail");
	const FString InviteFailReason_AlreadyFriends = TEXT("AlreadyFriends");
}
namespace FriendAcceptFailureReason
{
	const FString AcceptFailReason_NotPendingInbound = TEXT("NotPendingInbound");
}

bool bQueryRecentPlayersAfterPartyUpdates = true;
static FAutoConsoleVariableRef CVarQueryRecentPlayersAfterPartyUpdates(
	TEXT("Social.QueryRecentPlayersAfterPartyUpdates"), 
	bQueryRecentPlayersAfterPartyUpdates,
	TEXT("Query list of recent players when a player joins a party containing at least one non-friend, or a non-friend joins the player's existing party (unless all of the non-friended players are already in the list of recent players)"),
	ECVF_Default);

#if WITH_EDITOR || !UE_BUILD_SHIPPING
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnStartRandomizeUserPresence, uint8 /*NumRandomUser*/, float /*TickerTimer*/);
static FOnStartRandomizeUserPresence Debug_OnStartRandomizeUserPresenceEvent;
static FAutoConsoleCommandWithWorldAndArgs CMD_OnStartRandomUserPresence
(
	TEXT("SocialUI.StartRandomizeUserPresence"),
	TEXT("Randomize users' presence and fire off presence changed events to trigger friend list update/refresh etc. @param NumRandomUser uint8, @param TickerTimer float"),
	FConsoleCommandWithWorldAndArgsDelegate::CreateLambda([](const TArray<FString>&Args, UWorld* World)
{
	uint8 NumRandomUser = 2;
	float TickerTimer = 5.f;

	if (Args.Num() > 0)
	{
		NumRandomUser = FCString::Atoi(*Args[0]);
	}
	if (Args.Num() > 1)
	{
		TickerTimer = FCString::Atof(*Args[1]);
	}
	Debug_OnStartRandomizeUserPresenceEvent.Broadcast(NumRandomUser, TickerTimer);
})
);

DECLARE_MULTICAST_DELEGATE_OneParam(FOnStopRandomizeUserPresence, bool /*bClearGeneratedPresence*/)
static FOnStopRandomizeUserPresence Debug_OnStopRandomizeUserPresenceEvent;
static FAutoConsoleCommandWithWorldAndArgs CMD_OnStopRandomUserPresence
(
	TEXT("SocialUI.StopRandomizeUserPresence"),
	TEXT("Stop randomizing users' presnece, with optional param to clear off already generated presence (default to false). @param bClearGeneratedPresence uint8"),
	FConsoleCommandWithWorldAndArgsDelegate::CreateLambda([](const TArray<FString>&Args, UWorld* World)
{
	bool bClearGeneratedPresence = false;
	if (Args.Num() > 0)
	{
		bClearGeneratedPresence = Args[0].ToBool();
	}
	Debug_OnStopRandomizeUserPresenceEvent.Broadcast(bClearGeneratedPresence);
})
);

#endif

bool NameToSocialSubsystem(FName SubsystemName, ESocialSubsystem& OutSocialSubsystem)
{
	for (uint8 SocialSubsystemIdx = 0; SocialSubsystemIdx < (uint8)ESocialSubsystem::MAX; ++SocialSubsystemIdx)
	{
		if (SubsystemName == USocialManager::GetSocialOssName((ESocialSubsystem)SocialSubsystemIdx))
		{
			OutSocialSubsystem = (ESocialSubsystem)SocialSubsystemIdx;
			return true;
		}
	}
	return false;
}

//////////////////////////////////////////////////////////////////////////
// FSocialQuery_MapExternalIds
//////////////////////////////////////////////////////////////////////////

class FSocialQuery_MapExternalIds : public TSocialQuery<FString, const FUniqueNetIdRepl&>
{
public:
	static FName GetQueryId() { return TEXT("MapExternalIds"); }

	virtual void AddUserId(const FString& UserIdStr, const FOnQueryComplete& QueryCompleteHandler) override
	{
		// Prepend the environment prefix (if there is one) to the true ID we're after before actually adding the ID
		const FString MappableIdStr = USocialSettings::GetUniqueIdEnvironmentPrefix(SubsystemType) + UserIdStr;

		TSocialQuery<FString, const FUniqueNetIdRepl&>::AddUserId(MappableIdStr, QueryCompleteHandler);
	}

	virtual void ExecuteQuery() override
	{
		FUniqueNetIdRepl LocalUserPrimaryId = Toolkit.IsValid() ? Toolkit->GetLocalUserNetId(ESocialSubsystem::Primary) : FUniqueNetIdRepl();
		if (LocalUserPrimaryId.IsValid())
		{
			// The external mappings will always be checked on the primary OSS, so we use the passed-in OSS as the target we want to map to
			IOnlineSubsystem* OSS = GetOSS();
			check(OSS);
			const FSocialPlatformDescription* PlatformDescription = OSS ? USocialSettings::GetSocialPlatformDescriptionForOnlineSubsystem(OSS->GetSubsystemName()) : nullptr;
			IOnlineUserPtr PrimaryUserInterface = Toolkit->GetSocialOss(ESocialSubsystem::Primary)->GetUserInterface();
			if (ensure(PlatformDescription && PrimaryUserInterface))
			{
				bHasExecuted = true;
				
				TArray<FString> ExternalUserIds;
				CompletionCallbacksByUserId.GenerateKeyArray(ExternalUserIds);
				UE_LOGF(LogParty, Log, "FSocialQuery_MapExternalIds executing for [%d] users on subsystem [%ls]", ExternalUserIds.Num(), ToString(SubsystemType));

				FExternalIdQueryOptions QueryOptions(PlatformDescription->ExternalAccountType.ToLower(), false);
				PrimaryUserInterface->QueryExternalIdMappings(*LocalUserPrimaryId, QueryOptions, ExternalUserIds, IOnlineUser::FOnQueryExternalIdMappingsComplete::CreateSP(this, &FSocialQuery_MapExternalIds::HandleQueryExternalIdMappingsComplete));
			}
			else
			{
				UE_LOGF(LogParty, Warning, "%ls - PlatformDescription: %llx (%ls) - PrimaryUserInterface: %llx", ANSI_TO_TCHAR(__FUNCTION__), PlatformDescription, PlatformDescription ? *PlatformDescription->Name : TEXT("N/A"), PrimaryUserInterface.Get());
			}
		}
		else
		{
			UE_LOGF(LogParty, Warning, "FSocialQuery_MapExternalIds cannot execute query - unable to get a valid primary net ID for the local player.");
		}
	}

	void HandleQueryExternalIdMappingsComplete(bool bWasSuccessful, const FUniqueNetId&, const FExternalIdQueryOptions& QueryOptions, const TArray<FString>& ExternalIds, const FString& ErrorStr)
	{
		UE_LOGF(LogParty, Log, "FSocialQuery_MapExternalIds completed query for [%d] users Subsystem=[%ls] bWasSuccessful=[%ls] Error=[%ls]", ExternalIds.Num(), ToString(SubsystemType), *LexToString(bWasSuccessful), *ErrorStr);

		if (bWasSuccessful)
		{
			IOnlineUserPtr PrimaryUserInterface = Toolkit->GetSocialOss(ESocialSubsystem::Primary)->GetUserInterface();
			if (PrimaryUserInterface.IsValid() && bWasSuccessful)
			{
				for (const FString& ExternalId : ExternalIds)
				{
					FUniqueNetIdPtr PrimaryId = PrimaryUserInterface->GetExternalIdMapping(QueryOptions, ExternalId);
					if (!PrimaryId.IsValid())
					{
#if !UE_BUILD_SHIPPING
						UE_LOGF(LogParty, Verbose, "No primary Id exists that corresponds to external Id [%ls]", *ExternalId);
#endif	
					}
					else if (CompletionCallbacksByUserId[ExternalId].IsBound())
					{
						CompletionCallbacksByUserId[ExternalId].Execute(SubsystemType, bWasSuccessful, PrimaryId);
					}
				}
			}

			OnQueryCompleted.ExecuteIfBound(GetQueryId(), AsShared());
		}
		else
		{
			bHasExecuted = false;
		}
	}
};

//////////////////////////////////////////////////////////////////////////
// USocialToolkit
//////////////////////////////////////////////////////////////////////////

//@todo DanH Social: Need a non-backdoor way to get toolkits from the manager (an issue when we don't know where the manager is) - new game subsystems should be a nice solve
TMap<TWeakObjectPtr<const ULocalPlayer>, TWeakObjectPtr<USocialToolkit>> USocialToolkit::AllToolkitsByOwningPlayer;
USocialToolkit* USocialToolkit::GetToolkitForPlayerInternal(const ULocalPlayer* LocalPlayer)
{
	TWeakObjectPtr<USocialToolkit>* FoundToolkit = AllToolkitsByOwningPlayer.Find(LocalPlayer);
	return FoundToolkit ? FoundToolkit->Get() : nullptr;
}

USocialToolkit::USocialToolkit()
	: SocialUserClass(USocialUser::StaticClass())
	, ChatManagerClass(USocialChatManager::StaticClass())
{

}

void USocialToolkit::InitializeToolkit(ULocalPlayer& InOwningLocalPlayer)
{
#if  WITH_EDITOR || !UE_BUILD_SHIPPING
	Debug_OnStartRandomizeUserPresenceEvent.AddUObject(this, &USocialToolkit::Debug_OnStartRandomizeUserPresence);
	Debug_OnStopRandomizeUserPresenceEvent.AddUObject(this, &USocialToolkit::Debug_OnStopRandomizeUserPresence);
#endif

	LocalPlayerOwner = &InOwningLocalPlayer;

	// We want to allow reliable access to the SocialUser for the local player, but we can't initialize it until we actually log in
	LocalUser = NewObject<USocialUser>(this, SocialUserClass);

	SocialChatManager = USocialChatManager::CreateChatManager(*this);

	check(!AllToolkitsByOwningPlayer.Contains(LocalPlayerOwner));
	AllToolkitsByOwningPlayer.Add(LocalPlayerOwner, this);

	InOwningLocalPlayer.OnControllerIdChanged().AddUObject(this, &USocialToolkit::HandleControllerIdChanged);
	HandleControllerIdChanged(InOwningLocalPlayer.GetControllerId(), INVALID_CONTROLLERID);
}

bool USocialToolkit::IsOwnerLoggedIn() const
{
	const IOnlineIdentityPtr IdentityInterface = Online::GetIdentityInterface(GetWorld());
	if (ensure(IdentityInterface.IsValid()))
	{
		const ELoginStatus::Type CurrentLoginStatus = IdentityInterface->GetLoginStatus(GetLocalUserNum());
		return CurrentLoginStatus == ELoginStatus::LoggedIn;
	}
	return false;
}

USocialChatManager& USocialToolkit::GetChatManager() const
{
	check(SocialChatManager);
	return *SocialChatManager;
}

IOnlineSubsystem* USocialToolkit::GetSocialOss(ESocialSubsystem SubsystemType) const
{
	return Online::GetSubsystem(GetWorld(), USocialManager::GetSocialOssName(SubsystemType));
}

TSharedRef<ISocialUserList> USocialToolkit::CreateUserList(const FSocialUserListConfig& ListConfig) const
{
	CachedSocialUserLists.RemoveAll([](const TWeakPtr<FSocialUserList>& UserList)
	{
		return !UserList.IsValid(); 
	});

	TWeakPtr<FSocialUserList>* FoundUserList = CachedSocialUserLists.FindByPredicate([ListConfig](const TWeakPtr<FSocialUserList>& UserList)
	{
		return UserList.IsValid() ? UserList.Pin()->GetListConfig() == ListConfig : false;
	});

	if (FoundUserList && (*FoundUserList).IsValid())
	{
		UE_LOGF(LogParty, Verbose, "%ls Found Userlist %ls while creating Userlist %ls with the same list config.", ANSI_TO_TCHAR(__FUNCTION__), *(*FoundUserList).Pin()->GetListConfig().Name, *ListConfig.Name);
		return (*FoundUserList).Pin().ToSharedRef();
	}
	
	TSharedRef<FSocialUserList> NewUserList = FSocialUserList::CreateUserList(*this, ListConfig);
	CachedSocialUserLists.Add(NewUserList);
	return NewUserList;
}

USocialUser& USocialToolkit::GetLocalUser() const
{
	return *LocalUser;
}

FUniqueNetIdRepl USocialToolkit::GetLocalUserNetId(ESocialSubsystem SubsystemType) const
{
	return LocalUser->GetUserId(SubsystemType);
}

int32 USocialToolkit::GetLocalUserNum() const
{
	if (const ULocalPlayer* LocalPlayer = GetOwningLocalPlayerPtr())
	{
		return LocalPlayer->GetControllerId();
	}
	else
	{
		UE_LOGF(LogParty, Verbose, "%s - OwningLocalPlayer is nullptr. Likely the player has logged out.", __FUNCTION__);

		return INVALID_CONTROLLERID;
	}
}

const FOnlineUserPresence* USocialToolkit::GetPresenceInfo(ESocialSubsystem SubsystemType) const
{
	if (IOnlineSubsystem* Oss = GetSocialOss(SubsystemType))
	{
		IOnlinePresencePtr PresenceInterface = Oss->GetPresenceInterface();
		FUniqueNetIdRepl LocalUserId = GetLocalUserNetId(SubsystemType);
		if (PresenceInterface.IsValid() && LocalUserId.IsValid())
		{
			TSharedPtr<FOnlineUserPresence> CurrentPresence;
			PresenceInterface->GetCachedPresence(*LocalUserId, CurrentPresence);
			if (CurrentPresence.IsValid())
			{
				return CurrentPresence.Get();
			}
		}
	}
	return nullptr;
}

const UE::Online::TOnlineResult<UE::Online::FGetCachedPresence> USocialToolkit::GetPresenceInfoV2(ESocialSubsystem SubsystemType) const
{
	using namespace UE::Online;
	using namespace UE::OnlineFramework;

	TOnlineResult<FGetCachedPresence> Result(Errors::NotFound());

	FCommonConfig CommonConfig(this);
	if (IOnlineServicesPtr OnlineServices = CommonConfig.GetServices(USocialManager::GetPresenceFrameworkInstance()))
	{
		FUniqueNetIdRepl LocalUserNetId = GetLocalUserNetId(SubsystemType);
		if(LocalUserNetId.IsValid())
		{
			if (const FCommonAccountPtr LocalUserCommonAccountPtr = GetCommonAccountFromV1(CommonConfig, LocalUserNetId->AsShared(), GetFirstFrameworkInstanceName(LocalUserNetId)))
			{
				FAccountId LocalUserAccountId = LocalUserCommonAccountPtr->GetId(USocialManager::GetPresenceFrameworkInstance());

				if (IPresencePtr Presence = OnlineServices->GetPresenceInterface())
				{
					Result = Presence->GetCachedPresence({ LocalUserAccountId, LocalUserAccountId });
				}
			}
		}
	}

	return Result;
}

void USocialToolkit::SetLocalUserOnlineState(EOnlinePresenceState::Type OnlineState)
{
	if (IOnlineSubsystem* PrimaryOss = GetSocialOss(ESocialSubsystem::Primary))
	{
		 IOnlinePresencePtr PresenceInterface = PrimaryOss->GetPresenceInterface();
		 FUniqueNetIdRepl LocalUserId = GetLocalUserNetId(ESocialSubsystem::Primary);
		 if (PresenceInterface.IsValid() && LocalUserId.IsValid())
		 {
			 FOnlinePresenceSetPresenceParameters NewStatus;
			 NewStatus.State = OnlineState;
			 PresenceInterface->SetPresence(*LocalUserId, MoveTemp(NewStatus));
		 }
	}
}

void USocialToolkit::SetLocalUserOnlineStateV2(UE::Online::EUserPresenceStatus OnlineState)
{
	using namespace UE::Online;
	using namespace UE::OnlineFramework;
	
	FCommonConfig CommonConfig(this);
	if (IOnlineServicesPtr OnlineServices = CommonConfig.GetServices(USocialManager::GetPresenceFrameworkInstance()))
	{
		FUniqueNetIdRepl LocalUserNetId = GetLocalUserNetId(ESocialSubsystem::Primary);
		if(LocalUserNetId.IsValid())
		{
			if (const FCommonAccountPtr LocalUserCommonAccountPtr = GetCommonAccountFromV1(CommonConfig, LocalUserNetId->AsShared(), GetFirstFrameworkInstanceName(LocalUserNetId)))
			{
				if(FAccountId LocalUserAccountId = LocalUserCommonAccountPtr->GetId(USocialManager::GetPresenceFrameworkInstance()))
				{
					if (IPresencePtr Presence = OnlineServices->GetPresenceInterface())
					{
						FPartialUpdatePresence::Params::FMutations Mutations;
						Mutations.Status = OnlineState;

						FPartialUpdatePresence::Params Params = { LocalUserAccountId, Mutations };

						UE_LOGF(LogParty, Verbose, "Updating rich presence with OnlineServices");

						Presence->PartialUpdatePresence(MoveTemp(Params));
					}
				}
			}
		}
	}
}

void USocialToolkit::AddLocalUserOnlineProperties(FPresenceProperties OnlineProperties)
{
	if (IOnlineSubsystem* PrimaryOss = GetSocialOss(ESocialSubsystem::Primary))
	{
		IOnlinePresencePtr PresenceInterface = PrimaryOss->GetPresenceInterface();
		FUniqueNetIdRepl LocalUserId = GetLocalUserNetId(ESocialSubsystem::Primary);
		if (PresenceInterface.IsValid() && LocalUserId.IsValid())
		{
			TSharedPtr<FOnlineUserPresence> CurrentPresence;
			PresenceInterface->GetCachedPresence(*LocalUserId, CurrentPresence);

			FOnlinePresenceSetPresenceParameters NewStatus;
			NewStatus.Properties.Emplace(CurrentPresence.IsValid() ? CurrentPresence->Status.Properties : FPresenceProperties());
			for (TPair<FPresenceKey, FVariantData>& Pair : OnlineProperties)
			{
				NewStatus.Properties->Emplace(MoveTemp(Pair.Key), MoveTemp(Pair.Value));
			}

			PresenceInterface->SetPresence(*LocalUserId, MoveTemp(NewStatus));
		}
	}
}

USocialManager& USocialToolkit::GetSocialManager() const
{
	USocialManager* OuterSocialManager = GetTypedOuter<USocialManager>();
	check(OuterSocialManager);
	return *OuterSocialManager;
}

ULocalPlayer* USocialToolkit::GetOwningLocalPlayerPtr() const
{
	return LocalPlayerOwner.Get();
}

USocialUser* USocialToolkit::FindUser(const FUniqueNetIdRepl& UserId) const
{
	const TWeakObjectPtr<USocialUser>* FoundUser = UsersBySubsystemIds.Find(UserId);
	return FoundUser ? FoundUser->Get() : nullptr;
}

void USocialToolkit::TrySendFriendInvite(const FString& DisplayNameOrEmail) const
{
	IOnlineSubsystem* PrimaryOSS = GetSocialOss(ESocialSubsystem::Primary);
	IOnlineUserPtr UserInterface = PrimaryOSS ? PrimaryOSS->GetUserInterface() : nullptr;
	if (UserInterface.IsValid())
	{
		IOnlineUser::FOnQueryUserMappingComplete QueryCompleteDelegate = IOnlineUser::FOnQueryUserMappingComplete::CreateUObject(const_cast<USocialToolkit*>(this), &USocialToolkit::HandleQueryPrimaryUserIdMappingComplete);
		UserInterface->QueryUserIdMapping(*GetLocalUserNetId(ESocialSubsystem::Primary), DisplayNameOrEmail, QueryCompleteDelegate);
	}
	else
	{
		UE_LOGF(LogParty, Log, "SocialToolkit [%d] failed to execute TrySendFriendInvite.", GetLocalUserNum());
	}
}

bool USocialToolkit::GetAuthAttribute(ESocialSubsystem SubsystemType, const FString& AttributeKey, FString& OutValue) const
{
	IOnlineSubsystem* SocialOSS = GetSocialOss(SubsystemType);
	if (IOnlineIdentityPtr IdentityInterface = SocialOSS ? SocialOSS->GetIdentityInterface() : nullptr)
	{
		FUniqueNetIdRepl LocalUserId = GetLocalUserNetId(SubsystemType);
		if (TSharedPtr<FUserOnlineAccount> UserAccount = LocalUserId.IsValid() ? IdentityInterface->GetUserAccount(*LocalUserId) : nullptr)
		{
			return UserAccount->GetAuthAttribute(AttributeKey, OutValue);
		}
	}
	return false;
}

#if PARTY_PLATFORM_SESSIONS_PSN
void USocialToolkit::NotifyPSNFriendsListRebuilt()
{
	UE_LOGF(LogParty, Log, "SocialToolkit [%d] quietly refreshing PSN FriendInfo on existing users due to an external requery of the friends list.", GetLocalUserNum());

	TArray<TSharedRef<FOnlineFriend>> PSNFriendsList;
	IOnlineFriendsPtr FriendsInterfacePSN = Online::GetFriendsInterfaceChecked(GetWorld(), USocialManager::GetSocialOssName(ESocialSubsystem::Platform));
	FriendsInterfacePSN->GetFriendsList(GetLocalUserNum(), FriendListToQuery, PSNFriendsList);

	// This is a stealth update just to prevent the WeakPtr references to friend info on a given user disappearing out from under the user, so we don't actually want it to fire a real event
	static FOnRelationshipEstablished GarbageRelationshipEstablishedFiller;
	ProcessUserList(PSNFriendsList, ESocialSubsystem::Platform, GarbageRelationshipEstablishedFiller);
}
#endif

void USocialToolkit::NotifyPartyInviteReceived(USocialUser& SocialUser, const IOnlinePartyJoinInfo& Invite)
{
	OnPartyInviteReceived().Broadcast(SocialUser);
}

void USocialToolkit::NotifyPartyInviteRemoved(USocialUser& SocialUser, const IOnlinePartyJoinInfo& Invite)
{
	OnPartyInviteRemoved().Broadcast(SocialUser);
}

void USocialToolkit::QueueUserDependentAction(const FUniqueNetIdRepl& UserId, TFunction<void(USocialUser&)>&& UserActionFunc, bool bExecutePostInit)
{
	ESocialSubsystem CompatibleSubsystem = ESocialSubsystem::MAX;
	if (UserId.IsValid() && NameToSocialSubsystem(UserId.GetType(), CompatibleSubsystem))
	{
		QueueUserDependentActionInternal(UserId, CompatibleSubsystem, MoveTemp(UserActionFunc), bExecutePostInit);
	}
}

void USocialToolkit::QueueUserDependentAction(const FUniqueNetIdRepl& SubsystemId, FUserDependentAction UserActionDelegate)
{
	// MERGE-REVIEW: Was changed from FindOrCreate
	if (USocialUser* SocialUser = FindUser(SubsystemId))
	{
		SocialUser->RegisterInitCompleteHandler(UserActionDelegate);
	}
}

void USocialToolkit::QueueUserDependentActionInternal(const FUniqueNetIdRepl& SubsystemId, ESocialSubsystem SubsystemType, TFunction<void(USocialUser&)>&& UserActionFunc, bool bExecutePostInit)
{
	if (!ensure(SubsystemId.IsValid()))
	{
		return;
	}
	
	USocialUser* User = FindUser(SubsystemId);

	if (!User && ensureMsgf(USocialToolkit::IsOwnerLoggedIn(), TEXT("Cannot QueueUserDependentAction while local user is logged out! Toolkit [%d], ID [%s], Subsystem [%s]"), GetLocalUserNum(), *SubsystemId.ToDebugString(), ToString(SubsystemType)))
	{
		if (SubsystemType == ESocialSubsystem::Primary)
		{
			User = NewObject<USocialUser>(this, SocialUserClass);
			AllUsers.Add(User);
			User->Initialize(SubsystemId);
		}
		else
		{
			// Check to see if this external Id has already been mapped
			IOnlineUserPtr UserInterface = Online::GetUserInterfaceChecked(GetWorld(), USocialManager::GetSocialOssName(ESocialSubsystem::Primary));

			IOnlineSubsystem* OSS = GetSocialOss(SubsystemType);
			check(OSS);
			const FSocialPlatformDescription* PlatformDescription = OSS ? USocialSettings::GetSocialPlatformDescriptionForOnlineSubsystem(OSS->GetSubsystemName()) : nullptr;
			
			FExternalIdQueryOptions QueryOptions;
			QueryOptions.AuthType = PlatformDescription ? PlatformDescription->ExternalAccountType : FString();
			FUniqueNetIdRepl MappedPrimaryId = UserInterface->GetExternalIdMapping(QueryOptions, SubsystemId.ToString());
			if (MappedPrimaryId.IsValid())
			{
				HandleMapExternalIdComplete(SubsystemType, true, MappedPrimaryId, SubsystemId, UserActionFunc, bExecutePostInit);
				return;
			}
			else
			{
				// Gotta map this non-primary Id to the corresponding primary one (if there is one) before we can make a user
				UE_LOGF(LogParty, VeryVerbose, "Mapping primary Id for unknown, unmapped external Id [%ls] for user action", *SubsystemId.ToDebugString());

				FUniqueNetIdRepl LocalUserPrimaryNetId = GetLocalUserNetId(ESocialSubsystem::Primary);
				auto QueryCompleteHandler = FSocialQuery_MapExternalIds::FOnQueryComplete::CreateUObject(this, &USocialToolkit::HandleMapExternalIdComplete, SubsystemId, UserActionFunc, bExecutePostInit);
				FSocialQueryManager::AddUserId<FSocialQuery_MapExternalIds>(*this, SubsystemType, SubsystemId.ToString(), QueryCompleteHandler);
			}
		}
	}

	if (User && UserActionFunc)
	{
		if (User->IsInitialized() || !bExecutePostInit)
		{
			UserActionFunc(*User);
		}
		else
		{
			User->RegisterInitCompleteHandler(FOnNewSocialUserInitialized::CreateLambda(UserActionFunc));
		}
	}
}

void USocialToolkit::HandleControllerIdChanged(int32 NewId, int32 OldId)
{
	IOnlineSubsystem* PrimaryOss = GetSocialOss(ESocialSubsystem::Primary);
	if (const IOnlineIdentityPtr IdentityInterface = PrimaryOss ? PrimaryOss->GetIdentityInterface() : nullptr)
	{
		IdentityInterface->ClearOnLoginCompleteDelegates(OldId, this);
		IdentityInterface->ClearOnLoginStatusChangedDelegates(OldId, this);
		IdentityInterface->ClearOnLogoutCompleteDelegates(OldId, this);

		IdentityInterface->AddOnLoginStatusChangedDelegate_Handle(NewId, FOnLoginStatusChangedDelegate::CreateUObject(this, &USocialToolkit::HandlePlayerLoginStatusChanged));

		if (IdentityInterface->GetLoginStatus(NewId) == ELoginStatus::LoggedIn)
		{
			UE_CLOGF(OldId != INVALID_CONTROLLERID, LogParty, Error, "SocialToolkit updating controller IDs for local player while logged in. That makes no sense! OldId = [%d], NewId = [%d]", OldId, NewId);

			FUniqueNetIdPtr LocalUserId = IdentityInterface->GetUniquePlayerId(NewId);
			if (ensure(LocalUserId))
			{
				HandlePlayerLoginStatusChanged(NewId, ELoginStatus::NotLoggedIn, ELoginStatus::LoggedIn, *LocalUserId);
			}
		}
	}
}

void USocialToolkit::NotifySubsystemIdEstablished(USocialUser& SocialUser, ESocialSubsystem SubsystemType, const FUniqueNetIdRepl& SubsystemId)
{
	if (ensure(!UsersBySubsystemIds.Contains(SubsystemId)))
	{
		UE_LOGF(LogParty, VeryVerbose, "Toolkit [%d] establishing subsystem Id [%ls] for user [%ls]", GetLocalUserNum(), *SubsystemId.ToDebugString(), *SocialUser.ToDebugString());
		UsersBySubsystemIds.Add(SubsystemId, &SocialUser);
	}
	else
	{
		FString LogString = FString::Printf(TEXT("SubsystemId [%s] for user [%s] is already in the UsersBySubsystemIds map.\n"), *SubsystemId.ToDebugString(), *SocialUser.GetName());

		LogString += TEXT("Currently in the map:\n");
		for (const auto& IdUserPair : UsersBySubsystemIds)
		{
			LogString += FString::Printf(TEXT("ID: [%s], User: [%s]\n"), *IdUserPair.Key.ToDebugString(), IdUserPair.Value.IsValid() ? *IdUserPair.Value->GetName() : TEXT("ERROR - INVALID USER!"));
		}

		UE_LOGF(LogParty, Error, "%ls", *LogString);
	}
}

bool USocialToolkit::TrySendFriendInvite(USocialUser& SocialUser, ESocialSubsystem SubsystemType) const
{
	if (SocialUser.GetFriendInviteStatus(SubsystemType) == EInviteStatus::PendingOutbound)
	{
		OnFriendInviteSent().Broadcast(SocialUser, SubsystemType);
		return true;
	}
	else if (!SocialUser.IsFriend(SubsystemType) && !IsFriendshipRestricted(SocialUser, SubsystemType))
	{
		return SendFriendInviteInternal(SocialUser, SubsystemType);
	}
	else
	{
		return false;
	}
}


bool USocialToolkit::SendFriendInviteInternal(USocialUser& SocialUser, ESocialSubsystem SubsystemType) const
{
	IOnlineFriendsPtr FriendsInterface = Online::GetFriendsInterface(GetWorld(), USocialManager::GetSocialOssName(SubsystemType));
	const FUniqueNetIdRepl SocialUserSubsystemId = SocialUser.GetUserId(SubsystemType);
	if (FriendsInterface && SocialUserSubsystemId.IsValid())
	{
		return FriendsInterface->SendInvite(
			GetLocalUserNum(), 
			*SocialUserSubsystemId, 
			FriendListToQuery, 
			FOnSendInviteComplete::CreateUObject(
				const_cast<USocialToolkit*>(this), 
				&USocialToolkit::HandleSendFriendInviteComplete, 
				SubsystemType, 
				SocialUser.GetDisplayName()
			)
		);
	}
	else
	{
		return false;
	}
}


bool USocialToolkit::AcceptFriendInvite(const USocialUser& SocialUser, ESocialSubsystem SubsystemType) const
{
	if (SocialUser.GetFriendInviteStatus(SubsystemType) == EInviteStatus::PendingInbound)
	{
		return AcceptFriendInviteInternal(SocialUser, SubsystemType);
	}
	else
	{
		return false;
	}
}

bool USocialToolkit::AcceptFriendInviteInternal(const USocialUser& SocialUser, ESocialSubsystem SubsystemType) const
{
	IOnlineFriendsPtr FriendsInterface = GetSocialOss(SubsystemType)->GetFriendsInterface();
	check(FriendsInterface.IsValid());
	return FriendsInterface->AcceptInvite(
		GetLocalUserNum(),
		*SocialUser.GetUserId(SubsystemType),
		EFriendsLists::ToString(EFriendsLists::Default),
		FOnAcceptInviteComplete::CreateUObject(
			const_cast<USocialToolkit*>(this),
			&USocialToolkit::HandleAcceptFriendInviteComplete
		)
	);
}

bool USocialToolkit::IsFriendshipRestricted(const USocialUser& SocialUser, ESocialSubsystem SubsystemType) const
{
	return false;
}

//@todo DanH: Rename this in a way that keeps the intent but relates that more than just the primary login has completed (i.e. the game has also completed whatever specific stuff it wants to for login as well)
void USocialToolkit::OnOwnerLoggedIn()
{
	UE_LOGF(LogParty, Log, "LocalPlayer [%d] has logged in - starting up SocialToolkit.", GetLocalUserNum());

	// Establish the owning player's ID on each subsystem and bind to events for general social goings-on
	const int32 LocalUserNum = GetLocalUserNum();
	for (ESocialSubsystem SubsystemType : USocialManager::GetDefaultSubsystems())
	{
		FUniqueNetIdRepl LocalUserNetId = LocalUser->GetUserId(SubsystemType);
		if (LocalUserNetId.IsValid())
		{
			IOnlineSubsystem* OSS = GetSocialOss(SubsystemType);
			check(OSS);
			if (IOnlineFriendsPtr FriendsInterface = OSS->GetFriendsInterface())
			{
				FriendsInterface->AddOnFriendRemovedDelegate_Handle(FOnFriendRemovedDelegate::CreateUObject(this, &USocialToolkit::HandleFriendRemoved, SubsystemType));
				FriendsInterface->AddOnDeleteFriendCompleteDelegate_Handle(LocalUserNum, FOnDeleteFriendCompleteDelegate::CreateUObject(this, &USocialToolkit::HandleDeleteFriendComplete, SubsystemType));
				FriendsInterface->AddOnFriendsChangeDelegate_Handle(LocalUserNum, FOnFriendsChangeDelegate::CreateUObject(this, &USocialToolkit::HandleFriendsChanged, SubsystemType));
 
				FriendsInterface->AddOnInviteReceivedDelegate_Handle(FOnInviteReceivedDelegate::CreateUObject(this, &USocialToolkit::HandleFriendInviteReceived, SubsystemType));
				FriendsInterface->AddOnInviteAcceptedDelegate_Handle(FOnInviteAcceptedDelegate::CreateUObject(this, &USocialToolkit::HandleFriendInviteAccepted, SubsystemType));
				FriendsInterface->AddOnInviteRejectedDelegate_Handle(FOnInviteRejectedDelegate::CreateUObject(this, &USocialToolkit::HandleFriendInviteRejected, SubsystemType));
				FriendsInterface->AddOnInviteAbortedDelegate_Handle(FOnInviteAbortedDelegate::CreateUObject(this, &USocialToolkit::HandleFriendInviteRejected, SubsystemType));

				FriendsInterface->AddOnBlockedPlayerCompleteDelegate_Handle(LocalUserNum, FOnBlockedPlayerCompleteDelegate::CreateUObject(this, &USocialToolkit::HandleBlockPlayerComplete, SubsystemType));
				FriendsInterface->AddOnUnblockedPlayerCompleteDelegate_Handle(LocalUserNum, FOnUnblockedPlayerCompleteDelegate::CreateUObject(this, &USocialToolkit::HandleUnblockPlayerComplete, SubsystemType));
				FriendsInterface->AddOnBlockListChangeDelegate_Handle(LocalUserNum, FOnBlockListChangeDelegate::CreateUObject(this, &USocialToolkit::HandleBlockListChanged, SubsystemType));

				FriendsInterface->AddOnRecentPlayersAddedDelegate_Handle(FOnRecentPlayersAddedDelegate::CreateUObject(this, &USocialToolkit::HandleRecentPlayersAdded, SubsystemType));
			}

			if (IOnlinePartyPtr PartyInterface = OSS->GetPartyInterface())
			{
				PartyInterface->AddOnPartyInviteReceivedExDelegate_Handle(FOnPartyInviteReceivedExDelegate::CreateUObject(this, &USocialToolkit::HandlePartyInviteReceived));
				PartyInterface->AddOnPartyInviteRemovedExDelegate_Handle(FOnPartyInviteRemovedExDelegate::CreateUObject(this, &USocialToolkit::HandlePartyInviteRemoved));
				PartyInterface->AddOnPartyRequestToJoinReceivedDelegate_Handle(FOnPartyRequestToJoinReceivedDelegate::CreateUObject(this, &USocialToolkit::HandlePartyRequestToJoinReceived));
				PartyInterface->AddOnPartyRequestToJoinRemovedDelegate_Handle(FOnPartyRequestToJoinRemovedDelegate::CreateUObject(this, &USocialToolkit::HandlePartyRequestToJoinRemoved));
				PartyInterface->AddOnPartyDataReceivedDelegate_Handle(FOnPartyDataReceivedDelegate::CreateUObject(this, &USocialToolkit::HandlePartyDataReceived));
			}

			if (IOnlinePresencePtr PresenceInterface = OSS->GetPresenceInterface())
			{
				PresenceInterface->AddOnPresenceReceivedDelegate_Handle(FOnPresenceReceivedDelegate::CreateUObject(this, &USocialToolkit::HandlePresenceReceived, SubsystemType));
			}

			if (SubsystemType == ESocialSubsystem::Primary)
			{
				UE::OnlineFramework::FCommonConfig CommonConfig(this);
				if (UE::Online::IOnlineServicesPtr OnlineServices = CommonConfig.GetServices(USocialManager::GetPresenceFrameworkInstance()))
				{
					if (UE::Online::IPresencePtr Presence = OnlineServices->GetPresenceInterface())
					{
						UE::Online::FOnlineEventDelegateHandle OnPresenceUpdatedHandle = Presence->OnPresenceUpdated().Add(this, &USocialToolkit::HandleServicesPresenceUpdated, SubsystemType);
						OnPresenceUpdatedHandles.Emplace(SubsystemType, MoveTemp(OnPresenceUpdatedHandle));
					}
				}

				if (IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("onlineframework.party.shouldreadpresencefromonlineservices")))
				{
					OnReadPresenceV2CVarChanged = CVar->OnChangedDelegate().AddUObject(this, &USocialToolkit::HandlePresenceReadCVarChanged);
				}
			}
		}
	}

	// Now that everything is set up, immediately query whatever we can
	if (bQueryFriendsOnStartup)
	{
		QueryFriendsLists();
	}
	if (bQueryBlockedPlayersOnStartup)
	{
		QueryBlockedPlayers();
	}
	if (bQueryRecentPlayersOnStartup)
	{
		QueryRecentPlayers();
	}
	OnLoginChanged().Broadcast(true);
}

void USocialToolkit::OnOwnerLoggedOut()
{
	UE_LOGF(LogParty, Log, "LocalPlayer [%d] has logged out - wiping user roster from SocialToolkit.", GetLocalUserNum());

	const int32 LocalUserNum = GetLocalUserNum();
	for (ESocialSubsystem SubsystemType : USocialManager::GetDefaultSubsystems())
	{
		if (IOnlineSubsystem* OSS = GetSocialOss(SubsystemType))
		{
			IOnlineFriendsPtr FriendsInterface = OSS->GetFriendsInterface();
			if (FriendsInterface.IsValid())
			{
				FriendsInterface->ClearOnFriendRemovedDelegates(this);
				FriendsInterface->ClearOnDeleteFriendCompleteDelegates(LocalUserNum, this);

				FriendsInterface->ClearOnInviteReceivedDelegates(this);
				FriendsInterface->ClearOnInviteAcceptedDelegates(this);
				FriendsInterface->ClearOnInviteRejectedDelegates(this);
				FriendsInterface->ClearOnInviteAbortedDelegates(this);

				FriendsInterface->ClearOnBlockedPlayerCompleteDelegates(LocalUserNum, this);
				FriendsInterface->ClearOnUnblockedPlayerCompleteDelegates(LocalUserNum, this);
				FriendsInterface->ClearOnBlockListChangeDelegates(LocalUserNum, this);

				FriendsInterface->ClearOnRecentPlayersAddedDelegates(this);

				FriendsInterface->ClearOnQueryBlockedPlayersCompleteDelegates(this);
				FriendsInterface->ClearOnQueryRecentPlayersCompleteDelegates(this);
			}

			IOnlinePartyPtr PartyInterface = OSS->GetPartyInterface();
			if (PartyInterface.IsValid())
			{
				PartyInterface->ClearOnPartyInviteReceivedExDelegates(this);
				PartyInterface->ClearOnPartyInviteRemovedExDelegates(this);
				PartyInterface->ClearOnPartyRequestToJoinReceivedDelegates(this);
				PartyInterface->ClearOnPartyRequestToJoinRemovedDelegates(this);
				PartyInterface->ClearOnPartyDataReceivedDelegates(this);
			}

			IOnlinePresencePtr PresenceInterface = OSS->GetPresenceInterface();
			if (PresenceInterface.IsValid())
			{
				PresenceInterface->ClearOnPresenceReceivedDelegates(this);
			}

			if (OnReadPresenceV2CVarChanged.IsValid())
			{
				if (IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("onlineframework.party.shouldreadpresencefromonlineservices")))
				{
					CVar->OnChangedDelegate().Remove(OnReadPresenceV2CVarChanged);
				}

				OnReadPresenceV2CVarChanged.Reset();
			}

			if (UE::Online::FOnlineEventDelegateHandle* Handle = OnPresenceUpdatedHandles.Find(SubsystemType))
			{
				Handle->Unbind();
			}
		}
	}

	OnPresenceUpdatedHandles.Reset();
	UsersBySubsystemIds.Reset();
	AllUsers.Reset();

	// Remake a fresh uninitialized local user
	LocalUser = NewObject<USocialUser>(this, SocialUserClass);

	OnToolkitReset().Broadcast();
	OnLoginChanged().Broadcast(false);
}

void USocialToolkit::QueryFriendsLists()
{
	for (ESocialSubsystem SubsystemType : USocialManager::GetDefaultSubsystems())
	{
		const FUniqueNetIdRepl LocalUserNetId = LocalUser->GetUserId(SubsystemType);
		if (LocalUserNetId.IsValid())
		{
			IOnlineSubsystem* OSS = GetSocialOss(SubsystemType);
			check(OSS);

			if (IOnlineFriendsPtr FriendsInterface = OSS->GetFriendsInterface())
			{
				FriendsInterface->ReadFriendsList(GetLocalUserNum(), FriendListToQuery, FOnReadFriendsListComplete::CreateUObject(this, &USocialToolkit::HandleReadFriendsListComplete, SubsystemType));
			}
		}
	}
}

void USocialToolkit::QueryBlockedPlayers()
{
	for (ESocialSubsystem SubsystemType : USocialManager::GetDefaultSubsystems())
	{
		const FUniqueNetIdRepl LocalUserSubsystemId = LocalUser->GetUserId(SubsystemType);
		if (LocalUserSubsystemId.IsValid())
		{
			IOnlineSubsystem* OSS = GetSocialOss(SubsystemType);
			check(OSS);

			if (IOnlineFriendsPtr FriendsInterface = OSS->GetFriendsInterface())
			{
				//@todo DanH Social: There is an inconsistency in OSS interfaces - some just return false for unimplemented features while others return false and trigger the callback
				//		Seems like they should return false if the feature isn't implemented and trigger the callback for failure if it is implemented and couldn't start
				//		As it is now, there are two ways to know if the call didn't succeed and zero ways to know if it ever could
				FriendsInterface->AddOnQueryBlockedPlayersCompleteDelegate_Handle(FOnQueryBlockedPlayersCompleteDelegate::CreateUObject(this, &USocialToolkit::HandleQueryBlockedPlayersComplete, SubsystemType));
				if (!FriendsInterface->QueryBlockedPlayers(*LocalUserSubsystemId))
				{
					FriendsInterface->ClearOnQueryBlockedPlayersCompleteDelegates(this);
				}
			}
		}
	}
}

void USocialToolkit::QueryRecentPlayers()
{
	for (ESocialSubsystem SubsystemType : USocialManager::GetDefaultSubsystems())
	{
		const FUniqueNetIdRepl LocalUserSubsystemId = LocalUser->GetUserId(SubsystemType);
		if (LocalUserSubsystemId.IsValid())
		{
			IOnlineSubsystem* OSS = GetSocialOss(SubsystemType);
			check(OSS);

			if (IOnlineFriendsPtr FriendsInterface = OSS->GetFriendsInterface())
			{
				FriendsInterface->AddOnQueryRecentPlayersCompleteDelegate_Handle(FOnQueryRecentPlayersCompleteDelegate::CreateUObject(this, &USocialToolkit::HandleQueryRecentPlayersComplete, SubsystemType));
				if (RecentPlayerNamespaceToQuery.IsEmpty() || !FriendsInterface->QueryRecentPlayers(*LocalUserSubsystemId, RecentPlayerNamespaceToQuery))
				{
					FriendsInterface->ClearOnQueryRecentPlayersCompleteDelegates(this);
				}
			}
		}
	}
}

void USocialToolkit::HandlePlayerLoginStatusChanged(int32 LocalUserNum, ELoginStatus::Type OldStatus, ELoginStatus::Type NewStatus, const FUniqueNetId& NewId)
{
	if (LocalUserNum == GetLocalUserNum())
	{
		if (NewStatus == ELoginStatus::LoggedIn)
		{
			if (AllUsers.Num() != 0)
			{
				UE_LOGF(LogParty, Error, "HandlePlayerLoginStatusChanged: Changed login status but we were not informed their status had changed previously");

				// Nobody told us we logged out! Handle it now just so we're fresh, but not good!
				OnOwnerLoggedOut();
			}

			AllUsers.Add(LocalUser);
			LocalUser->InitLocalUser();

			if (IsOwnerLoggedIn())
			{
				OnOwnerLoggedIn();
			}
		}
		else if (NewStatus == ELoginStatus::NotLoggedIn)
		{
			OnOwnerLoggedOut();
		}
	}
}

void USocialToolkit::HandleReadFriendsListComplete(int32 LocalUserNum, bool bWasSuccessful, const FString& ListName, const FString& ErrorStr, ESocialSubsystem SubsystemType)
{
	UE_LOGF(LogParty, Log, "SocialToolkit [%d] finished querying friends list ListName=[%ls] Subsystem=[%ls] bWasSuccessful=[%ls] Error=[%ls].", GetLocalUserNum(), *ListName, ToString(SubsystemType), *LexToString(bWasSuccessful), *ErrorStr);
	if (bWasSuccessful)
	{
		TArray<TSharedRef<FOnlineFriend>> FriendsList;
		IOnlineFriendsPtr FriendsInterface = Online::GetFriendsInterfaceChecked(GetWorld(), USocialManager::GetSocialOssName(SubsystemType));
		FriendsInterface->GetFriendsList(LocalUserNum, ListName, FriendsList);

		//@todo DanH: This isn't actually quite correct - some of these could actually just be friend info for pending invites, not fully accepted friends. Should piece out the list into respective categories and process each separately (or make the associated event determination more complex)
		ProcessUserList(FriendsList, SubsystemType, OnFriendshipEstablished());
		OnQueryFriendsListSuccess(SubsystemType, FriendsList);
	}
	else
	{
		//@todo DanH: This is a really big deal on primary and a frustrating deal on platform
		// In both cases I think we should give it another shot, but I dunno how long to wait and if we should behave differently between the two
	}

	HandleExistingPartyInvites(SubsystemType);

	OnReadFriendsListComplete(LocalUserNum, bWasSuccessful, ListName, ErrorStr, SubsystemType);
}

void USocialToolkit::HandleQueryBlockedPlayersComplete(const FUniqueNetId& UserId, bool bWasSuccessful, const FString& ErrorStr, ESocialSubsystem SubsystemType)
{
	if (UserId == *GetLocalUserNetId(SubsystemType))
	{
		UE_LOGF(LogParty, Log, "SocialToolkit [%d] finished querying blocked players Subsystem=[%ls] bWasSuccessful=[%ls] Error=[%ls].", GetLocalUserNum(), ToString(SubsystemType), *LexToString(bWasSuccessful), *ErrorStr);

		if (bWasSuccessful)
		{
			IOnlineFriendsPtr FriendsInterface = Online::GetFriendsInterfaceChecked(GetWorld(), USocialManager::GetSocialOssName(SubsystemType));
			FriendsInterface->ClearOnQueryBlockedPlayersCompleteDelegates(this);

			TArray<TSharedRef<FOnlineBlockedPlayer>> BlockedPlayers;
			FriendsInterface->GetBlockedPlayers(UserId, BlockedPlayers);

			ProcessOSSBlockedPlayers(SubsystemType, BlockedPlayers);

			OnQueryBlockedPlayersSuccess(SubsystemType, BlockedPlayers);
		}
		else
		{
			//@todo DanH: Only bother retrying on primary
		}
	}

	OnQueryBlockedPlayersComplete(UserId, bWasSuccessful, ErrorStr, SubsystemType);
}

void USocialToolkit::HandleQueryRecentPlayersComplete(const FUniqueNetId& UserId, const FString& Namespace, bool bWasSuccessful, const FString& ErrorStr, ESocialSubsystem SubsystemType)
{
	if (UserId == *GetLocalUserNetId(SubsystemType))
	{
		UE_LOGF(LogParty, Log, "SocialToolkit [%d] finished querying recent player list Namespace=[%ls] Subsystem=[%ls] bWasSuccessful=[%ls] Error=[%ls].", GetLocalUserNum(), *Namespace, ToString(SubsystemType), *LexToString(bWasSuccessful), *ErrorStr);

		if (bWasSuccessful)
		{
			IOnlineFriendsPtr FriendsInterface = Online::GetFriendsInterfaceChecked(GetWorld(), USocialManager::GetSocialOssName(SubsystemType));
			FriendsInterface->ClearOnQueryRecentPlayersCompleteDelegates(this);

			TArray<TSharedRef<FOnlineRecentPlayer>> RecentPlayers;
			FriendsInterface->GetRecentPlayers(UserId, Namespace, RecentPlayers);
			ProcessUserList(RecentPlayers, SubsystemType, OnRecentPlayerAdded());
			OnQueryRecentPlayersSuccess(SubsystemType, RecentPlayers);
		}
		else
		{
			//@todo DanH: Only bother retrying on primary
		}
	}

	OnQueryRecentPlayersComplete(UserId, Namespace, bWasSuccessful, ErrorStr, SubsystemType);
}

void USocialToolkit::HandleRecentPlayersAdded(const FUniqueNetId& LocalUserId, const TArray<TSharedRef<FOnlineRecentPlayer>>& NewRecentPlayers, ESocialSubsystem SubsystemType)
{
	if (LocalUserId == *GetLocalUserNetId(SubsystemType))
	{
		for (const TSharedRef<FOnlineRecentPlayer>& RecentPlayerInfo : NewRecentPlayers)
		{
			QueueUserDependentActionInternal(RecentPlayerInfo->GetUserId(), SubsystemType,
				[this, RecentPlayerInfo, SubsystemType] (USocialUser& User)
				{
					User.EstablishOssInfo(RecentPlayerInfo, SubsystemType);
					OnRecentPlayerAdded().Broadcast(User, SubsystemType, true);
				});
		}
	}
}

void USocialToolkit::HandleMapExternalIdComplete(ESocialSubsystem SubsystemType, bool bWasSuccessful, const FUniqueNetIdRepl& MappedPrimaryId, FUniqueNetIdRepl ExternalId, TFunction<void(USocialUser&)> UserActionFunc, bool bExecutePostInit)
{
	if (bWasSuccessful && MappedPrimaryId.IsValid())
	{
		QueueUserDependentActionInternal(MappedPrimaryId, ESocialSubsystem::Primary,
			[this, SubsystemType, ExternalId, UserActionFunc] (USocialUser& User)
			{
				// Make sure the primary user info agreed about the external Id
				if (ensure(User.GetUserId(SubsystemType) == ExternalId) && UserActionFunc)
				{
					UserActionFunc(User);
				}
			}
			//@todo DanH: Since we're relying on the primary UserInfo as the authority here, platform ID-based queued actions always execute post-init. Revisit this #future
			/*, bExecutePostInit*/);
	}
}

void USocialToolkit::HandleServicesPresenceUpdated(const UE::Online::FPresenceUpdated& Event, ESocialSubsystem SubsystemType)
{
	if (USocialManager::ShouldReadPresenceFromOnlineServices())
	{
		using namespace UE::Online;
		using namespace UE::OnlineFramework;

		TSharedRef<const FUserPresence> UpdatedPresence = Event.UpdatedPresence;

		FCommonConfig CommonConfig(this);
	
		if (FCommonAccountPtr CommonAccountPtr = GetCommonAccountFromV2(CommonConfig, UpdatedPresence->AccountId, USocialManager::GetPresenceFrameworkInstance()))
		{
			// We need a services instance with an Auth adapter to make this conversion
			if (FUniqueNetIdPtr UserId = GetV1FromCommonAccount(CommonAccountPtr.ToSharedRef(), USocialManager::GetPresenceFrameworkAdapterInstance()))
			{
				QUICK_SCOPE_CYCLE_COUNTER(STAT_USocialToolkit_HandlePresenceReceived);
				if (USocialUser* UpdatedUser = FindUser(UserId.ToSharedRef()))
				{
					UpdatedUser->NotifyPresenceChanged(SubsystemType);

					if (UpdatedUser->IsFriend())
					{
						QueueUserDependentActionInternal(UserId.ToSharedRef(), ESocialSubsystem::Primary,
							[this, UpdatedPresence](USocialUser& SocialUser)
							{
								OnFriendPresenceDidChangeV2(SocialUser, UpdatedPresence, ESocialSubsystem::Primary);
							});
					}
				}
				else if (SubsystemType == ESocialSubsystem::Platform)
				{
					UE_LOGF(LogParty, Error, "Platform presence received for UserId [%ls], but existing SocialUser could not be found.", *UserId->ToDebugString());
				}
			}
		}
	}
	else
	{
		NonReadPresencesReceivedV2.Emplace(Event.UpdatedPresence->AccountId);
	}
}

void USocialToolkit::HandlePresenceReceived(const FUniqueNetId& UserId, const TSharedRef<FOnlineUserPresence>& NewPresence, ESocialSubsystem SubsystemType)
{
	if (!USocialManager::ShouldReadPresenceFromOnlineServices() || SubsystemType == ESocialSubsystem::Platform)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_USocialToolkit_HandlePresenceReceived);
		if (USocialUser* UpdatedUser = FindUser(UserId.AsShared()))
		{
			UpdatedUser->NotifyPresenceChanged(SubsystemType);

			if (UpdatedUser->IsFriend())
			{
				QueueUserDependentActionInternal(UserId.AsShared(), ESocialSubsystem::Primary,
					[this, NewPresence](USocialUser& SocialUser)
					{
						OnFriendPresenceDidChange(SocialUser, NewPresence, ESocialSubsystem::Primary);
					});
			}
		}
		else if (SubsystemType == ESocialSubsystem::Platform)
		{
			UE_LOGF(LogParty, Error, "Platform presence received for UserId [%ls], but existing SocialUser could not be found.", *UserId.ToDebugString());
		}
	}
	else
	{
		NonReadPresencesReceivedV1.Emplace(UserId.AsShared());
	}
}

void USocialToolkit::HandlePresenceReadCVarChanged(IConsoleVariable*)
{
	if (USocialManager::ShouldReadPresenceFromOnlineServices())
	{
		using namespace UE::Online;
		using namespace UE::OnlineFramework;

		FCommonConfig CommonConfig(this);
		IOnlineServicesPtr OnlineServices = CommonConfig.GetServices(USocialManager::GetPresenceFrameworkInstance());
		IPresencePtr Presence = OnlineServices ? OnlineServices->GetPresenceInterface() : nullptr;
		FUniqueNetIdRepl LocalUserIdV1 = LocalUser ? LocalUser->GetUserId(ESocialSubsystem::Primary) : nullptr;
		const FCommonAccountPtr LocalUserCommonAccount = LocalUserIdV1.IsValid() ? GetCommonAccountFromV1(CommonConfig, LocalUserIdV1.GetV1(), GetFirstFrameworkInstanceName(*LocalUserIdV1)) : nullptr;
		const FAccountId LocalUserAccountId = LocalUserCommonAccount ? LocalUserCommonAccount->GetId(USocialManager::GetPresenceFrameworkInstance()) : FAccountId();
		if (Presence && LocalUserAccountId.IsValid())
		{
			const TSet<FAccountId> PreviouslyUpdatedPresences = MoveTemp(NonReadPresencesReceivedV2);
			NonReadPresencesReceivedV2.Empty();
			for (const FAccountId TargetAccountId : PreviouslyUpdatedPresences)
			{
				TOnlineResult<FGetCachedPresence> Result = Presence->GetCachedPresence({
					.LocalAccountId = LocalUserAccountId,
					.TargetAccountId = TargetAccountId
				});

				if (Result.IsOk())
				{
					HandleServicesPresenceUpdated({
						.LocalAccountId = LocalUserAccountId,
						.UpdatedPresence = Result.GetOkValue().Presence
					}, ESocialSubsystem::Primary);
				}
			}
		}
	}
	else 
	{
		if (IOnlineSubsystem* Oss = GetSocialOss(ESocialSubsystem::Primary))
		{
			IOnlinePresencePtr Presence = Oss->GetPresenceInterface();
			FUniqueNetIdRepl LocalUserId = LocalUser ? LocalUser->GetUserId(ESocialSubsystem::Primary) : nullptr;
			if (Presence && LocalUserId.IsValid())
			{
				const FUniqueNetIdSet PreviouslyUpdatedNetIds = MoveTemp(NonReadPresencesReceivedV1);
				NonReadPresencesReceivedV1.Empty();
				for (const FUniqueNetIdRef& TargetNetId : PreviouslyUpdatedNetIds)
				{
					TSharedPtr<FOnlineUserPresence> UserPresence;
					EOnlineCachedResult::Type Result = Presence->GetCachedPresenceForApp(*LocalUserId, *TargetNetId, Oss->GetAppId(), UserPresence);
					if (UserPresence)
					{
						HandlePresenceReceived(*TargetNetId, UserPresence.ToSharedRef(), ESocialSubsystem::Primary);
					}
				}
			}
		}
	}
}

void USocialToolkit::HandleQueryPrimaryUserIdMappingComplete(bool bWasSuccessful, const FUniqueNetId& RequestingUserId, const FString& DisplayName, const FUniqueNetId& IdentifiedUserId, const FString& Error)
{
	if (!IdentifiedUserId.IsValid())
	{
		OnSendFriendInviteComplete(IdentifiedUserId, DisplayName, false, FriendInviteFailureReason::InviteFailReason_NotFound);
	}
	else if (RequestingUserId == IdentifiedUserId)
	{
		OnSendFriendInviteComplete(IdentifiedUserId, DisplayName, false, FriendInviteFailureReason::InviteFailReason_AddingSelfFail);
	}
	else
	{
		QueueUserDependentActionInternal(IdentifiedUserId.AsShared(), ESocialSubsystem::Primary,
			[this, DisplayName] (USocialUser& SocialUser)
			{
				if (SocialUser.IsBlocked())
				{
					OnSendFriendInviteComplete(*SocialUser.GetUserId(ESocialSubsystem::Primary), DisplayName, false, FriendInviteFailureReason::InviteFailReason_AddingBlockedFail);
				}
				else if (SocialUser.IsFriend(ESocialSubsystem::Primary))
				{
					OnSendFriendInviteComplete(*SocialUser.GetUserId(ESocialSubsystem::Primary), DisplayName, false, FriendInviteFailureReason::InviteFailReason_AlreadyFriends);
				}
				else
				{
					TrySendFriendInvite(SocialUser, ESocialSubsystem::Primary);
				}
			});
	}
}

void USocialToolkit::HandleFriendInviteReceived(const FUniqueNetId& LocalUserId, const FUniqueNetId& SenderId, ESocialSubsystem SubsystemType)
{
	if (LocalUserId == *GetLocalUserNetId(SubsystemType))
	{
		QueueUserDependentActionInternal(SenderId.AsShared(), SubsystemType,
			[this, SubsystemType] (USocialUser& SocialUser)
			{
				//@todo DanH: This event should send the name of the list the accepting friend is on, shouldn't it?
				IOnlineFriendsPtr FriendsInterface = Online::GetFriendsInterfaceChecked(GetWorld(), USocialManager::GetSocialOssName(SubsystemType));
				if (TSharedPtr<FOnlineFriend> OssFriend = FriendsInterface->GetFriend(GetLocalUserNum(), *SocialUser.GetUserId(SubsystemType), FriendListToQuery))
				{
					SocialUser.EstablishOssInfo(OssFriend.ToSharedRef(), SubsystemType);
					if (ensure(SocialUser.GetFriendInviteStatus(SubsystemType) == EInviteStatus::PendingInbound))
					{
						OnFriendInviteReceived().Broadcast(SocialUser, SubsystemType);
					}
				}
			});
	}
}

void USocialToolkit::HandleFriendInviteAccepted(const FUniqueNetId& LocalUserId, const FUniqueNetId& FriendId, ESocialSubsystem SubsystemType)
{
	if (LocalUserId == *GetLocalUserNetId(SubsystemType))
	{
		QueueUserDependentActionInternal(FriendId.AsShared(), SubsystemType,
			[this, SubsystemType] (USocialUser& SocialUser)
			{
				//@todo DanH: This event should send the name of the list the accepting friend is on, shouldn't it?
				IOnlineFriendsPtr FriendsInterface = Online::GetFriendsInterfaceChecked(GetWorld(), USocialManager::GetSocialOssName(SubsystemType));
				if (TSharedPtr<FOnlineFriend> OssFriend = FriendsInterface->GetFriend(GetLocalUserNum(), *SocialUser.GetUserId(SubsystemType), FriendListToQuery))
				{
					SocialUser.EstablishOssInfo(OssFriend.ToSharedRef(), SubsystemType);
					if (SocialUser.IsFriend(SubsystemType))
					{
						OnFriendshipEstablished().Broadcast(SocialUser, SubsystemType, true);
					}
				}
			});
	}
}

void USocialToolkit::HandleFriendInviteRejected(const FUniqueNetId& LocalUserId, const FUniqueNetId& FriendId, ESocialSubsystem SubsystemType)
{
	if (LocalUserId == *GetLocalUserNetId(SubsystemType))
	{
		if (USocialUser* InvitedUser = FindUser(FriendId.AsShared()))
		{
			InvitedUser->NotifyFriendInviteRemoved(SubsystemType);
		}
	}
}

void USocialToolkit::HandleSendFriendInviteComplete(int32 LocalUserNum, bool bWasSuccessful, const FUniqueNetId& InvitedUserId, const FString& ListName, const FString& ErrorStr, ESocialSubsystem SubsystemType, FString DisplayName)
{
	if (bWasSuccessful)
	{
		QueueUserDependentActionInternal(InvitedUserId.AsShared(), SubsystemType,
			[this, SubsystemType, ListName, LocalUserNum] (USocialUser& SocialUser)
			{
				IOnlineFriendsPtr FriendsInterface = Online::GetFriendsInterfaceChecked(GetWorld(), USocialManager::GetSocialOssName(SubsystemType));
				if (TSharedPtr<FOnlineFriend> OssFriend = FriendsInterface->GetFriend(LocalUserNum, *SocialUser.GetUserId(SubsystemType), ListName))
				{
					SocialUser.EstablishOssInfo(OssFriend.ToSharedRef(), SubsystemType);
					if (SocialUser.GetFriendInviteStatus(SubsystemType) == EInviteStatus::PendingOutbound)
					{
						OnFriendInviteSent().Broadcast(SocialUser, SubsystemType);
					}
				}
			});
	}
	OnSendFriendInviteComplete(InvitedUserId, DisplayName, bWasSuccessful, ErrorStr);
}

void USocialToolkit::HandleFriendRemoved(const FUniqueNetId& LocalUserId, const FUniqueNetId& FriendId, ESocialSubsystem SubsystemType)
{
	if (LocalUserId == *GetLocalUserNetId(SubsystemType))
	{
		USocialUser* FormerFriend = FindUser(FriendId.AsShared());
		if (ensure(FormerFriend))
		{
			FormerFriend->NotifyUserUnfriended(SubsystemType);
		}
	}
}
void USocialToolkit::HandleFriendsChanged(ESocialSubsystem SubsystemType)
{
	UE_LOGF(LogParty, Log, "SocialToolkit [%d] quietly refreshing Friend list due to an external requery of the friends list.", GetLocalUserNum());

	TArray<TSharedRef<FOnlineFriend>> FriendsList;
	IOnlineFriendsPtr FriendsInterface = Online::GetFriendsInterfaceChecked(GetWorld(), USocialManager::GetSocialOssName(SubsystemType));
	FriendsInterface->GetFriendsList(GetLocalUserNum(), FriendListToQuery, FriendsList);
	ProcessUserList(FriendsList, SubsystemType, OnFriendshipEstablished());
}
void USocialToolkit::HandleDeleteFriendComplete(int32 InLocalUserNum, bool bWasSuccessful, const FUniqueNetId& DeletedFriendId, const FString& ListName, const FString& ErrorStr, ESocialSubsystem SubsystemType)
{
	if (bWasSuccessful && InLocalUserNum == GetLocalUserNum())
	{
		USocialUser* FormerFriend = FindUser(DeletedFriendId.AsShared());
		if (ensure(FormerFriend))
		{
			FormerFriend->NotifyUserUnfriended(SubsystemType);
		}
	}

	OnDeleteFriendComplete(InLocalUserNum, bWasSuccessful, DeletedFriendId, ListName, ErrorStr, SubsystemType);
}

void USocialToolkit::HandleAcceptFriendInviteComplete(int32 LocalUserNum, bool bWasSuccessful, const FUniqueNetId& InviterUserId, const FString& ListName, const FString& ErrorStr)
{
	OnAcceptFriendInviteComplete(InviterUserId, bWasSuccessful, ErrorStr);
}

const bool USocialToolkit::IsInviteAllowedFromUser(const USocialUser& User, const TSharedRef<const IOnlinePartyJoinInfo>& InvitePtr) const
{
  return User.IsFriend();
}

void USocialToolkit::HandlePartyInviteReceived(const FUniqueNetId& LocalUserId, const IOnlinePartyJoinInfo& Invite)
{
	if (LocalUserId == GetLocalUserNetId(ESocialSubsystem::Primary))
	{
		PartyInvitations.Add(Invite.AsShared());

		// We really should know about the sender of the invite already, but queue it up in case we receive it during initial setup
		QueueUserDependentActionInternal(Invite.GetSourceUserId(), ESocialSubsystem::Primary,
			[this, Invite = Invite.AsShared()] (USocialUser& User)
			{
				if (IsInviteAllowedFromUser(User, Invite))
				{
#if PARTY_PLATFORM_INVITE_PERMISSIONS
					CanReceiveInviteFrom(User, Invite, [this, Invite, UserId = User.GetUserId(ESocialSubsystem::Primary)](const bool bResult)
					{
						// Check whether the invitation was removed while the async check was completing.
						if (PartyInvitations.Find(Invite) == nullptr)
						{
							UE_LOGF(LogParty, Log, "USocialToolkit::HandlePartyInviteReceived Invitation is no longer valid. LocalUser=[%ls] Inviter=[%ls]",
								*GetLocalUserNetId(ESocialSubsystem::Primary).ToDebugString(), *UserId.ToDebugString());
							return;
						}

						UE_LOGF(LogParty, Log, "USocialToolkit::HandlePartyInviteReceived LocalUser=[%ls] Inviter=[%ls] CanReceiveInviteFrom=[%ls]",
							*GetLocalUserNetId(ESocialSubsystem::Primary).ToDebugString(), *UserId.ToDebugString(), *LexToString(bResult));

						USocialUser* User = FindUser(UserId);
						if (bResult && User)
						{
							User->HandlePartyInviteReceived(*Invite);
						}
					});
#else
					User.HandlePartyInviteReceived(*Invite);
#endif
				}
			});
	}
}

void USocialToolkit::HandlePartyInviteRemoved(const FUniqueNetId& LocalUserId, const IOnlinePartyJoinInfo& Invite, EPartyInvitationRemovedReason Reason)
{
	if (LocalUserId == GetLocalUserNetId(ESocialSubsystem::Primary))
	{
		PartyInvitations.Remove(Invite.AsShared());

		if (USocialUser* User = FindUser(Invite.GetSourceUserId()))
		{
			User->HandlePartyInviteRemoved(Invite, Reason);
		}
	}
}

void USocialToolkit::HandlePartyDataReceived(const FUniqueNetId& LocalUserId, const FOnlinePartyId& PartyId, const FName& Namespace, const FOnlinePartyData& PartyData)
{
	if (bQueryRecentPlayersAfterPartyUpdates)
	{
		if (LocalUserId == GetLocalUserNetId(ESocialSubsystem::Primary))
		{
			bool RecentPlayerListRequiresUpdate = false;
			if (const USocialParty* SocialParty = GetSocialManager().GetParty(PartyId))
			{
				for (const UPartyMember* Member : SocialParty->GetPartyMembers())
				{
					const USocialUser& SocialUser = Member->GetSocialUser();
					const FUniqueNetIdRepl SocialUserId = SocialUser.GetUserId(ESocialSubsystem::Primary);
					if (!SocialUserId.IsValid() || *SocialUserId == LocalUserId)
					{
						continue;
					}
					if (!SocialUser.IsFriend() && !SocialUser.IsRecentPlayer())
					{
						RecentPlayerListRequiresUpdate = true;
						break;
					}
				}
			}

			if (RecentPlayerListRequiresUpdate)
			{
				// Get the latest from the IOnlineFriends system
				QueryRecentPlayers();
			}
		}
	}
}

void USocialToolkit::HandleBlockPlayerComplete(int32 LocalUserNum, bool bWasSuccessful, const FUniqueNetId& BlockedPlayerId, const FString& ListName, const FString& ErrorStr, ESocialSubsystem SubsystemType)
{
	if (bWasSuccessful && LocalUserNum == GetLocalUserNum())
	{
		QueueUserDependentActionInternal(BlockedPlayerId.AsShared(), SubsystemType, 
			[this, SubsystemType] (USocialUser& User)
			{
				// Quite frustrating that the event doesn't sent the FOnlineBlockedPlayer in the first place or provide a direct getter on the interface...
				TArray<TSharedRef<FOnlineBlockedPlayer>> AllBlockedPlayers;
				IOnlineFriendsPtr FriendsInterface = Online::GetFriendsInterfaceChecked(GetWorld(), USocialManager::GetSocialOssName(SubsystemType));
				if (FriendsInterface->GetBlockedPlayers(*GetLocalUserNetId(SubsystemType), AllBlockedPlayers))
				{
					FUniqueNetIdRepl BlockedUserId = User.GetUserId(SubsystemType);
					const TSharedRef<FOnlineBlockedPlayer>* BlockedPlayerInfoPtr = AllBlockedPlayers.FindByPredicate(
						[&BlockedUserId] (TSharedRef<FOnlineBlockedPlayer> BlockedPlayerInfo)
						{
							return *BlockedPlayerInfo->GetUserId() == *BlockedUserId;
						});

					if (BlockedPlayerInfoPtr)
					{
						UE_LOGF(LogParty, Log, "%ls Block player %ls complete, Establishing Oss info...", ANSI_TO_TCHAR(__FUNCTION__), *BlockedUserId.ToDebugString());
						User.EstablishOssInfo(*BlockedPlayerInfoPtr, SubsystemType);
						OnUserBlocked().Broadcast(User, SubsystemType, true);
					}
				}
			});
	}

	OnBlockPlayerComplete(LocalUserNum, bWasSuccessful, BlockedPlayerId, ListName, ErrorStr, SubsystemType);
}

void USocialToolkit::HandleUnblockPlayerComplete(int32 LocalUserNum, bool bWasSuccessful, const FUniqueNetId& UnblockedPlayerId, const FString& ListName, const FString& ErrorStr, ESocialSubsystem SubsystemType)
{
	if (bWasSuccessful && LocalUserNum == GetLocalUserNum())
	{
		USocialUser* UnblockedUser = FindUser(UnblockedPlayerId.AsShared());
		if (ensure(UnblockedUser))
		{
			UnblockedUser->NotifyUserUnblocked(SubsystemType);
		}
	}

	OnUnblockPlayerComplete(LocalUserNum, bWasSuccessful, UnblockedPlayerId, ListName, ErrorStr, SubsystemType);
}

void USocialToolkit::HandleBlockListChanged(int32 LocalUserNum, const FString& ListName, ESocialSubsystem SubsystemType)
{
	if (LocalUserNum == GetLocalUserNum())
	{
		UE_LOGF(LogParty, Log, "SocialToolkit [%d] Subsystem=[%ls] Blocklist changed", GetLocalUserNum(), ToString(SubsystemType));

		TArray<TSharedRef<FOnlineBlockedPlayer>> BlockedPlayers;
		IOnlineFriendsPtr FriendsInterface = Online::GetFriendsInterfaceChecked(GetWorld(), USocialManager::GetSocialOssName(SubsystemType));
		FriendsInterface->GetBlockedPlayers(*GetLocalUserNetId(SubsystemType), BlockedPlayers);

		ProcessOSSBlockedPlayers(SubsystemType, BlockedPlayers);
	}
}

void USocialToolkit::ProcessOSSBlockedPlayers(ESocialSubsystem SubsystemType, const TArray<TSharedRef<FOnlineBlockedPlayer>>& InBlockedPlayers)
{
	TSet<FUniqueNetIdRepl> InBlockedPlayerNetIds;
	Algo::Transform(InBlockedPlayers, InBlockedPlayerNetIds, [](const TSharedRef<FOnlineBlockedPlayer>& BlockedPlayer) { return BlockedPlayer->GetUserId(); });

	TSet<FUniqueNetIdRepl>& CurrentBlockedUserNetIds = BlockedUsers.FindOrAdd(SubsystemType);
	TSet<FUniqueNetIdRepl> UnblockedNetIds = CurrentBlockedUserNetIds.Difference(InBlockedPlayerNetIds);

	UE_LOGF(LogParty, Log, "SocialToolkit [%d] Subsystem=[%ls] BlockedPlayers=[%ls] UnblockedPlayers=[%ls]",
		GetLocalUserNum(),
		LexToString(SubsystemType),
		*FString::JoinBy(InBlockedPlayerNetIds, TEXT(","), &FUniqueNetIdRepl::ToDebugString),
		*FString::JoinBy(UnblockedNetIds, TEXT(","), &FUniqueNetIdRepl::ToDebugString));

	CurrentBlockedUserNetIds = MoveTemp(InBlockedPlayerNetIds);

	for (const FUniqueNetIdRepl& UnblockedNetId : UnblockedNetIds)
	{
		if (TWeakObjectPtr<USocialUser>* WeakUnblockedUser = UsersBySubsystemIds.Find(UnblockedNetId))
		{
			if (USocialUser* UnblockedUser = WeakUnblockedUser->Get())
			{
				UnblockedUser->NotifyUserUnblocked(SubsystemType);
			}
		}
	}

	ProcessUserList(InBlockedPlayers, SubsystemType, OnUserBlocked());
}

//@todo DanH recent players: Where is the line for this between backend and game to update this stuff? #required
//		Seems like I should just be able to get an event for OnRecentPlayersAdded or even a full OnRecentPlayersListRefreshed from IOnlineFriends
void USocialToolkit::HandlePartyMemberExited(const FUniqueNetId& LocalUserId, const FOnlinePartyId& PartyId, const FUniqueNetId& MemberId, const EMemberExitedReason Reason)
{
	// If the party member wasn't a friend, they're now a recent player
}

void USocialToolkit::HandleGameDestroyed(const FName SessionName, bool bWasSuccessful)
{
	// Update the recent player list whenever a game session ends
}

void USocialToolkit::HandleUserInvalidated(USocialUser& InvalidUser)
{
	AllUsers.Remove(&InvalidUser);
	// Remove user from id maps
	if (bRemoveInvalidatedUserFromMaps)
	{
		for (const ESocialSubsystem& SocialSubsystemType : InvalidUser.GetRelevantSubsystems())
		{
			FUniqueNetIdRepl SubsystemUserId = InvalidUser.GetUserId(SocialSubsystemType);
			if (SubsystemUserId.IsValid())
			{
				UsersBySubsystemIds.Remove(SubsystemUserId);
			}
		}
	}
	OnSocialUserInvalidated().Broadcast(InvalidUser);
}

void USocialToolkit::HandleExistingPartyInvites(ESocialSubsystem SubsystemType)
{
	if (SubsystemType == ESocialSubsystem::Primary)
	{
		if (IOnlineSubsystem* Oss = GetSocialOss(SubsystemType))
		{
			IOnlinePartyPtr PartyInterface = Oss->GetPartyInterface();
			FUniqueNetIdRepl LocalUserId = GetLocalUserNetId(SubsystemType);
			if (PartyInterface.IsValid() && LocalUserId.IsValid())
			{
				TArray<IOnlinePartyJoinInfoConstRef> PendingInvites;
				PartyInterface->GetPendingInvites(*LocalUserId, PendingInvites);
				for (const IOnlinePartyJoinInfoConstRef& PendingInvite : PendingInvites)
				{
					HandlePartyInviteReceived(*LocalUserId, *PendingInvite);
				}
			}
		}
	}
}

void USocialToolkit::HandlePartyRequestToJoinReceived(const FUniqueNetId& LocalUserId, const FOnlinePartyId& PartyId, const FUniqueNetId& RequesterId, const IOnlinePartyRequestToJoinInfo& Request)
{
	if (LocalUserId == GetLocalUserNetId(ESocialSubsystem::Primary))
	{
		QueueUserDependentActionInternal(RequesterId.AsShared(), ESocialSubsystem::Primary,
			[this, RequestRef = Request.AsShared()](USocialUser& User)
			{
				// The requesting user won't know they're blocked, so we can't prevent their request from being sent.
				// Instead, ignore the request at the receiving end.
				if (!User.IsBlocked())
				{
					User.HandleRequestToJoinReceived(*RequestRef);
					OnPartyRequestToJoinReceived().Broadcast(User, RequestRef);
				}
				else
				{
					UE_LOGF(LogParty, VeryVerbose, "%ls - Join request from blocked user [%ls] ignored", ANSI_TO_TCHAR(__FUNCTION__), *User.GetUserId(ESocialSubsystem::Primary).ToDebugString());
				}
			});
	}
}

void USocialToolkit::HandlePartyRequestToJoinRemoved(const FUniqueNetId& LocalUserId, const FOnlinePartyId& PartyId, const FUniqueNetId& RequesterId, const IOnlinePartyRequestToJoinInfo& Request, EPartyRequestToJoinRemovedReason Reason)
{
	if (LocalUserId == GetLocalUserNetId(ESocialSubsystem::Primary))
	{
		QueueUserDependentActionInternal(RequesterId.AsShared(), ESocialSubsystem::Primary,
			[this, RequestRef = Request.AsShared(), Reason](USocialUser& User)
			{
				User.HandleRequestToJoinRemoved(*RequestRef, Reason);
				OnPartyRequestToJoinRemoved().Broadcast(User, RequestRef, Reason);
			});
	}
}

#if  WITH_EDITOR || !UE_BUILD_SHIPPING
void USocialToolkit::Debug_OnStartRandomizeUserPresence(uint8 NumRandomUser, float TickerTimer)
{
	if (Debug_PresenceTickerHandle.IsValid())
	{
		FTSTicker::RemoveTicker(Debug_PresenceTickerHandle);
		Debug_PresenceTickerHandle.Reset();
	}

	if (ensure(TickerTimer > 0.f))
	{
		bDebug_IsRandomlyChangingUserPresence = true;
		Debug_PresenceTickerHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateUObject(this, &USocialToolkit::Debug_HandleRandomizeUserPresenceTick, NumRandomUser), TickerTimer);
	}
}

void USocialToolkit::Debug_OnStopRandomizeUserPresence(bool bClearGeneratedPresence)
{
	bDebug_IsRandomlyChangingUserPresence = false;
	if (Debug_PresenceTickerHandle.IsValid())
	{
		FTSTicker::RemoveTicker(Debug_PresenceTickerHandle);
		Debug_PresenceTickerHandle.Reset();
	}

	if (bClearGeneratedPresence)
	{
		// Refresh all existing presence data to revert them back to normal
		for (USocialUser* User : AllUsers)
		{
			User->bDebug_IsPresenceArtificial = false;
			User->NotifyPresenceChanged(ESocialSubsystem::Primary);
		}
	}
}

bool USocialToolkit::Debug_HandleRandomizeUserPresenceTick(float DeltaTime, uint8 NumRandomUser)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_USocialToolkit_Debug_HandleRandomizeUserPresenceTick);
	Debug_ChangeRandomUserPresence(NumRandomUser);
	return true;
}

void USocialToolkit::Debug_ChangeRandomUserPresence(uint8 NumRandomUser)
{
	const TArray<USocialUser*> SocialUsers = GetAllUsers();
	if (SocialUsers.Num() > 0)
	{
		for (int32 i = 0; i < NumRandomUser; i++)
		{
			int32 UserIndex = FMath::RandRange(0, SocialUsers.Num() - 1);
			if (USocialUser* SocialUser = SocialUsers[UserIndex])
			{
				SocialUser->Debug_RandomizePresence();
				SocialUser->NotifyPresenceChanged(ESocialSubsystem::Primary);
			}
		}
	}
}
#endif
