// Copyright Epic Games, Inc. All Rights Reserved.

#include "SocialTypes.h"
#include "OnlineSubsystem.h"
#include "PartyModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SocialTypes)

//////////////////////////////////////////////////////////////////////////
// FUserPlatform
//////////////////////////////////////////////////////////////////////////

#define PLATFORM_NAME_MOBILE	TEXT("MOBILE")
#define PLATFORM_NAME_DESKTOP	TEXT("DESKTOP")
#define PLATFORM_NAME_CONSOLE	TEXT("CONSOLE")

FUserPlatform::FUserPlatform()
{

}

FUserPlatform::FUserPlatform(const FString& InPlatform)
{
	const TArray<FSocialPlatformDescription>& SocialPlatformDescriptions = USocialSettings::GetSocialPlatformDescriptions();
	for (const FSocialPlatformDescription& Entry : SocialPlatformDescriptions)
	{
		if (Entry.Name == InPlatform)
		{
			PlatformDescription = Entry;
			break;
		}
	}

	ensureMsgf(IsValid(), TEXT("[FUserPlatform] PlatformStr [%s] is not valid."), *InPlatform);
}

bool FUserPlatform::operator==(const FString& OtherStr) const
{
	return PlatformDescription.Name == OtherStr;
}

bool FUserPlatform::operator==(const FUserPlatform& Other) const
{
	return PlatformDescription.Name == Other.PlatformDescription.Name;
}

bool FUserPlatform::IsValid() const
{
	return !PlatformDescription.Name.IsEmpty();
}

bool FUserPlatform::IsDesktop() const
{
	return PlatformDescription.PlatformType == PLATFORM_NAME_DESKTOP;
}

bool FUserPlatform::IsMobile() const
{
	return PlatformDescription.PlatformType == PLATFORM_NAME_MOBILE;
}

bool FUserPlatform::IsConsole() const
{
	return PlatformDescription.PlatformType == PLATFORM_NAME_CONSOLE;
}

bool FUserPlatform::IsCrossplayWith(const FUserPlatform& OtherPlatform) const
{
	return PlatformDescription.CrossplayPool != OtherPlatform.PlatformDescription.CrossplayPool;
}

bool FUserPlatform::IsCrossplayWith(const FString& OtherPlatformStr) const
{
	return IsCrossplayWith(FUserPlatform(OtherPlatformStr));
}

bool FUserPlatform::IsCrossplayWithLocalPlatform() const
{
	return IsCrossplayWith(IOnlineSubsystem::GetLocalPlatformName());
}

//////////////////////////////////////////////////////////////////////////
// FSocialActionTimeTracker
//////////////////////////////////////////////////////////////////////////

void FSocialActionTimeTracker::BeginStep(FName StepName)
{
	UE_LOGF(LogParty, VeryVerbose, "Beginning social action step [%ls]", *StepName.ToString());

	FSocialActionStep& NewStep = ActionSteps[ActionSteps.Add(FSocialActionStep())];
	NewStep.StepName = StepName;
}

void FSocialActionTimeTracker::CompleteStep(FName StepName)
{
	//@todo DanH Social: Enforce that this is the currently running step? #suggested
	if (FSocialActionStep* Step = ActionSteps.FindByKey(StepName))
	{
		Step->EndTime = FPlatformTime::Seconds();

		UE_LOGF(LogParty, VeryVerbose, "Finished social action step [%ls] in %.2fs", *StepName.ToString(), Step->GetDurationMs());
	}
}

double FSocialActionTimeTracker::GetActionStartTime() const
{
	return ActionSteps.Num() > 0 ? ActionSteps[0].StartTime : 0.0;
}

double FSocialActionTimeTracker::GetTotalDurationMs() const
{
	double TotalDuration = 0.0;
	for (const FSocialActionStep& Step : ActionSteps)
	{
		TotalDuration += Step.GetDurationMs();
	}
	return TotalDuration;
}

FName FSocialActionTimeTracker::GetCurrentStepName() const
{
	return ActionSteps.Num() > 0 ? ActionSteps.Last().StepName : NAME_None;
}

double FSocialActionTimeTracker::GetStepDurationMs(FName StepName) const
{
	if (const FSocialActionStep* Step = ActionSteps.FindByKey(StepName))
	{
		return Step->GetDurationMs();
	}
	return 0.0;
}

double FSocialActionTimeTracker::FSocialActionStep::GetDurationMs() const
{
	double TotalSeconds = (EndTime != 0.0 ? EndTime : FPlatformTime::Seconds()) - StartTime;
	return TotalSeconds * 1000.0;
}

