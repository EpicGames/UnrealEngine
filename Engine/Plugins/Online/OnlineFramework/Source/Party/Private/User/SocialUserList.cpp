// Copyright Epic Games, Inc. All Rights Reserved.

#include "User/SocialUserList.h"
#include "SocialToolkit.h"
#include "Party/PartyMember.h"

#include "SocialManager.h"
#include "Party/SocialParty.h"

TSharedRef<FSocialUserList> FSocialUserList::CreateUserList(const USocialToolkit& InOwnerToolkit, const FSocialUserListConfig& InConfig)
{
	TSharedRef<FSocialUserList> NewList = MakeShareable(new FSocialUserList(InOwnerToolkit, InConfig));
	NewList->InitializeList();
	return NewList;
}

FSocialUserList::FSocialUserList(const USocialToolkit& InOwnerToolkit, const FSocialUserListConfig& InConfig)
	: OwnerToolkit(&InOwnerToolkit)
	, ListConfig(InConfig)
{
	if (HasPresenceFilters() && ListConfig.RequiredPresenceFlags != ESocialUserStateFlags::SameParty && ListConfig.ForbiddenPresenceFlags != ESocialUserStateFlags::SameParty &&
		ListConfig.RelationshipType != ESocialRelationship::Any &&
		ListConfig.RelationshipType != ESocialRelationship::AnyWithLocal &&
		ListConfig.RelationshipType != ESocialRelationship::Friend &&
		ListConfig.RelationshipType != ESocialRelationship::PartyInvite)
	{
		UE_LOGF(LogParty, Error, "A user list with friend presence filters can only ever track friends. No users will ever appear in this list.");
	}
}

void FSocialUserList::InitializeList()
{
	check(OwnerToolkit.IsValid());

	// Bind appropriate events based on the desired relationship filter
	switch (ListConfig.RelationshipType)
	{
	case ESocialRelationship::FriendInviteReceived:
		OwnerToolkit->OnFriendInviteReceived().AddSP(this, &FSocialUserList::HandleFriendInviteReceived);
	case ESocialRelationship::FriendInviteSent:
		OwnerToolkit->OnFriendshipEstablished().AddSP(this, &FSocialUserList::HandleFriendshipEstablished);
		break;
	case ESocialRelationship::PartyInvite:
		OwnerToolkit->OnPartyInviteReceived().AddSP(this, &FSocialUserList::HandlePartyInviteReceived);
		OwnerToolkit->OnPartyInviteRemoved().AddSP(this, &FSocialUserList::HandlePartyInviteRemoved);
		break;
	case ESocialRelationship::Friend:
		OwnerToolkit->OnFriendshipEstablished().AddSP(this, &FSocialUserList::HandleFriendshipEstablished);
		break;
	case ESocialRelationship::RecentPlayer:
		OwnerToolkit->OnRecentPlayerAdded().AddSP(this, &FSocialUserList::HandleRecentPlayerAdded);
		OwnerToolkit->OnFriendshipEstablished().AddSP(this, &FSocialUserList::HandleFriendshipEstablished);
		break;
	case ESocialRelationship::SuggestedFriend:
		OwnerToolkit->OnFriendshipEstablished().AddSP(this, &FSocialUserList::HandleFriendshipEstablished);
		break;
	case ESocialRelationship::JoinRequest:
		OwnerToolkit->OnPartyRequestToJoinReceived().AddSP(this, &FSocialUserList::HandleRequestToJoinReceived);
		OwnerToolkit->OnPartyRequestToJoinRemoved().AddSP(this, &FSocialUserList::HandleRequestToJoinRemoved);
		break;
	}

	OwnerToolkit->OnToolkitReset().AddSP(this, &FSocialUserList::HandleOwnerToolkitReset);
	OwnerToolkit->OnUserBlocked().AddSP(this, &FSocialUserList::HandleUserBlocked);

	// Run through all the users on the toolkit and add any that qualify for this list
	check(Users.Num() == 0);
	for (USocialUser* User : OwnerToolkit->GetAllUsers())
	{
		check(User);
		TryAddUserFast(*User);
	}
	
	if (EnumHasAnyFlags(ListConfig.ForbiddenPresenceFlags, ESocialUserStateFlags::SameParty) ||
		EnumHasAnyFlags(ListConfig.RequiredPresenceFlags, ESocialUserStateFlags::SameParty))
	{
		OwnerToolkit->GetSocialManager().OnPartyJoined().AddSP(this, &FSocialUserList::HandlePartyJoined);
		if (USocialParty* PersistentParty = OwnerToolkit->GetSocialManager().GetPersistentParty())
		{
			HandlePartyJoined(*PersistentParty);
		}
	}

	AutoUpdatePeriod = USocialSettings::GetUserListAutoUpdateRate();
	SetAllowAutoUpdate(ListConfig.bAutoUpdate);
}

void FSocialUserList::UpdateNow()
{
	UpdateListInternal();
}

void FSocialUserList::SetAllowAutoUpdate(bool bIsEnabled)
{
	bIsEnabled ? AutoUpdateRequests++ : AutoUpdateRequests--;
	AutoUpdateRequests = FMath::Max(AutoUpdateRequests, 0);

	if (AutoUpdateRequests == 0 && UpdateTickerHandle.IsValid())
	{
		FTSTicker::RemoveTicker(UpdateTickerHandle);
		UpdateTickerHandle.Reset();
	}
	else if (bIsEnabled && !UpdateTickerHandle.IsValid())
	{
		UpdateTickerHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateSP(this, &FSocialUserList::HandleAutoUpdateList), AutoUpdatePeriod);
	}
}

void FSocialUserList::SetAllowSortDuringUpdate(bool bIsEnabled)
{
	ListConfig.bSortDuringUpdate = bIsEnabled;
}

bool FSocialUserList::HasPresenceFilters() const
{
	return ListConfig.RequiredPresenceFlags != ESocialUserStateFlags::None || ListConfig.ForbiddenPresenceFlags != ESocialUserStateFlags::None;
}

void FSocialUserList::HandleOwnerToolkitReset()
{
	const bool bTriggerChangeEvent = Users.Num() > 0;

	for (TWeakObjectPtr<USocialUser> UserPtr : Users)
	{
		if (USocialUser* User = UserPtr.Get(); ensureMsgf(User, TEXT("Encountered a nullptr entry in FSocialUserList::Users array!")))
		{
			OnUserRemoved().Broadcast(*User);
		}
	}
	Users.Reset();
	PendingAdds.Reset();
	PendingRemovals.Reset();
	UsersWithDirtyPresence.Reset();
	if (bTriggerChangeEvent)
	{
		OnUpdateComplete().Broadcast();
	}

	if (OwnerToolkit.IsValid())
	{
		OwnerToolkit->GetSocialManager().OnPartyJoined().RemoveAll(this);
	}
}

void FSocialUserList::HandlePartyInviteReceived(USocialUser& InvitingUser)
{
	TryAddUser(InvitingUser);
	UpdateListInternal();
}

void FSocialUserList::HandlePartyInviteRemoved(USocialUser& InvitingUser)
{
	TryRemoveUser(InvitingUser);
	UpdateListInternal();
}

void FSocialUserList::HandlePartyInviteHandled(USocialUser* InvitingUser)
{
	TryRemoveUser(*InvitingUser);
	UpdateListInternal();
}

void FSocialUserList::HandleFriendInviteReceived(USocialUser& User, ESocialSubsystem SubsystemType)
{
	TryAddUser(User);
	UpdateListInternal();
}

void FSocialUserList::HandleFriendInviteRemoved(ESocialSubsystem SubsystemType, USocialUser* User)
{
	TryRemoveUser(*User);
	UpdateListInternal();
}

void FSocialUserList::HandleFriendshipEstablished(USocialUser& NewFriend, ESocialSubsystem SubsystemType, bool bIsNewRelationship)
{
	if (ListConfig.RelationshipType == ESocialRelationship::Friend ||
		ListConfig.RelationshipType == ESocialRelationship::FriendInviteReceived)
	{
		TryAddUser(NewFriend);
	}

	if (ListConfig.RelationshipType != ESocialRelationship::Friend)
	{
		// Any non-friends list that cares about friendship does so to remove entries (i.e. invites & recent players)
		TryRemoveUser(NewFriend);
		UpdateListInternal();
	}
}

void FSocialUserList::HandleFriendRemoved(ESocialSubsystem SubsystemType, USocialUser* User)
{
	TryRemoveUser(*User);
	UpdateListInternal();
}

void FSocialUserList::HandleUserBlocked(USocialUser& BlockedUser, ESocialSubsystem SubsystemType, bool bIsNewRelationship)
{
	if (ListConfig.RelationshipType == ESocialRelationship::BlockedPlayer)
	{
		TryAddUser(BlockedUser);
	}
	else
	{
		// When a player is blocked, any other existing relationship is implicitly nixed
		TryRemoveUser(BlockedUser);
	}
	
	UpdateListInternal();
}

void FSocialUserList::HandleUserBlockStatusChanged(ESocialSubsystem SubsystemType, bool bIsBlocked, USocialUser* User)
{
	if (!bIsBlocked)
	{
		TryRemoveUser(*User);
		UpdateListInternal();
	}
}

void FSocialUserList::HandleRecentPlayerAdded(USocialUser& AddedUser, ESocialSubsystem SubsystemType, bool bIsNewRelationship)
{
	TryAddUser(AddedUser);
}

void FSocialUserList::HandleRecentPlayerRemoved(USocialUser& RemovedUser, ESocialSubsystem SubsystemType)
{
	TryRemoveUser(RemovedUser);
}

void FSocialUserList::HandleUserPresenceChanged(ESocialSubsystem SubsystemType, USocialUser* User)
{
	MarkUserAsDirty(*User);
}

void FSocialUserList::HandleUserGameSpecificStatusChanged(USocialUser* User)
{
	MarkUserAsDirty(*User);
	UpdateNow();
}

void FSocialUserList::HandleRequestToJoinReceived(USocialUser& SocialUser, IOnlinePartyRequestToJoinInfoConstRef Request)
{
	TryAddUser(SocialUser);
	UpdateListInternal();
}

void FSocialUserList::HandleRequestToJoinRemoved(USocialUser& SocialUser, IOnlinePartyRequestToJoinInfoConstRef Request, EPartyRequestToJoinRemovedReason Reason)
{
	TryRemoveUser(SocialUser);
	UpdateListInternal();
}

void FSocialUserList::MarkUserAsDirty(USocialUser& User)
{
	// Save this dirtied user for re-evaluation during the next update
	UsersWithDirtyPresence.Add(&User);
	bNeedsSort = true;
}

void FSocialUserList::TryAddUser(USocialUser& User)
{
	if (!PendingAdds.Contains(&User) && (!Users.Contains(&User) || PendingRemovals.Contains(&User)))
	{
		TryAddUserFast(User);
	}
	else
	{
		// Something changed about a user already in the list, so we'll need to re-sort
		bNeedsSort = true;
	}
}

void FSocialUserList::TryAddUserFast(USocialUser& User)
{
	bool bCanAdd = false;

	TArray<ESocialSubsystem> ActiveRelationshipSubsystems = User.GetRelationshipSubsystems(ListConfig.RelationshipType);
	for (ESocialSubsystem RelationshipSubsystem : ActiveRelationshipSubsystems)
	{
		// Is the relationship on this subsystem relevant to us?
		if (ListConfig.ForbiddenSubsystems.Contains(RelationshipSubsystem))
		{
			// Immediately bail entirely if the relationship exists on any forbidden subsystems
			return;
		}
		else if (!bCanAdd && ListConfig.RelevantSubsystems.Contains(RelationshipSubsystem))
		{
			// Even if the user does not qualify for the list now due to presence filters, we still want to know about any changes to their presence to reevaluate
			if ((HasPresenceFilters() || ListConfig.OnCustomFilterUser.IsBound()) && !User.OnUserPresenceChanged().IsBoundToObject(this))
			{
				User.OnUserPresenceChanged().AddSP(this, &FSocialUserList::HandleUserPresenceChanged, &User);
			}

			if (ListConfig.GameSpecificStatusFilters.Num() > 0)
			{
				if (!User.OnUserGameSpecificStatusChanged().IsBoundToObject(this))
				{
					User.OnUserGameSpecificStatusChanged().AddSP(this, &FSocialUserList::HandleUserGameSpecificStatusChanged, &User);
				}
			}

			// Check that the user's current presence is acceptable
			if (EvaluateUserPresence(User, RelationshipSubsystem))
			{
				// Last step is to check the custom filter, if provided
				bCanAdd = ListConfig.OnCustomFilterUser.IsBound() ? ListConfig.OnCustomFilterUser.Execute(User) : true;

				// do an initial pass on the GameSpecificStatusFilters on newly added user, and for users that passed other checks but don't yet have the correct presence or game status,
				// this will be called again from UpdateListInternal(), after the user broadcasts OnGameSpecificStatusChanged, or OnUserPresenceChanged, and the list mark it dirty.
				for (TFunction<bool(const USocialUser&)> CustomFilterFunction : ListConfig.GameSpecificStatusFilters)
				{
					if (!bCanAdd)
					{
						break;
					}
					bCanAdd &= CustomFilterFunction(User);
				}
			}
		}
	}

	if (bCanAdd)
	{
		// Bind directly to the user we're adding to find out when we should remove them
		// ** Be sure to unbind within TryRemoveUserFast **
		switch (ListConfig.RelationshipType)
		{
		case ESocialRelationship::FriendInviteReceived:
		case ESocialRelationship::FriendInviteSent:
			User.OnFriendInviteRemoved().AddSP(this, &FSocialUserList::HandleFriendInviteRemoved, &User);
			break;
		case ESocialRelationship::PartyInvite:
			// We don't care whether the invite was accepted or rejected, just that it was handled in some way
			User.OnPartyInviteAccepted().AddSP(this, &FSocialUserList::HandlePartyInviteHandled, &User);
			User.OnPartyInviteRejected().AddSP(this, &FSocialUserList::HandlePartyInviteHandled, &User);
		case ESocialRelationship::Friend:
			User.OnFriendRemoved().AddSP(this, &FSocialUserList::HandleFriendRemoved, &User);
			break;
		case ESocialRelationship::BlockedPlayer:
			User.OnBlockedStatusChanged().AddSP(this, &FSocialUserList::HandleUserBlockStatusChanged, &User);
			break;
		}

		PendingRemovals.Remove(&User);
		PendingAdds.Add(&User);
	}
}

void FSocialUserList::TryRemoveUser(USocialUser& User)
{
	if (!PendingRemovals.Contains(&User) && (Users.Contains(&User) || PendingAdds.Contains(&User)))
	{
		TryRemoveUserFast(User);
	}
}

void FSocialUserList::TryRemoveUserFast(USocialUser& User)
{
	bool bUnbindFromPresenceUpdates = true;
	bool bRemoveUser = true;
	TArray<ESocialSubsystem> ActiveRelationshipSubsystems = User.GetRelationshipSubsystems(ListConfig.RelationshipType);
	for (ESocialSubsystem RelationshipSubsystem : ActiveRelationshipSubsystems)
	{
		if (ListConfig.ForbiddenSubsystems.Contains(RelationshipSubsystem))
		{
			bRemoveUser = true;
			break;
		}
		else if (bRemoveUser && ListConfig.RelevantSubsystems.Contains(RelationshipSubsystem))
		{
			bUnbindFromPresenceUpdates = false;
			if (EvaluateUserPresence(User, RelationshipSubsystem))
			{
				// We're going to keep the user based on the stock filters, but the custom filter can still veto
				bRemoveUser = ListConfig.OnCustomFilterUser.IsBound() ? !ListConfig.OnCustomFilterUser.Execute(User) : false;

				// For users that we decide to remove due to incorrect presence or game status, we'll keep listening to their presence and game status change since they may qualify again.
				// For users that have presence or game status changed but still fit this list, these will be run again next time we got a presence changed or game status changed event.
				if (!bRemoveUser)
				{
					for (TFunction<bool(const USocialUser&)> CustomFilterFunction : ListConfig.GameSpecificStatusFilters)
					{
						bRemoveUser |= !CustomFilterFunction(User);
					}
				}
			}
		}
	}

	if (bRemoveUser)
	{
		PendingAdds.Remove(&User);
		PendingRemovals.Add(&User);

		// Clear out all direct user bindings
		User.OnFriendInviteRemoved().RemoveAll(this);
		User.OnPartyInviteAccepted().RemoveAll(this);
		User.OnPartyInviteRejected().RemoveAll(this);
		User.OnBlockedStatusChanged().RemoveAll(this);

		if (bUnbindFromPresenceUpdates)
		{
			// Not only does this user not qualify for the list, they don't even have the appropriate relationship anymore (so we no longer care about presence changes)
			User.OnUserPresenceChanged().RemoveAll(this);
			User.OnUserGameSpecificStatusChanged().RemoveAll(this);
		}
	}
}

bool FSocialUserList::EvaluateUserPresence(const USocialUser& User, ESocialSubsystem SubsystemType)
{
	if (HasPresenceFilters())
	{
		bool bIsOnline = false;
		bool bIsPlayingThisGame = false;
		bool bInSameParty = false;

		if (USocialManager::ShouldReadPresenceFromOnlineServices() && SubsystemType == ESocialSubsystem::Primary)
		{
			using namespace UE::Online;

			const TOnlineResult<FGetCachedPresence>& GetCachedPresenceResult = User.GetFriendPresenceInfoV2(SubsystemType);
			if (GetCachedPresenceResult.IsOk())
			{
				const TSharedRef<const FUserPresence> CachedPresence = GetCachedPresenceResult.GetOkValue().Presence;
				bIsOnline = CachedPresence->Status != UE::Online::EUserPresenceStatus::Offline;
				bIsPlayingThisGame = CachedPresence->GameStatus == UE::Online::EUserPresenceGameStatus::PlayingThisGame;
			}
		}
		else
		{
			if (const FOnlineUserPresence* UserPresence = User.GetFriendPresenceInfo(SubsystemType))
			{
				bIsOnline = UserPresence->bIsOnline;
				bIsPlayingThisGame = UserPresence->bIsPlayingThisGame;
			}
		}
		
		if (OwnerToolkit.IsValid())
		{
			if (const USocialParty* CurrentParty = OwnerToolkit->GetSocialManager().GetPersistentParty())
			{
				bInSameParty = CurrentParty->ContainsUser(User);
			}

#if WITH_EDITOR || !UE_BUILD_SHIPPING
			if (OwnerToolkit->Debug_IsRandomlyChangingPresence())
			{
				if (USocialManager::ShouldReadPresenceFromOnlineServices())
				{
					bIsOnline = User.GetOnlineStatusV2() != UE::Online::EUserPresenceStatus::Offline;
				}
				else
				{
					bIsOnline = User.GetOnlineStatus() != EOnlinePresenceState::Offline;
				}

				bIsOnline = User.GetOnlineStatus() != EOnlinePresenceState::Offline;
				bIsPlayingThisGame = bIsOnline;
			}
#endif
		}

		return EvaluatePresenceFlag(bIsOnline, ESocialUserStateFlags::Online)
			&& EvaluatePresenceFlag(bIsPlayingThisGame, ESocialUserStateFlags::SameApp)
			&& EvaluatePresenceFlag(bInSameParty, ESocialUserStateFlags::SameParty);
		// && EvaluatePresenceFlag(UserPresence->bIsJoinable, ESocialUserStateFlags::Joinable) <-- //@todo DanH: This property exists on presence, but is ALWAYS false... 
		// && EvaluateFlag(UserPresence->?, ESocialUserStateFlag::SamePlatform)
		// && EvaluateFlag(UserPresence->?, ESocialUserStateFlag::LookingForGroup)
	}

	return true;
}

bool FSocialUserList::EvaluatePresenceFlag(bool bPresenceValue, ESocialUserStateFlags Flag) const
{
	if (EnumHasAnyFlags(ListConfig.RequiredPresenceFlags, Flag))
	{
		// It's required, so value must be true to be eligible
		return bPresenceValue;
	}
	else if (EnumHasAnyFlags(ListConfig.ForbiddenPresenceFlags, Flag))
	{
		// It's forbidden, so value must be false to be eligible
		return !bPresenceValue;
	}
	// Irrelevant
	return true;
}

// encapsulates UserList sorting comparator and supporting data needed
struct FUserSortData
{
	FUserSortData(USocialUser* InUser, EOnlinePresenceState::Type InStatus, bool InPlayingThisGame, FString InDisplayName, TSharedRef<const TArray<int64>> InSortParams)
		: User(InUser)
		, OnlineStatus(InStatus)
		, PlayingThisGame(InPlayingThisGame)
		, DisplayName(MoveTemp(InDisplayName))
		, SortParams(InSortParams)
	{ }

	FUserSortData(USocialUser* InUser, UE::Online::EUserPresenceStatus InStatus, bool InPlayingThisGame, FString InDisplayName, TSharedRef<const TArray<int64>> InSortParams)
		: User(InUser)
		, OnlineStatusV2(InStatus)
		, PlayingThisGame(InPlayingThisGame)
		, DisplayName(MoveTemp(InDisplayName))
		, SortParams(InSortParams)
	{ }

	USocialUser* User = nullptr;
	TOptional<EOnlinePresenceState::Type> OnlineStatus;
	TOptional<UE::Online::EUserPresenceStatus> OnlineStatusV2;
	bool PlayingThisGame;
	FString DisplayName;
	TSharedPtr<const TArray<int64>> SortParams;

	bool operator<(const FUserSortData& OtherSortData) const
	{
		// Sorts in order of if online, playing this game, custom list of search parameters, then alphabetical
		if ((OnlineStatus.IsSet() && *OnlineStatus == *OtherSortData.OnlineStatus) || (OnlineStatusV2.IsSet() && *OnlineStatusV2 == *OtherSortData.OnlineStatusV2))
		{
			if (PlayingThisGame == OtherSortData.PlayingThisGame)
			{
				if (SortParams.IsValid() && OtherSortData.SortParams.IsValid() && SortParams->Num() == OtherSortData.SortParams->Num())
				{
					for (int32 SortIndex = 0; SortIndex < SortParams->Num() && SortIndex < OtherSortData.SortParams->Num(); ++SortIndex)
					{
						if ((*SortParams)[SortIndex] != (*OtherSortData.SortParams)[SortIndex])
						{
							// Sort first by this parameter
							return (*SortParams)[SortIndex] > (*OtherSortData.SortParams)[SortIndex];
						}
					}

					// If we made it here, our sort params were all equal, so just sort alphabetically
					return DisplayName < OtherSortData.DisplayName;
				}
				else
				{
					// Our sort params don't match, so in this case just sort alphabetically
					return DisplayName < OtherSortData.DisplayName;
				}
			}
			else
			{
				return PlayingThisGame > OtherSortData.PlayingThisGame;
			}
		}
		else
		{
			// @todo StephanJ: note Online < Offline < Away, but it's okay for now since we show offline in a separate list #future
			return OnlineStatus.IsSet() ? *OnlineStatus < *OtherSortData.OnlineStatus : OnlineStatusV2.IsSet() ? *OnlineStatusV2 < *OtherSortData.OnlineStatusV2 : false;		
		}
	}
};

bool FSocialUserList::HandleAutoUpdateList(float)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FSocialUserList_HandleAutoUpdateList);
	UpdateListInternal();
	return true;
}

void FSocialUserList::UpdateListInternal()
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_SocialUserList_UpdateList);

	// Re-evaluate whether each user with dirtied presence is still fit for the list
	for (TWeakObjectPtr<USocialUser> DirtyUser : UsersWithDirtyPresence)
	{
		if (DirtyUser.IsValid())
		{
			const bool bContainsUser = Users.Contains(DirtyUser);
			const bool bPendingAdd = PendingAdds.Contains(DirtyUser);
			const bool bPendingRemove = PendingRemovals.Contains(DirtyUser);

			if (bPendingRemove || (!bContainsUser && !bPendingAdd))
			{
				TryAddUserFast(*DirtyUser);
			}
			else if (bPendingAdd || (bContainsUser && !bPendingRemove))
			{
				TryRemoveUser(*DirtyUser);
			}
		}
	}
	UsersWithDirtyPresence.Reset();

	// Update the users in the list
	bool bListUpdated = false;
	if (PendingRemovals.Num() > 0)
	{
		bListUpdated = true;

		Users.RemoveAllSwap(
			[this] (TWeakObjectPtr<USocialUser> User)
		{
				if (PendingRemovals.Contains(User))
				{
					PendingRemovals.Remove(User);
					OnUserRemoved().Broadcast(*User);
					return true;
				}

				return false;
			});

		PendingRemovals.Reset();
	}

	if (PendingAdds.Num() > 0)
	{
		bListUpdated = true;
		Users.Append(PendingAdds);

		for (TWeakObjectPtr<USocialUser> UserPtr : PendingAdds)
		{
			if (USocialUser* User = UserPtr.Get(); ensure(User))
			{
				OnUserAdded().Broadcast(*User);
			}
		}
		PendingAdds.Reset();
	}

	if (bListUpdated || bNeedsSort)
	{
		if (ListConfig.bSortDuringUpdate)
		{
			bNeedsSort = false;
			bListUpdated = true;

			const int32 NumUsers = Users.Num();
			if (NumUsers > 1)
			{
				SCOPED_NAMED_EVENT(STAT_SocialUserList_Sort, FColor::Orange);

				UE_LOGF(LogParty, VeryVerbose, "%ls sorting list of [%d] users", ANSI_TO_TCHAR(__FUNCTION__), NumUsers);

				TArray<FUserSortData> SortedData;
				SortedData.Reserve(NumUsers);

				Algo::TransformIf(Users, SortedData,
					[](const TWeakObjectPtr<USocialUser> UserPtr) { return UserPtr.IsValid(); },
					[this](const TWeakObjectPtr<USocialUser> UserPtr) -> FUserSortData
					{
						USocialUser* User = UserPtr.Get();
						TSharedRef<TArray<int64>> SortParams = MakeShared<TArray<int64>>();
						User->PopulateSortParameterList(SortParams.Get());

						if (USocialManager::ShouldReadPresenceFromOnlineServices())
						{
							return FUserSortData(User, User->GetOnlineStatusV2(), User->IsPlayingThisGame(), User->GetDisplayName(), SortParams);
						}
						else
						{
							return FUserSortData(User, User->GetOnlineStatus(), User->IsPlayingThisGame(), User->GetDisplayName(), SortParams);
						}
					});

				Algo::Sort(SortedData);

				// replace contents of Users from SortedData array
				Users.Reset(SortedData.Num());
				Algo::Transform(SortedData, Users, [](const FUserSortData& Data){ return Data.User; });
			}
		}
		else
		{
			bNeedsSort = true;
		}

		if (bListUpdated)
		{
			OnUpdateComplete().Broadcast();
		}
	}
}

void FSocialUserList::HandlePartyJoined(USocialParty& Party)
{
	Party.OnPartyLeft().AddSP(this, &FSocialUserList::HandlePartyLeft, &Party);
	Party.OnPartyMemberCreated().AddSP(this, &FSocialUserList::HandlePartyMemberCreated);

	for (UPartyMember* PartyMember : Party.GetPartyMembers())
	{
		if (PartyMember)
		{
			PartyMember->OnLeftParty().AddSP(this, &FSocialUserList::HandlePartyMemberLeft, PartyMember, true);
			MarkPartyMemberAsDirty(*PartyMember);
		}
	}

	UpdateNow();
}

void FSocialUserList::HandlePartyLeft(EMemberExitedReason Reason, USocialParty* LeftParty)
{
	if (LeftParty)
	{
		for (UPartyMember* ExistingMember : LeftParty->GetPartyMembers())
		{
			HandlePartyMemberLeft(EMemberExitedReason::Left, ExistingMember, false);
		}

		LeftParty->OnPartyLeft().RemoveAll(this);
		LeftParty->OnPartyMemberCreated().RemoveAll(this);

		UpdateNow();
	}
}

void FSocialUserList::HandlePartyMemberCreated(UPartyMember& Member)
{
	Member.OnLeftParty().AddSP(this, &FSocialUserList::HandlePartyMemberLeft, &Member, true);
	MarkPartyMemberAsDirty(Member);
	UpdateNow();
}

void FSocialUserList::HandlePartyMemberLeft(EMemberExitedReason Reason, UPartyMember* Member, bool bUpdateNow)
{
	if (ensure(Member))
	{
		Member->OnLeftParty().RemoveAll(this);
		MarkPartyMemberAsDirty(*Member);

		if (bUpdateNow)
		{
			UpdateNow();
		}
	}
}

USocialUser* FSocialUserList::FindOwnersRelationshipTo(UPartyMember& TargetPartyMember) const
{
	if (OwnerToolkit.IsValid())
	{
		const FUniqueNetIdRepl& PartyMemberNetId = TargetPartyMember.GetPrimaryNetId();
		return OwnerToolkit->FindUser(PartyMemberNetId);
	}

	return nullptr;
}

void FSocialUserList::MarkPartyMemberAsDirty(UPartyMember& PartyMember)
{
	// Find the USocialUser representing the PartyMember in the owning toolkit's set of USocialUsers.
	// Must be looked up specifically for this owner player rather than the party because each player
	// will have their own set of relationships (e.g. muted, blocked) to each of the other players.
	if (OwnerToolkit.IsValid())
	{
		if (USocialUser* const PartyMemberSocialUser = FindOwnersRelationshipTo(PartyMember))
		{
			MarkUserAsDirty(*PartyMemberSocialUser);
		}
	}
}
