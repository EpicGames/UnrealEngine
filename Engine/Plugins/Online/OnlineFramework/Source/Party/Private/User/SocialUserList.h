// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "User/ISocialUserList.h"
#include "Containers/Ticker.h"

class IOnlinePartyRequestToJoinInfo;
class UPartyMember;
class USocialParty;
class USocialToolkit;
enum class EPartyRequestToJoinRemovedReason : uint8;

class USocialUser;

enum class EMemberExitedReason : uint8;
typedef TSharedRef<const IOnlinePartyRequestToJoinInfo> IOnlinePartyRequestToJoinInfoConstRef;

class FSocialUserList : public ISocialUserList, public TSharedFromThis<FSocialUserList>
{
	friend class USocialToolkit;
public:
	static TSharedRef<FSocialUserList> CreateUserList(const USocialToolkit& InOwnerToolkit, const FSocialUserListConfig& Config);

	FOnUserAdded& OnUserAdded() const override { return OnUserAddedEvent; }
	FOnUserRemoved& OnUserRemoved() const override { return OnUserRemovedEvent; }
	FOnUpdateComplete& OnUpdateComplete() const override { return OnUpdateCompleteEvent; }

	virtual FString GetListName() const override { return ListConfig.Name; }

	virtual void UpdateNow() override;
	virtual void SetAllowAutoUpdate(bool bIsEnabled) override;
	virtual void SetAllowSortDuringUpdate(bool bIsEnabled) override;
	virtual const TArray<TWeakObjectPtr<USocialUser>>& GetUsers() const override { return Users; }

	bool HasPresenceFilters() const;

protected:
	const FSocialUserListConfig& GetListConfig() const { return ListConfig; }
	virtual void MarkUserAsDirty(USocialUser& User) override;

private:
	void HandleOwnerToolkitReset();

	void HandlePartyInviteReceived(USocialUser& InvitingUser);
	void HandlePartyInviteRemoved(USocialUser& InvitingUser);
	void HandlePartyInviteHandled(USocialUser* InvitingUser);

	void HandleFriendInviteReceived(USocialUser& User, ESocialSubsystem SubsystemType);
	void HandleFriendInviteRemoved(ESocialSubsystem SubsystemType, USocialUser* User);

	void HandleFriendshipEstablished(USocialUser& NewFriend, ESocialSubsystem SubsystemType, bool bIsNewRelationship);
	void HandleFriendRemoved(ESocialSubsystem SubsystemType, USocialUser* User);
	
	void HandleUserBlocked(USocialUser& BlockedUser, ESocialSubsystem SubsystemType, bool bIsNewRelationship);
	void HandleUserBlockStatusChanged(ESocialSubsystem SubsystemType, bool bIsBlocked, USocialUser* User);

	void HandleRecentPlayerAdded(USocialUser& AddedUser, ESocialSubsystem SubsystemType, bool bIsNewRelationship);
	void HandleRecentPlayerRemoved(USocialUser& RemovedUser, ESocialSubsystem SubsystemType);
	
	void HandleUserPresenceChanged(ESocialSubsystem SubsystemType, USocialUser* User);
	void HandleUserGameSpecificStatusChanged(USocialUser* User);

	void HandleRequestToJoinReceived(USocialUser& SocialUser, IOnlinePartyRequestToJoinInfoConstRef Request);
	void HandleRequestToJoinRemoved(USocialUser& SocialUser, IOnlinePartyRequestToJoinInfoConstRef Request, EPartyRequestToJoinRemovedReason Reason);

	void TryAddUser(USocialUser& User);
	void TryAddUserFast(USocialUser& User);

	void TryRemoveUser(USocialUser& User);
	void TryRemoveUserFast(USocialUser& User);
	
	bool EvaluateUserPresence(const USocialUser& User, ESocialSubsystem SubsystemType);
	bool EvaluatePresenceFlag(bool bPresenceValue, ESocialUserStateFlags Flag) const;

	bool HandleAutoUpdateList(float);
	void UpdateListInternal();

	void HandlePartyJoined(USocialParty& Party);
	void HandlePartyLeft(EMemberExitedReason Reason, USocialParty* LeftParty);
	void HandlePartyMemberCreated(UPartyMember& Member);
	void HandlePartyMemberLeft(EMemberExitedReason Reason, UPartyMember* Member, bool bUpdateNow);

	USocialUser* FindOwnersRelationshipTo(UPartyMember& TargetPartyMember) const;
	void MarkPartyMemberAsDirty(UPartyMember& PartyMember);

private:
	FSocialUserList(const USocialToolkit& InOwnerToolkit, const FSocialUserListConfig& Config);
	void InitializeList();

	TWeakObjectPtr<const USocialToolkit> OwnerToolkit;
	TArray<TWeakObjectPtr<USocialUser>> Users;
	TArray<TWeakObjectPtr<USocialUser>> PendingAdds;

	TSet<TWeakObjectPtr<USocialUser>> UsersWithDirtyPresence;
	TArray<TWeakObjectPtr<const USocialUser>> PendingRemovals;

	FSocialUserListConfig ListConfig;

	bool bNeedsSort = false;
	int32 AutoUpdateRequests = 0;
	float AutoUpdatePeriod = .5f;
	FTSTicker::FDelegateHandle UpdateTickerHandle;

	mutable FOnUserAdded OnUserAddedEvent;
	mutable FOnUserRemoved OnUserRemovedEvent;
	mutable FOnUpdateComplete OnUpdateCompleteEvent;
};
