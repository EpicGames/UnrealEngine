// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once
//#include "UObject/Interface.h"

#include "Interactions/SocialInteractionHandle.h"

#include "UObject/WeakObjectPtr.h"

#define UE_API PARTY_API

class FSocialUserList;
class FChatSlashCommand;
class FInteractionCommandWrapper;
class ULocalPlayer;
class USocialManager;
class USocialChatManager;
class USocialUser;
class USocialToolkit;



///////////////////////////////////////////////////////////////////////////////
struct FAutoCompleteStruct
{
	FAutoCompleteStruct(const FString& InFullString, const TWeakPtr<const FChatSlashCommand>& InCmd, TWeakObjectPtr<USocialUser> InOptionalTargetUser);

	//Cacheing data with strings is tricky because they will become invalid when the user changes their localization
	//but a user can't change this while typing a command, so it is okay in this case.
	FString FullString;
	TWeakPtr<const FChatSlashCommand> SlashCommand;
	TWeakObjectPtr<USocialUser> OptionalTargetUser;
	TArray<FString> Tokens;
};

///////////////////////////////////////////////////////////////////////////////
//Slash Command Component
class FRegisteredSlashCommands : public TSharedFromThis<FRegisteredSlashCommands>
{
public:
	static UE_API void TokenizeMessage(const FString& InChatText, TArray<FString>& Tokens);
	static UE_API bool TokensExactMatch(TArray<FString>& TokensLHS, TArray<FString>& TokensRHS);
	static UE_API bool CmdMatchesFirstToken(const FString& CmdString, const TArray<FString>& Tokens);

	FRegisteredSlashCommands() = default;
	UE_API void Init(USocialToolkit& Toolkit);

	/** main entry point for class encapsulated behavior; returns true if command executed */
	UE_API bool NotifyUserTextChanged(const FText& InText);

	UE_API bool TryExecuteCommandByMatchingText(const FString& UserTypedText);
	bool HasAutoCompleteSuggestions(){ return AutoCompleteData.Num() != 0; }
	const TArray<TSharedPtr<FAutoCompleteStruct>>& GetAutoCompleteStrings() const { return AutoCompleteData; }
	UE_API void RegisterCommand(const TSharedPtr<FChatSlashCommand>& NewSlashCommand);
	UE_API bool IsEnabled();

private: 
	UE_API void PrepareInteractionAutocompleteStrings(const TArray<FString>& StringTokens);
	UE_API void HandleCultureChanged() const;

	uint32 LastQueryTextLen = 0;
	bool bValidUsersCached = false;
	UE_API bool SpaceWasJustTyped(const FString& NewUserText);

private: 
	TArray<TSharedPtr<FChatSlashCommand>> RegisteredCustomSlashCommands;
	TArray<TSharedPtr<FInteractionCommandWrapper>> RegisteredInteractionSlashCommands;

	//once set, this should always be valid since lifetime of SocialManager is tied to game instance
	TWeakObjectPtr<USocialToolkit> MyToolkit;
	mutable TArray<TSharedPtr<FAutoCompleteStruct>> AutoCompleteData;

	FRegisteredSlashCommands(const FRegisteredSlashCommands& copy) = delete;
	FRegisteredSlashCommands(FRegisteredSlashCommands&& move) = delete;
	FRegisteredSlashCommands& operator=(const FRegisteredSlashCommands& copy) = delete;
	FRegisteredSlashCommands& operator=(FRegisteredSlashCommands&& move) = delete;
};
///////////////////////////////////////////////////////////////////////////////



//////////////////////////////////////////////////////////////////////////////////
class FChatSlashCommand : public TSharedFromThis<FChatSlashCommand>
{
public:
	/** @param InCommandName Command name including / prefix. eg "/party".	*/
	UE_API explicit FChatSlashCommand(const FText& InCommandName);
	virtual ~FChatSlashCommand() = default;

	UE_API virtual void Init(USocialToolkit& InToolkit);
	virtual bool IsEnabled() const = 0;
	virtual void ExecuteSlashCommand(USocialUser* OptionalTargetUser) const = 0;

	UE_API virtual void GetAutoCompleteStrings(TArray< TSharedPtr<FAutoCompleteStruct> >& OutStringContainer, const TArray<FString>& UserTextTokens) const;
	virtual bool CanExecuteSpacebarFromPartialTokens(const TArray<FString>& UserTextTokens) const { return false;  }
	virtual bool HasSpacebarExecuteFunctionality() const { return false; }
	virtual bool RequiresUserForExecution() const {return false;}
	UE_API virtual void RecacheForLocalization() const;

	const FString& GetCommandNameString() const { return CommandNameString; }

protected:
	USocialToolkit* GetToolkit() const { return MyToolkit.Get(); }
	mutable FString CommandNameString;

private:
	const FText CommandNameTextSrc;
	TWeakObjectPtr<USocialToolkit> MyToolkit;

	FChatSlashCommand(const FChatSlashCommand& copy) = delete;
	FChatSlashCommand(FChatSlashCommand&& move) = delete;
	FChatSlashCommand& operator=(const FChatSlashCommand& copy) = delete;
	FChatSlashCommand& operator=(FChatSlashCommand&& move) = delete;
};

//////////////////////////////////////////////////////////////////////////////////

class FInteractionCommandWrapper: public FChatSlashCommand
{
public:
	/** Interaction tokens will have / prefix appended.*/
	UE_API FInteractionCommandWrapper(FSocialInteractionHandle Interaction);

	UE_API virtual void ExecuteSlashCommand(USocialUser* OptionalTargetUser) const override; 
	virtual bool IsEnabled() const override	{ return true; 	}
	virtual bool HasSpacebarExecuteFunctionality() const { return true; }
	virtual bool RequiresUserForExecution() const override{ return true; }
	virtual bool CanExecuteSpacebarFromPartialTokens(const TArray<FString>& UserTextTokens) const{return false;	}
	
	UE_API void ResetUserCache(); 
	inline void TryCacheValidAutoCompleteUser(USocialUser& User, const TArray<FString>& StringTokens);
	UE_API virtual void GetAutoCompleteStrings(TArray< TSharedPtr<FAutoCompleteStruct> >& OutStringContainer, const TArray<FString>& UserTextTokens) const override;
	UE_API virtual void RecacheForLocalization() const override;

private:
	FSocialInteractionHandle WrappedInteraction;
	mutable FString CachedCommandToken;
	TArray<TWeakObjectPtr<USocialUser>> CachedValidUsers;

	/* 
	* NOTE: we cannot simply cache the FText long term because localization changes while running will invalidate cache.
	* So, there exists the following function to re-query the localization
	*/
	UE_API void CacheStringDataForLocalization() const;
};

#undef UE_API
