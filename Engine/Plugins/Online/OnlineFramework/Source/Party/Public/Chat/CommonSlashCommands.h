// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Chat/ChatSlashCommands.h"
#include "UObject/Object.h"

#define UE_API PARTY_API

class ULocalPlayer;
class USocialUser;
class USocialManager;
class USocialToolkit;
class USocialChatChannel;
enum class ESocialChannelType : uint8;

class FChannelChangeSlashCommand : public FChatSlashCommand
{
public:
	UE_API FChannelChangeSlashCommand(const FText& CommandText, ESocialChannelType InChannelType);

	UE_API virtual void ExecuteSlashCommand(USocialUser* OptionalTargetUser) const override;
	UE_API virtual bool IsEnabled() const override;
	UE_API virtual bool CanExecuteSpacebarFromPartialTokens(const TArray<FString>& UserTextTokens) const override;
	virtual bool HasSpacebarExecuteFunctionality() const { return true; }

private:
	ESocialChannelType ChannelType;

};

class FPartyChannelSlashCommand : public FChannelChangeSlashCommand
{
public:
	UE_API FPartyChannelSlashCommand();
};

class FGlobalChannelSlashCommand : public FChannelChangeSlashCommand
{
public:
	UE_API FGlobalChannelSlashCommand();
};

class FTeamChannelSlashCommand : public FChannelChangeSlashCommand
{
public:
	UE_API FTeamChannelSlashCommand();
};

class FFounderChannelSlashCommand : public FChannelChangeSlashCommand
{
public:
	UE_API FFounderChannelSlashCommand();
};

/////////////////////////////////////////////////////////////////////////////////////////////////////////////

class FReplySlashCommand: public FChatSlashCommand
{
public:
	UE_API FReplySlashCommand();
	UE_API virtual void Init(USocialToolkit& InToolkit) override;
	UE_API virtual void ExecuteSlashCommand(USocialUser* OptionalTargetUser) const override;
	UE_API virtual bool IsEnabled() const override;
	TWeakObjectPtr<USocialChatChannel> LastUserChannel = nullptr;

private:
	void HandleChannelCreated(USocialChatChannel& NewChannel);
	void HandleChannelLeft(USocialChatChannel& LeavingChannel);
};

#undef UE_API
