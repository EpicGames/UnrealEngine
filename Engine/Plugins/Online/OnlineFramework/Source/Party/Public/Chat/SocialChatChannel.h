// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Interfaces/OnlineChatInterface.h"
#include "OnlineSubsystem.h"
#include "SocialTypes.h"
#include "SocialChatChannel.generated.h"

#define UE_API PARTY_API

class FSocialLocalChatMessage;
class USocialChatManager;
class USocialToolkit;
class USocialUser;

UENUM(BlueprintType)
enum class ESocialChannelType : uint8
{
	General,
	Founder,
	Party,
	Team,
	System,
	Private
};

/** Base SocialCore chat channel class (partial ISocialChatChannel implementation) */
UCLASS(MinimalAPI, Abstract, Within=SocialChatManager)
class USocialChatChannel : public UObject
{
	GENERATED_BODY()

public:
	USocialChatChannel() {}

	DECLARE_EVENT_OneParam(USocialChatChannel, FOnChannelUserChanged, USocialUser&);
	virtual FOnChannelUserChanged& OnUserJoinedChannel() const { return OnUserJoinedEvent; }
	virtual FOnChannelUserChanged& OnUserLeftChannel() const { return OnUserLeftEvent; }
	virtual FOnChannelUserChanged& OnChannelUserChanged() const { return OnUserChangedEvent; }

	DECLARE_EVENT_OneParam(USocialChatChannel, FOnMessageReceived, const FSocialChatMessageRef&);
	virtual FOnMessageReceived& OnMessageReceived() const { return OnMessageReceivedEvent; }

	DECLARE_EVENT_OneParam(USocialChatChannel, FOnChannelDisplayNameChanged, const FText&);
	virtual FOnChannelDisplayNameChanged& OnChannelDisplayNameChanged() const { return OnChannelDisplayNameChangedEvent; }

	virtual void Initialize(USocialUser* InSocialUser, const FChatRoomId& InChannelId, ESocialChannelType InSourceChannelType) PURE_VIRTUAL(USocialChatChannel::Initialize, return;);

	/**
	* Manually adds the given message to the channel's log locally. Representations of this channel on other clients will not receive the message.
	* Useful for adding custom messages that did not originate from a user.
	*/
	UE_API virtual void InjectLocalMessage(const TSharedRef<FSocialLocalChatMessage>& LocalMessage);
	virtual const FText& GetChannelDisplayName() const { return ChannelDisplayName; }
	virtual const TArray<FSocialChatMessageRef>& GetMessageHistory() const { return MessageHistory; }

	virtual void UpdateNow() {}
	virtual void SetAutoUpdatePeriod(float) {}

	/**
	* Sends a text message to all other users in this channel.
	* @return True if the message was sent successfully
	*/
	virtual bool SendMessage(const FString& Message) PURE_VIRTUAL(USocialChatChannel::SendMessage, return false;)

	UE_API void SetChannelDisplayName(const FText& InDisplayName);
	
	UE_API void NotifyUserJoinedChannel(USocialUser& InUser);
	UE_API void NotifyUserLeftChannel(USocialUser& InUser);
	UE_API void NotifyChannelUserChanged(USocialUser& InUser);
	UE_API void NotifyMessageReceived(const TSharedRef<FChatMessage>& InChatMessage);
	
	UE_API virtual void ListenToChannel(USocialChatChannel& Channel);

	UE_API virtual void HandleListenedChannelMessageReceived(const FSocialChatMessageRef& Message, USocialChatChannel* SourceChannel);

	ESocialChannelType GetChannelType() const { return ChannelType; }
	void SetChannelType(ESocialChannelType InType) { ChannelType = InType; }

	virtual bool SupportsMessageSending() const { return true; }
	
	DECLARE_EVENT_OneParam(USocialChatChannel, FOnHiddenChanged, bool);
	FOnHiddenChanged& OnHiddenChanged() { return OnHiddenChangedEvent; }
	bool GetIsHidden() const { return bIsHidden; }
	UE_API void SetIsHidden(bool InValue);

	// used by external classes to duplicate a message into a channel that didn't otherwise receive it
	UE_API void AddMirroredMessage(FSocialChatMessageRef NewMessage);
	UE_API void AddSystemMessage(const FText& MessageBody);
protected:

	UE_API IOnlineChatPtr GetChatInterface() const;
	UE_API void SanitizeMessage(FString& RawMessage) const;

	UE_API void AddMessageInternal(FSocialChatMessageRef NewMessage);
	
	FText ChannelDisplayName;
	ESocialChannelType ChannelType;

	UE_API USocialToolkit& GetOwningToolkit() const;

private:
	bool bIsHidden = false;
	FOnHiddenChanged OnHiddenChangedEvent;

	TArray<FSocialChatMessageRef> MessageHistory;

	mutable FOnChannelUserChanged OnUserJoinedEvent;
	mutable FOnChannelUserChanged OnUserLeftEvent;
	mutable FOnChannelUserChanged OnUserChangedEvent;
	mutable FOnMessageReceived OnMessageReceivedEvent;
	mutable FOnChannelDisplayNameChanged OnChannelDisplayNameChangedEvent;

};

#undef UE_API
