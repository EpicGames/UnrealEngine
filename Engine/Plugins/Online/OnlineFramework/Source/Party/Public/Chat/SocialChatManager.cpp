// Copyright Epic Games, Inc. All Rights Reserved.

#include "SocialChatManager.h"

#include "Chat/SocialChatRoom.h"
#include "Chat/SocialGroupChannel.h"
#include "Chat/SocialPrivateMessageChannel.h"
#include "SocialPartyChatRoom.h"
#include "Chat/SocialReadOnlyChatChannel.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SocialChatManager)

USocialChatRoom* USocialChatManager::GetChatRoom(const FChatRoomId& ChannelId) const
{
	if (TObjectPtr<USocialChatRoom>const* FoundChannel = ChatRoomsById.Find(ChannelId))
	{
		return *FoundChannel;
	}
	return nullptr;
}

void USocialChatManager::GetJoinedChannels(TArray<USocialChatChannel*>& JoinedChannels) const
{
	JoinedChannels.Reset();

	TArray<decltype(ChatRoomsById)::ValueType> JoinedRooms;
	ChatRoomsById.GenerateValueArray(JoinedRooms);
	JoinedChannels.Append(JoinedRooms);

	TArray<decltype(DirectChannelsByTargetUser)::ValueType> JoinedDirectChannels;
	DirectChannelsByTargetUser.GenerateValueArray(JoinedDirectChannels);
	JoinedChannels.Append(JoinedDirectChannels);

	TArray<decltype(ReadOnlyChannelsByDisplayName)::ValueType> ReadOnlyChannels;
	ReadOnlyChannelsByDisplayName.GenerateValueArray(ReadOnlyChannels);
	JoinedChannels.Append(ReadOnlyChannels);
}

void USocialChatManager::JoinChatRoomPublic(const FChatRoomId& RoomId, const FChatRoomConfig& InChatRoomConfig /*= FChatRoomConfig()*/, ESocialSubsystem InSocialSubsystem /*= ESocialSubsystem::Primary*/)
{
	if (!RoomId.IsEmpty())
	{
		IOnlineChatPtr ChatInterface = GetOnlineChatInterface(InSocialSubsystem);
		if (ChatInterface.IsValid())
		{
			USocialUser& LocalUser = GetOwningToolkit().GetLocalUser();
			const FUniqueNetIdPtr LocalUserNetId = LocalUser.GetUserId(InSocialSubsystem).GetUniqueNetId();
			check(LocalUserNetId.IsValid());
			ChatInterface->JoinPublicRoom(*LocalUserNetId.Get(), RoomId, LocalUser.GetDisplayName(InSocialSubsystem), InChatRoomConfig);
			UE_LOGF(LogOnline, VeryVerbose, "USocialChatManager::JoinChatRoomPublic - Attempting to join room %ls", *RoomId);
		}
		else
		{
			UE_LOGF(LogOnline, Warning, "USocialChatManager::JoinChatRoomPublic - Missing ChatInterface for subsystem %ls when asked to join room %ls", LexToString(InSocialSubsystem), *RoomId);
		}
	}
	else
	{
		UE_LOGF(LogOnline, Warning, "USocialChatManager::JoinChatRoomPublic - Missing RoomId when asked to join room");
	}
}

void USocialChatManager::JoinChatRoomPrivate(const FChatRoomId& RoomId, const FChatRoomConfig& InChatRoomConfig /*= FChatRoomConfig()*/, ESocialSubsystem InSocialSubsystem /*= ESocialSubsystem::Primary*/)
{
	if (!RoomId.IsEmpty())
	{
		IOnlineChatPtr ChatInterface = GetOnlineChatInterface(InSocialSubsystem);
		if (ChatInterface.IsValid())
		{
			USocialUser& LocalUser = GetOwningToolkit().GetLocalUser();
			const FUniqueNetIdPtr LocalUserNetId = LocalUser.GetUserId(InSocialSubsystem).GetUniqueNetId();
			check(LocalUserNetId.IsValid());
			ChatInterface->JoinPrivateRoom(*LocalUserNetId.Get(), RoomId, LocalUser.GetDisplayName(InSocialSubsystem), InChatRoomConfig);
			UE_LOGF(LogOnline, VeryVerbose, "USocialChatManager::JoinChatRoomPrivate - Attempting to join room %ls", *RoomId);
		}
		else
		{
			UE_LOGF(LogOnline, Warning, "USocialChatManager::JoinChatRoomPrivate - Missing ChatInterface for subsystem %ls when asked to join room %ls", LexToString(InSocialSubsystem), *RoomId);
		}
	}
	else
	{
		UE_LOGF(LogOnline, Warning, "USocialChatManager::JoinChatRoomPrivate - Missing RoomId when asked to join room");
	}
}

void USocialChatManager::ExitChatRoom(const FChatRoomId& RoomId, ESocialSubsystem InSocialSubsystem /*= ESocialSubsystem::Primary*/)
{
	if (!RoomId.IsEmpty())
	{
		IOnlineChatPtr ChatInterface = GetOnlineChatInterface(InSocialSubsystem);
		if (ChatInterface.IsValid())
		{
			USocialUser& LocalUser = GetOwningToolkit().GetLocalUser();
			const FUniqueNetIdPtr LocalUserNetId = LocalUser.GetUserId(InSocialSubsystem).GetUniqueNetId();
			check(LocalUserNetId.IsValid());
			ChatInterface->ExitRoom(*LocalUserNetId.Get(), RoomId);
			UE_LOGF(LogOnline, VeryVerbose, "USocialChatManager::ExitChatRoom - Attempting to exit room %ls", *RoomId);
		}
		else
		{
			UE_LOGF(LogOnline, Warning, "USocialChatManager::ExitChatRoom - Missing ChatInterface for subsystem %ls when asked to join room %ls", LexToString(InSocialSubsystem), *RoomId);
		}
	}
	else
	{
		UE_LOGF(LogOnline, Warning, "USocialChatManager::ExitChatRoom - Missing RoomId when asked to join room");
	}
}

void USocialChatManager::OnChannelCreatedInternal(USocialChatChannel& CreatedChannel)
{
	ESocialChannelType ChannelType = CreatedChannel.GetChannelType();
	switch (ChannelType)
	{
		case ESocialChannelType::Founder:
		case ESocialChannelType::General:
		case ESocialChannelType::Party:
		case ESocialChannelType::Team:
		{
			ChannelsByType.Add(ChannelType, &CreatedChannel);
		}
		break;
	}

	OnChannelCreated().Broadcast(CreatedChannel);
}

void USocialChatManager::OnChannelLeftInternal(USocialChatChannel& ChannelLeft)
{
	ESocialChannelType ChannelType = ChannelLeft.GetChannelType();
	ChannelsByType.Remove(ChannelType);

	OnChannelLeft().Broadcast(ChannelLeft);
}

USocialChatChannel& USocialChatManager::CreateChatChannel(USocialUser& InRecipient)
{
	return FindOrCreateChannel(InRecipient);
}

USocialChatChannel* USocialChatManager::CreateChatChannel(FSocialChatChannelConfig& InConfig)
{
	ensure(InConfig.SocialUser || !InConfig.RoomId.IsEmpty() || !InConfig.DisplayName.IsEmpty());
	USocialChatChannel* CreatedChannel = nullptr;

	if (InConfig.SocialUser)
	{
		CreatedChannel = &FindOrCreateChannel(*InConfig.SocialUser);
	}
	else if (!InConfig.RoomId.IsEmpty())
	{
		CreatedChannel = &FindOrCreateRoom(InConfig.RoomId);
	}
	else if (!InConfig.DisplayName.IsEmpty())
	{
		CreatedChannel = &FindOrCreateChannel(InConfig.DisplayName);
	}

	if (CreatedChannel)
	{
		CreatedChannel->SetChannelDisplayName(InConfig.DisplayName);
		for (USocialChatChannel* Channel : InConfig.ListenChannels)
		{
			if (ensure(Channel))
			{
				CreatedChannel->ListenToChannel(*Channel);
			}
		}
	}

	return CreatedChannel;
}

void USocialChatManager::FocusChatChannel(USocialUser& InChannelUser)
{
	USocialChatChannel& Channel = FindOrCreateChannel(InChannelUser);
	OnChannelFocusRequestedEvent.Broadcast(Channel);
}

void USocialChatManager::FocusChatChannel(USocialChatChannel& InChannel)
{
	OnChannelFocusRequestedEvent.Broadcast(InChannel);
}

void USocialChatManager::DisplayChatChannel(USocialChatChannel& InChannel)
{
	OnChannelDisplayRequestedEvent.Broadcast(InChannel);
}

TSubclassOf<USocialChatRoom> USocialChatManager::GetClassForChatRoom(ESocialChannelType Type) const
{
	if (Type == ESocialChannelType::Party)
	{
		return USocialPartyChatRoom::StaticClass();
	}

	return USocialChatRoom::StaticClass();
}

bool USocialChatManager::IsChatRestricted() const
{
	return false;
}

USocialToolkit& USocialChatManager::GetOwningToolkit() const
{
	USocialToolkit* OwnerToolkit = GetOuterUSocialToolkit();
	check(OwnerToolkit);
	return *OwnerToolkit;
}

USocialChatChannel* USocialChatManager::GetChatRoomForType(ESocialChannelType Key)
{
	if (TWeakObjectPtr<USocialChatChannel>* Channel = ChannelsByType.Find(Key))
	{
		return Channel->Get();
	}
	return nullptr;
}

// This should only be possible for external code to do this via an ISocialUser directly
// The user will hold a weak ptr to the direct message channel - if invalid, it'll ask the chat manager to make a channel for it
// That way we still hit all the channel creation events but give the user fast access when sending whispers
// The other half of it is receiving a whisper, which will be handled here, creating the channel if needed and associating it with the user (maybe?)
//void USocialChatManager::SendDirectMessage(const ISocialUserRef& InRecipient, const FString& InMessage)
//{
//	// Make sure the recipient isn't the local user (no reason to support messaging yourself)
//	// If the channel doesn't exist yet, make it
//	// Send the message through the channel
//}

USocialChatManager* USocialChatManager::CreateChatManager(USocialToolkit& InOwnerToolkit)
{
	USocialChatManager* ChatManager = NewObject<USocialChatManager>(&InOwnerToolkit, InOwnerToolkit.GetChatManagerClass());
	check(ChatManager);

	ChatManager->InitializeChatManager();
	return ChatManager;
}

void USocialChatManager::InitializeChatManager()
{
	USocialToolkit& OwningToolkit = GetOwningToolkit();

	IOnlineChatPtr ChatInterface = GetOnlineChatInterface();
	if (ChatInterface.IsValid())
	{
		ChatInterface->AddOnChatRoomCreatedDelegate_Handle(FOnChatRoomCreatedDelegate::CreateUObject(this, &USocialChatManager::HandleChatRoomCreated));
		ChatInterface->AddOnChatRoomConfiguredDelegate_Handle(FOnChatRoomConfiguredDelegate::CreateUObject(this, &USocialChatManager::HandleChatRoomConfigured));
		ChatInterface->AddOnChatRoomJoinPublicDelegate_Handle(FOnChatRoomJoinPublicDelegate::CreateUObject(this, &USocialChatManager::HandleChatRoomJoinPublic));
		ChatInterface->AddOnChatRoomJoinPrivateDelegate_Handle(FOnChatRoomJoinPrivateDelegate::CreateUObject(this, &USocialChatManager::HandleChatRoomJoinPrivate));
		ChatInterface->AddOnChatRoomExitDelegate_Handle(FOnChatRoomExitDelegate::CreateUObject(this, &USocialChatManager::HandleChatRoomExit));
		ChatInterface->AddOnChatRoomMemberJoinDelegate_Handle(FOnChatRoomMemberJoinDelegate::CreateUObject(this, &USocialChatManager::HandleChatRoomMemberJoin));
		ChatInterface->AddOnChatRoomMemberExitDelegate_Handle(FOnChatRoomMemberExitDelegate::CreateUObject(this, &USocialChatManager::HandleChatRoomMemberExit));
		ChatInterface->AddOnChatRoomMemberUpdateDelegate_Handle(FOnChatRoomMemberUpdateDelegate::CreateUObject(this, &USocialChatManager::HandleChatRoomMemberUpdate));
		ChatInterface->AddOnChatRoomMessageSentDelegate_Handle(FOnChatRoomMessageSentDelegate::CreateUObject(this, &USocialChatManager::HandleChatRoomMessageSent));
		ChatInterface->AddOnChatRoomMessageReceivedDelegate_Handle(FOnChatRoomMessageReceivedDelegate::CreateUObject(this, &USocialChatManager::HandleChatRoomMessageReceived));
		ChatInterface->AddOnChatPrivateMessageSentDelegate_Handle(FOnChatPrivateMessageSentDelegate::CreateUObject(this, &USocialChatManager::HandleChatPrivateMessageSent));
		ChatInterface->AddOnChatPrivateMessageReceivedDelegate_Handle(FOnChatPrivateMessageReceivedDelegate::CreateUObject(this, &USocialChatManager::HandleChatPrivateMessageReceived));
	}
	else
	{
		// separate warning?  this is the expected result of running a subsystem-less execution (testing / no network)
	}
}

IOnlineChatPtr USocialChatManager::GetOnlineChatInterface(ESocialSubsystem InSocialSubsystem) const
{
	// Chat expects to only operate via the primary subsystem
	if (IOnlineSubsystem* OnlineSub = GetOwningToolkit().GetSocialOss(InSocialSubsystem))
	{
		return OnlineSub->GetChatInterface();
	}

	return nullptr;
}

USocialChatRoom& USocialChatManager::FindOrCreateRoom(const FChatRoomId& RoomId)
{
	if (TObjectPtr<USocialChatRoom>* Channel = ChatRoomsById.Find(RoomId))
	{
		return **Channel;
	}

	// @todo don.eubanks - ChannelType needs to be converted to an FName so that the game can extend it and do its own tracking
	ESocialChannelType ChannelType = TryChannelTypeLookupByRoomId(RoomId);

	TSubclassOf<USocialChatRoom> NewRoomClass = GetClassForChatRoom(ChannelType);
	check(NewRoomClass);

	USocialChatRoom* NewRoomChannel = NewObject<USocialChatRoom>(this, NewRoomClass);
	check(NewRoomChannel);

	NewRoomChannel->Initialize(&GetOwningToolkit().GetLocalUser(), RoomId, ChannelType);
	ChatRoomsById.Add(RoomId, NewRoomChannel);
	OnChannelCreatedInternal(*NewRoomChannel);

	return *NewRoomChannel;
}

USocialChatChannel& USocialChatManager::FindOrCreateChannel(USocialUser& SocialUser)
{
	if (TObjectPtr<USocialPrivateMessageChannel>* Channel = DirectChannelsByTargetUser.Find(&SocialUser))
	{
		return **Channel;
	}

	TSubclassOf<USocialChatChannel> NewPMChannelClass = GetClassForPrivateMessage();
	check(NewPMChannelClass);

	USocialPrivateMessageChannel* NewPMChannel = NewObject<USocialPrivateMessageChannel>(this, NewPMChannelClass);
	check(NewPMChannel);

	NewPMChannel->Initialize(&SocialUser, FChatRoomId(TEXT("private")), ESocialChannelType::Private);
	DirectChannelsByTargetUser.Add(&SocialUser, NewPMChannel);
	OnChannelCreatedInternal(*NewPMChannel);

	return *NewPMChannel;
}

USocialChatChannel& USocialChatManager::FindOrCreateChannel(const FText& DisplayName)
{
	if (TObjectPtr<USocialReadOnlyChatChannel>* Channel = ReadOnlyChannelsByDisplayName.Find(DisplayName.ToString()))
	{
		return **Channel;
	}

	TSubclassOf<USocialChatChannel> NewReadOnlyChannelClass = GetClassForReadOnlyChannel();
	check(NewReadOnlyChannelClass);

	USocialReadOnlyChatChannel* NewReadOnlyChannel = NewObject<USocialReadOnlyChatChannel>(this, NewReadOnlyChannelClass);
	check(NewReadOnlyChannel);

	NewReadOnlyChannel->Initialize(&GetOwningToolkit().GetLocalUser(), DisplayName.ToString(), ESocialChannelType::General);
	ReadOnlyChannelsByDisplayName.Add(DisplayName.ToString(), NewReadOnlyChannel);
	OnChannelCreatedInternal(*NewReadOnlyChannel);

	return *NewReadOnlyChannel;
}

void USocialChatManager::HandleChatRoomCreated(const FUniqueNetId& LocalUserId, const FChatRoomId& RoomId, bool bWasSuccessful, const FString& Error)
{
	if (IsUniqueIdOfOwner(LocalUserId))
	{
		if (bWasSuccessful)
		{
			FindOrCreateRoom(RoomId);
		}
		else
		{
			HandleChatRoomCreatedFailure(LocalUserId, RoomId, Error);
		}
	}
}

void USocialChatManager::HandleChatRoomConfigured(const FUniqueNetId& LocalUserId, const FChatRoomId& RoomId, bool bWasSuccessful, const FString& Error)
{
	if (IsUniqueIdOfOwner(LocalUserId))
	{
		if (bWasSuccessful)
		{
			FindOrCreateRoom(RoomId);
		}
		else
		{
			HandleChatRoomConfiguredFailure(LocalUserId, RoomId, Error);
		}
	}
}

void USocialChatManager::HandleChatRoomJoinPublic(const FUniqueNetId& LocalUserId, const FChatRoomId& RoomId, bool bWasSuccessful, const FString& Error)
{
	if (IsUniqueIdOfOwner(LocalUserId))
	{
		if (bWasSuccessful)
		{
			FindOrCreateRoom(RoomId);
		}
		else
		{
			HandleChatRoomJoinPublicFailure(LocalUserId, RoomId, Error);
		}
	}
}

void USocialChatManager::HandleChatRoomJoinPrivate(const FUniqueNetId& LocalUserId, const FChatRoomId& RoomId, bool bWasSuccessful, const FString& Error)
{
	if (IsUniqueIdOfOwner(LocalUserId))
	{
		if (bWasSuccessful)
		{
			FindOrCreateRoom(RoomId);
		}
		else
		{
			HandleChatRoomJoinPrivateFailure(LocalUserId, RoomId, Error);
		}
	}
}

void USocialChatManager::HandleChatRoomExit(const FUniqueNetId& LocalUserId, const FChatRoomId& RoomId, bool bWasSuccessful, const FString& Error)
{
	if (IsUniqueIdOfOwner(LocalUserId))
	{
		if (bWasSuccessful)
		{
			TObjectPtr<USocialChatRoom>* Room = ChatRoomsById.Find(RoomId);
			if (ensure(Room))
			{
				ChatRoomsById.Remove(RoomId);
				OnChannelLeftInternal(**Room);
			}
		}
		else
		{
			HandleChatRoomExitFailure(LocalUserId, RoomId, Error);
		}
	}
}

void USocialChatManager::HandleChatRoomMemberJoin(const FUniqueNetId& LocalUserId, const FChatRoomId& RoomId, const FUniqueNetId& MemberId)
{
	if (IsUniqueIdOfOwner(LocalUserId))
	{
		// This is potentially a previously unknown user, so establish them first
		GetOwningToolkit().QueueUserDependentAction(MemberId.AsShared(),
			[this, RoomId](USocialUser& User)
		{
			if (TObjectPtr<USocialChatRoom>* Channel = ChatRoomsById.Find(RoomId))
			{
				(*Channel)->NotifyUserJoinedChannel(User);
			}
		});
	}
}

void USocialChatManager::HandleChatRoomMemberExit(const FUniqueNetId& LocalUserId, const FChatRoomId& RoomId, const FUniqueNetId& MemberId)
{
	if (IsUniqueIdOfOwner(LocalUserId))
	{
		//@todo SocialChat: Should the channel be tracking users in it? Should be up to a SocialUserList, shouldn't it? 
		//	Maybe a channel is actually a UserList? Overkill for PMs, but sounds reasonable for a ChatRoom
		USocialChatRoom* Channel = GetChatRoom(RoomId);
		USocialUser* SocialUser = GetOwningToolkit().FindUser(MemberId.AsShared());
		if (ensure(Channel) && SocialUser)
		{
			Channel->NotifyUserLeftChannel(*SocialUser);
		}
	}
}

void USocialChatManager::HandleChatRoomMemberUpdate(const FUniqueNetId& LocalUserId, const FChatRoomId& RoomId, const FUniqueNetId& MemberId)
{
	if (IsUniqueIdOfOwner(LocalUserId))
	{
		USocialChatRoom* Channel = GetChatRoom(RoomId);
		USocialUser* SocialUser = GetOwningToolkit().FindUser(MemberId.AsShared());
		if (ensure(Channel) && ensure(SocialUser))
		{
			Channel->NotifyChannelUserChanged(*SocialUser);
		}
	}
}

void USocialChatManager::HandleChatRoomMessageSent(const FUniqueNetId& LocalUserId, const FChatRoomId& RoomId, bool bWasSuccessful, const FString& Error)
{
}

void USocialChatManager::HandleChatRoomMessageReceived(const FUniqueNetId& LocalUserId, const FChatRoomId& RoomId, const TSharedRef<FChatMessage>& ChatMessage)
{
	if (IsUniqueIdOfOwner(LocalUserId))
	{
		USocialChatRoom* Channel = GetChatRoom(RoomId);
		if (ensure(Channel))
		{
			Channel->NotifyMessageReceived(ChatMessage);
		}
	}
}

void USocialChatManager::HandleChatPrivateMessageSent(const FUniqueNetId& LocalUserId, const FUniqueNetId& RecipientId, bool bWasSuccessful, const FString& Error)
{
}

void USocialChatManager::HandleChatPrivateMessageReceived(const FUniqueNetId& LocalUserId, const TSharedRef<FChatMessage>& ChatMessage)
{
	if (IsUniqueIdOfOwner(LocalUserId))
	{
		// We can expect that we already know about a user that is sending us a private message and should not have to create one
		USocialUser* SocialUser = GetOwningToolkit().FindUser(ChatMessage->GetUserId());
		if (ensure(SocialUser))
		{
			USocialChatChannel& Channel = FindOrCreateChannel(*SocialUser);
			Channel.NotifyMessageReceived(ChatMessage);
		}
	}
}

ESocialChannelType USocialChatManager::TryChannelTypeLookupByRoomId(const FChatRoomId& RoomID)
{
	return ESocialChannelType::System;
}


//----------------------------------------------------------------------
// KIAROS GROUP MANAGEMENT, tbd channels?

void USocialChatManager::InitializeGroupChannels()
{
	ESocialSubsystem InSocialSubsystem = ESocialSubsystem::Primary;

	GetOwningToolkit().QueueUserDependentAction(
		GetOwningToolkit().GetLocalUserNetId(InSocialSubsystem), FUserDependentAction::CreateUObject(this, &ThisClass::LocalUserInitialized));
}

void USocialChatManager::LocalUserInitialized(USocialUser& LocalUser)
{
	ESocialSubsystem InSocialSubsystem = ESocialSubsystem::Primary;

	IOnlineGroupsPtr GroupInterface = GetOnlineGroupInterface();
	if (GroupInterface.IsValid())
	{
		GroupInterface->OnGroupUpdated.AddUObject(this, &ThisClass::OnGroupUpdated);

		const FUniqueNetIdPtr LocalUserNetId = LocalUser.GetUserId(InSocialSubsystem).GetUniqueNetId();
		check(LocalUserNetId.IsValid());

		GroupInterface->QueryUserMembership(*LocalUserNetId.Get(), *LocalUserNetId.Get(), FOnGroupsRequestCompleted::CreateUObject(this, &ThisClass::RefreshGroupsRequestCompleted));
	}
}

void USocialChatManager::RefreshGroupsRequestCompleted(FGroupsResult Result)
{
	IOnlineGroupsPtr GroupInterface = GetOnlineGroupInterface();
	if (GroupInterface.IsValid())
	{
		USocialToolkit& OwningToolkit = GetOwningToolkit();

		USocialUser& LocalUser = OwningToolkit.GetLocalUser();
		const FUniqueNetIdPtr LocalUserNetId = LocalUser.GetUserId(ESocialSubsystem::Primary).GetUniqueNetId();
		check(LocalUserNetId.IsValid());

		TSharedPtr<const IUserMembership> Membership = GroupInterface->GetCachedUserMembership(*LocalUserNetId.Get(), *LocalUserNetId.Get());
		if (Membership.IsValid())
		{
			TArray<FUserMembershipEntry> GroupMemberships;
			Membership->CopyEntries(GroupMemberships);

			for (const FUserMembershipEntry& Entry : GroupMemberships)
			{
				USocialGroupChannel& GroupChannel = FindOrCreateGroupChannel(GroupInterface, *Entry.GroupId.Get());
				GroupChannel.SetDisplayName(Entry.Name);
			}
		}
	}
}

void USocialChatManager::OnGroupUpdated(const FUniqueNetId& GroupId)
{
}

bool USocialChatManager::IsUniqueIdOfOwner(const FUniqueNetId& LocalUserId) const
{
	const USocialUser& Owner = GetOwningToolkit().GetLocalUser();
	return LocalUserId == Owner.GetUserId(ESocialSubsystem::Primary);
}

void USocialChatManager::GetGroupChannels(TArray<USocialGroupChannel*>& JoinedChannels) const
{
	for (auto& Entry : GroupChannels)
	{
		JoinedChannels.Add(Entry.Value);
	}
}

IOnlineGroupsPtr USocialChatManager::GetOnlineGroupInterface(ESocialSubsystem InSocialSubsystem) const
{
	if (IOnlineSubsystem* OnlineSub = GetOwningToolkit().GetSocialOss(InSocialSubsystem))
	{
		return OnlineSub->GetGroupsInterface();
	}

	return nullptr;
}

USocialGroupChannel& USocialChatManager::FindOrCreateGroupChannel(IOnlineGroupsPtr InGroupInterface, const FUniqueNetId& GroupId)
{
	TSubclassOf<USocialGroupChannel> NewGroupClass = GetClassForGroupChannel();
	check(NewGroupClass);

	USocialGroupChannel* NewGroupChannel = NewObject<USocialGroupChannel>(this, NewGroupClass);
	check(NewGroupChannel);

	GroupChannels.Add(GroupId.AsShared(), NewGroupChannel);

	NewGroupChannel->Initialize(InGroupInterface, GetOwningToolkit().GetLocalUser(), GroupId);

	//TODO
	//OnChannelCreated().Broadcast(*NewRoomChannel);

	return *NewGroupChannel;
}

//----------------------------------------------------------------------
