// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Interfaces/OnlineChatInterface.h"
#include "Misc/DateTime.h"
#include "User/SocialUser.h"

#define UE_API PARTY_API

enum class ESocialChannelType : uint8;

/**
* Barebones RTTI for chat messages to allow games to generate messages of custom types.
* The alternative is to make them UObjects, but for the potential quantity of objects in play with messages, that feels unwise
*/
#define DERIVED_CHAT_MESSAGE(MessageType, ParentMessageType)	\
protected:	\
	virtual bool IsDerivedFrom(FName MessageTypeName) const override { return StaticMessageType() == MessageTypeName || ParentMessageType::IsDerivedFrom(MessageTypeName); }	\
public:	\
	static FName StaticMessageType() { return FName(#MessageType); }	\

/** Represents a single message sent within a chat channel */
class FSocialChatMessage : public TSharedFromThis<FSocialChatMessage>
{
public:
	virtual ~FSocialChatMessage() = default;

	const FString& GetMessageBody() const { return MessageBody; }
	const FDateTime& GetTimestamp() const { return Timestamp; }

	template <typename ChatMessageT>
	ChatMessageT* AsTypedMessage()
	{
		if (IsDerivedFrom(ChatMessageT::StaticMessageType()))
		{
			return static_cast<ChatMessageT*>(this);
		}
		return nullptr;
	}

	void SetPreviousMessage(TSharedPtr<FSocialChatMessage> InPrevious) { PreviousMessage = InPrevious; }
	TSharedPtr<FSocialChatMessage> GetPreviousMessage() { return PreviousMessage; }

	virtual FString GetSourceName() const PURE_VIRTUAL(FSocialChatMessage::GetSourceName, return TEXT(""););
	ESocialChannelType GetSourceChannelType() const { return SourceChannelType; }

protected:
	FSocialChatMessage(const FString& InMessageBody, ESocialChannelType InSourceChannelType)
		: MessageBody(InMessageBody)
		, Timestamp(FDateTime::UtcNow())
		, SourceChannelType(InSourceChannelType)
	{}

	FSocialChatMessage(const FChatMessage& ChatMessage, ESocialChannelType InSourceChannelType)
		: MessageBody(ChatMessage.GetBody())
		, Timestamp(ChatMessage.GetTimestamp())
		, SourceChannelType(InSourceChannelType)
	{}

	static FName StaticMessageType() { return TEXT("Base"); }
	virtual bool IsDerivedFrom(FName MessageTypeName) const { return MessageTypeName == StaticMessageType(); }

private:
	FString MessageBody;
	FDateTime Timestamp;
	TSharedPtr<FSocialChatMessage> PreviousMessage;
	ESocialChannelType SourceChannelType;
};


/** A chat message that originated from a particular SocialUser - by far the most common type of message */
class FSocialUserChatMessage : public FSocialChatMessage
{
	DERIVED_CHAT_MESSAGE(FSocialUserChatMessage, FSocialChatMessage)

public:
	static FSocialChatMessageRef Create(USocialUser& Sender, const FChatMessage& Message, ESocialChannelType SourceChannelType)
	{
		return MakeShareable(new FSocialUserChatMessage(Sender, Message, SourceChannelType));
	}

	static FSocialChatMessageRef Create(USocialUser& Sender, const FString& MessageBody, ESocialChannelType SourceChannelType)
	{
		return MakeShareable(new FSocialUserChatMessage(Sender, MessageBody, SourceChannelType));
	}

	UE_API virtual FString GetSourceName() const override;
	USocialUser* GetSender() const { return FromUser.Get(); }

protected:
	FSocialUserChatMessage(USocialUser& Sender, const FChatMessage& Message, ESocialChannelType InSourceChannelType)
		: FSocialChatMessage(Message, InSourceChannelType)
		, FromUser(&Sender)
	{}

	FSocialUserChatMessage(USocialUser& Sender, const FString& MessageBody, ESocialChannelType InSourceChannelType)
		: FSocialChatMessage(MessageBody, InSourceChannelType)
		, FromUser(&Sender)
	{}

#if WITH_EDITOR
	FSocialUserChatMessage(const FString& MessageBody, ESocialChannelType InSourceChannelType)
		: FSocialChatMessage(MessageBody, InSourceChannelType)
		, FromUser(nullptr)
	{}
#endif

	TWeakObjectPtr<USocialUser> FromUser;
};

/**
* A locally generated chat message that was not sent by a particular user.
* Use cases include server admin messages, in-game notifications, etc.
*/
class FSocialLocalChatMessage : public FSocialChatMessage
{
	DERIVED_CHAT_MESSAGE(FSocialLocalChatMessage, FSocialChatMessage)

public:
	virtual FString GetSourceName() const override { return SourceName; }

protected:
	FSocialLocalChatMessage(const FString& InSourceName, const FString& MessageBody, ESocialChannelType SourceChannelType)
		: FSocialChatMessage(MessageBody, SourceChannelType)
		, SourceName(InSourceName)
	{}

	FString SourceName;
};

enum class EChatSystemMessagePurpose : uint8
{
	Info,
	Warning,
	Error,
};

class FSocialSystemChatMessage : public FSocialLocalChatMessage
{
	DERIVED_CHAT_MESSAGE(FSocialSystemChatMessage, FSocialLocalChatMessage)

public:
	static FSocialChatMessageRef Create(const FString& InSourceName, const FString& InMessageBody, ESocialChannelType InSourceChannelType, EChatSystemMessagePurpose InPurpose)
	{
		return MakeShareable(new FSocialSystemChatMessage(InSourceName, InMessageBody, InSourceChannelType, InPurpose));
	}

protected:
	FSocialSystemChatMessage(const FString& InSourceName, const FString& InMessageBody, ESocialChannelType InSourceChannelType, EChatSystemMessagePurpose InPurpose)
		: FSocialLocalChatMessage(InSourceName, InMessageBody, InSourceChannelType)
		, Purpose(InPurpose)
	{}

	EChatSystemMessagePurpose Purpose;
};

#if WITH_EDITOR
class FDesignerPreviewSocialUserChatMessage : public FSocialUserChatMessage
{
	DERIVED_CHAT_MESSAGE(FDesignerPreviewSocialUserChatMessage, FSocialUserChatMessage)

public:
	static FSocialChatMessageRef Create(FString& InSenderName, const FString& MessageBody, ESocialChannelType SourceChannelType)
	{
		return MakeShareable(new FDesignerPreviewSocialUserChatMessage(InSenderName, MessageBody, SourceChannelType));
	}

	virtual FString GetSourceName() const override { return SenderName; }

protected:
	FDesignerPreviewSocialUserChatMessage(FString& InSenderName, const FString& InMessage, ESocialChannelType InSourceChannelType)
		: FSocialUserChatMessage(InMessage, InSourceChannelType)
		, SenderName(InSenderName)
	{}

	FString SenderName;
};
#endif

#undef UE_API
