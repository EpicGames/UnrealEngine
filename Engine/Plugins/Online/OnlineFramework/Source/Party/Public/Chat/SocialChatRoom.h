// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SocialChatChannel.h"
#include "SocialChatRoom.generated.h"

#define UE_API PARTY_API

class USocialChatManager;

/** A multi-user chat room channel. Used for all chat situations outside of private user-to-user direct messages. */
UCLASS(MinimalAPI)
class USocialChatRoom : public USocialChatChannel
{
	GENERATED_BODY()

public:
	UE_API virtual void Initialize(USocialUser* InSocialUser, const FChatRoomId& InChannelId, ESocialChannelType InSourceChannelType);

	/*virtual bool HasJoinedChatRoom() const override;

	virtual bool LeaveChannel() override;*/

	UE_API virtual bool SendMessage(const FString& InMessage) override;
	const FChatRoomId& GetChatRoomId() const { return RoomId; }
	
	/*virtual void NotifyUserJoinedChannel(const ISocialUserRef& InUser) override;
	virtual void NotifyUserLeftChannel(const ISocialUserRef& InUser) override;
	virtual void NotifyChannelUserChanged(const ISocialUserRef& InUser) override;
	virtual void NotifyMessageReceived(const TSharedRef<FChatMessage>& InChatMessage) override;*/

	//virtual bool HasUnreadMessages() const override;
	UE_API virtual const FText DetermineChannelDisplayName(ESocialChannelType InSourceChannelType, const FChatRoomId& InRoomId);

private:
	void SetRoomId(const FChatRoomId& Id) { RoomId = Id; }

	FChatRoomId RoomId;
};

#undef UE_API
