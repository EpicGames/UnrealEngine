// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Interfaces/OnlineGroupsInterface.h"
#include "User/SocialUser.h"
#include "SocialGroupChannel.generated.h"

#define UE_API PARTY_API

class USocialUser;

/**
 * 
 */
UCLASS(MinimalAPI)
class USocialGroupChannel : public UObject
{
	GENERATED_BODY()

public:
	UE_API USocialGroupChannel();

	UE_API void Initialize(IOnlineGroupsPtr InGroupInterface, USocialUser& InSocialUser, const FUniqueNetId& InGroupId);

	void SetDisplayName(const FText& InDisplayName) { DisplayName = InDisplayName; }
	FText GetDisplayName() const { return DisplayName; }

	const TArray<USocialUser*>& GetMembers() const { return Members; }

private:
	UE_API void RefreshCompleted_GroupInfo(FGroupsResult Result);
	UE_API void RefreshCompleted_Roster(FGroupsResult Result);

private:
	UPROPERTY()
	TObjectPtr<USocialUser> SocialUser;

	UPROPERTY()
	FUniqueNetIdRepl GroupId;

	UPROPERTY()
	FText DisplayName;

	UPROPERTY()
	TArray<TObjectPtr<USocialUser>> Members;

	TWeakPtr<IOnlineGroups, ESPMode::ThreadSafe> GroupInterfacePtr;
};

#undef UE_API
