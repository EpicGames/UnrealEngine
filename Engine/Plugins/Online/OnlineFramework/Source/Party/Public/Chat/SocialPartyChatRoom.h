// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SocialChatRoom.h"
#include "SocialPartyChatRoom.generated.h"

#define UE_API PARTY_API

class UPartyMember;
enum class ESocialChannelType : uint8;

class USocialChatManager;
enum class EMemberExitedReason : uint8;

/** A multi-user chat room channel. Used for all chat situations outside of private user-to-user direct messages. */
UCLASS(MinimalAPI)
class USocialPartyChatRoom : public USocialChatRoom
{
	GENERATED_BODY()

public:
	UE_API virtual void Initialize(USocialUser* InSocialUser, const FChatRoomId& InChannelId, ESocialChannelType InSourceChannelType);

protected:
	UE_API virtual void HandlePartyMemberLeft(EMemberExitedReason Reason);
	UE_API virtual void HandlePartyMemberJoined(UPartyMember& NewMember);
};

#undef UE_API
