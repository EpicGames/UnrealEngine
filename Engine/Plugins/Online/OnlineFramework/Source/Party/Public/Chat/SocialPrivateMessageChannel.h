// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Chat/SocialChatChannel.h"
#include "SocialPrivateMessageChannel.generated.h"

#define UE_API PARTY_API

/**
 * A modified version of a chat room that only contains two participants - the current user and a private recipient of their messages.
 * This is equivalent to sending a "whisper".
 */
UCLASS(MinimalAPI)
class USocialPrivateMessageChannel : public USocialChatChannel
{
	GENERATED_BODY()

public:
	UE_API virtual void Initialize(USocialUser* InSocialUser, const FChatRoomId& InChannelId, ESocialChannelType InSourceChannelType) override;

	UE_API virtual bool SendMessage(const FString& InMessage) override;
	
	UE_API USocialUser& GetTargetUser() const;

private:
	void SetTargetUser(USocialUser& InTargetUser);

	/** The recipient of the current user's messages */
	UPROPERTY()
	TObjectPtr<USocialUser> TargetUser;
};

#undef UE_API
