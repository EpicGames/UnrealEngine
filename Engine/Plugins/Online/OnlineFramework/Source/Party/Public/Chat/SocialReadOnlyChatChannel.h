// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Chat/SocialChatChannel.h"
#include "SocialReadOnlyChatChannel.generated.h"

#define UE_API PARTY_API

class USocialChatManager;

/**
 * A strawman chat channel that relies exclusively on other channels messages for content, does not support sending messages
 */
UCLASS(MinimalAPI)
class USocialReadOnlyChatChannel : public USocialChatChannel
{
	GENERATED_BODY()

public:
	UE_API virtual void Initialize(USocialUser* InSocialUser, const FChatRoomId& InChannelId, ESocialChannelType InSourceChannelType) override;

	UE_API virtual bool SendMessage(const FString& InMessage) override;

	virtual bool SupportsMessageSending() const override { return false; }
};

#undef UE_API
