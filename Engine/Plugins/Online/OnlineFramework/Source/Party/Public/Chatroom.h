// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Engine/TimerHandle.h"
#include "Interfaces/OnlineChatInterface.h"
#include "Chatroom.generated.h"

#define UE_API PARTY_API

class Error;
class FTimerManager;
struct FUniqueNetIdRepl;

/**
 * Delegate fired when chat room attempt has completed
 *
 * @param ChatRoomId id of room joined
 * @param bAvailable has a chat room been joined
 */
DECLARE_DELEGATE_TwoParams(FOnChatRoomCreatedOrJoined, const FChatRoomId& /** ChatRoomId */, bool /** bAvailable */);

/**
 * Delegate fired when chat room has been left
 *
 * @param ChatRoomId id of room left
 */
DECLARE_DELEGATE_OneParam(FOnChatRoomLeft, const FChatRoomId& /** ChatRoomId */);

/**
 * Helper class for maintaining a single chat room at the game level
 */
UCLASS(MinimalAPI, Config = Game)
class UChatroom : public UObject
{
	GENERATED_BODY()

public:

	/** Ctor */
	UE_API UChatroom();

public:

	/**
	 * Chat room functions
	 */

	/**
	 * Try to join a chat room by calling CreateRoom, will either create or join if it already exists
	 *
	 * @param LocalUserId local user that will be joining the room
	 * @param ChatRoomId chat room id to connect with
	 * @param CompletionDelegate delegate fired when operation completes
	 * @param RoomConfig optional room configuration
     * NOTE: All parameters are by value because ClearTimer/SetTimer can destroy the internal lambda function
	 */
	UE_API void CreateOrJoinChatRoom(FUniqueNetIdRepl LocalUserId, FChatRoomId ChatRoomId, FOnChatRoomCreatedOrJoined CompletionDelegate, FChatRoomConfig RoomConfig = FChatRoomConfig());

	/**
	 * Leave the joined chat room
	 *
	 * @param LocalUserId user leaving the chat room
	 * @param CompletionDelegate delegate fired when operation completes
	 */
	UE_API void LeaveChatRoom(const FUniqueNetIdRepl& LocalUserId, const FOnChatRoomLeft& CompletionDelegate);

private:

	/**
	 * Delegate fired after CreateOrJoinChatRoom completes
	 *
	 * @param LocalUserId user joining the chat room
	 * @param RoomId id of room joined
	 * @param bWasSuccessful was the join successful
	 * @param Error error string if not successful
	 * @param CompletionDelegate user passed in delegate
	 * @param RoomConfig room configuration from previous call if needed on retry
	 */
	void OnChatRoomCreatedOrJoined(const FUniqueNetId& LocalUserId, const FChatRoomId& RoomId, bool bWasSuccessful, const FString& Error, FOnChatRoomCreatedOrJoined CompletionDelegate, FChatRoomConfig RoomConfig);

	/**
	 * Delegate fired after LeaveChatRoom completes
	 *
	 * @param LocalUserId user leaving the chat room
	 * @param RoomId id of room left
	 * @param bWasSuccessful was the leave successful
	 * @param Error error string if not successful
	 * @param ChatRoomIdCopy room we are trying to leave
	 * @param CompletionDelegate user passed in delegate
	 */
	void OnChatRoomLeft(const FUniqueNetId& LocalUserId, const FChatRoomId& RoomId, bool bWasSuccessful, const FString& Error, FChatRoomId ChatRoomIdCopy, FOnChatRoomLeft CompletionDelegate);

	/**
	 * Common code called at the end of chat room cleanup
	 *
	 * @param RoomId id of the room let
	 * @param CompletionDelegate delegate to fire to notify that the room has been left
	 */
	void ChatRoomLeftInternal(const FChatRoomId& RoomId, const FOnChatRoomLeft& CompletionDelegate);

private:

	/** Current chat room associated with this object (FString so UPROPERTY works) */
	UPROPERTY()
	FString CurrentChatRoomId;

	/** Max number of retries before giving up on chat */
	UPROPERTY()
	int32 MaxChatRoomRetries;

	/** Current number of retries on a chat room */
	UPROPERTY()
	int32 NumChatRoomRetries;

	/** Handle on chat room retry timer */
	FTimerHandle ChatRoomRetryTimerHandle;

	/** OSS delegate handles */
	FDelegateHandle ChatRoomCreatedDelegateHandle;
	FDelegateHandle ChatRoomLeftDelegateHandle;

protected:

	UE_API void UnregisterDelegates();

	/** @return true if the local player is signed in, false otherwise */
	UE_API virtual bool IsOnline() const;

	/** @return true if the user is already logged as being a part of a given chat room, false otherwise */
	UE_API bool IsAlreadyInChatRoom(const FUniqueNetIdRepl& LocalUserId, const FChatRoomId& ChatRoomId) const;

	UE_API UWorld* GetWorld() const;
	UE_API FTimerManager& GetTimerManager() const;
};

#undef UE_API
