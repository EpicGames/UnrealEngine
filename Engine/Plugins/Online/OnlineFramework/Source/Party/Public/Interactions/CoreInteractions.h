// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "SocialInteractionMacros.h"
#include "User/SocialUser.h"

/**
 * Core set of user/friend interactions supported by the Social framework
 */

DECLARE_SOCIAL_INTERACTION_EXPORT(PARTY_API, AddFriend);
DECLARE_SOCIAL_INTERACTION_EXPORT(PARTY_API, AddPlatformFriend);
DECLARE_SOCIAL_INTERACTION_EXPORT(PARTY_API, RemoveFriend);

DECLARE_SOCIAL_INTERACTION_EXPORT(PARTY_API, AcceptFriendInvite);
DECLARE_SOCIAL_INTERACTION_EXPORT(PARTY_API, RejectFriendInvite);

DECLARE_SOCIAL_INTERACTION_EXPORT(PARTY_API, Block);
DECLARE_SOCIAL_INTERACTION_EXPORT(PARTY_API, Unblock);
DECLARE_SOCIAL_INTERACTION_EXPORT(PARTY_API, PrivateMessage);

DECLARE_SOCIAL_INTERACTION_EXPORT(PARTY_API, ShowPlatformProfile);
