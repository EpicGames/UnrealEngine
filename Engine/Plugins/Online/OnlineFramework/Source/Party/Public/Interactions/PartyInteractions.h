// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "SocialInteractionMacros.h"
#include "User/SocialUser.h"

/**
* Default Party interactions supported by the Social framework
*/

DECLARE_SOCIAL_INTERACTION_EXPORT(PARTY_API, InviteToParty);
DECLARE_SOCIAL_INTERACTION_EXPORT(PARTY_API, JoinParty);

DECLARE_SOCIAL_INTERACTION_EXPORT(PARTY_API, RequestToJoinParty);
DECLARE_SOCIAL_INTERACTION_EXPORT(PARTY_API, AcceptJoinRequest);
DECLARE_SOCIAL_INTERACTION_EXPORT(PARTY_API, DismissJoinRequest);

DECLARE_SOCIAL_INTERACTION_EXPORT(PARTY_API, AcceptPartyInvite);
DECLARE_SOCIAL_INTERACTION_EXPORT(PARTY_API, RejectPartyInvite);

DECLARE_SOCIAL_INTERACTION_EXPORT(PARTY_API, LeaveParty);
DECLARE_SOCIAL_INTERACTION_EXPORT(PARTY_API, KickPartyMember);
DECLARE_SOCIAL_INTERACTION_EXPORT(PARTY_API, PromoteToPartyLeader);
