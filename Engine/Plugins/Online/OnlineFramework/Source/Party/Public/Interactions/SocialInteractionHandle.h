// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Containers/Map.h" // IWYU pragma: keep

#define UE_API PARTY_API

class FName;
class FText;

class USocialUser;
class ISocialInteractionWrapper;

/**
 * Represents a single discrete interaction between a local player and another user.
 * Useful for when you'd like to create some tangible list of interactions to compare/sort/classify/iterate.
 * Not explicitly required if you have a particular known interaction in mind - feel free to access the static API of a given interaction directly.
 */
class FSocialInteractionHandle
{
public:
	FSocialInteractionHandle() {}

	UE_API bool IsValid() const;
	UE_API bool operator==(const FSocialInteractionHandle& Other) const;
	bool operator!=(const FSocialInteractionHandle& Other) const { return !operator==(Other); }

	UE_API FName GetInteractionName() const;
	UE_API FText GetDisplayName(const USocialUser& User) const;
	UE_API FString GetSlashCommandToken() const;

	UE_API bool IsAvailable(const USocialUser& User) const;
	UE_API void ExecuteInteraction(USocialUser& User) const;
	UE_API void ExecuteInteractionWithContext(USocialUser& User, const TMap<FString, FString>& AnalyticsContext) const;

private:
	template <typename> friend class TSocialInteractionWrapper;
	UE_API FSocialInteractionHandle(const ISocialInteractionWrapper& Wrapper);

	const ISocialInteractionWrapper* InteractionWrapper = nullptr;
};

#undef UE_API
