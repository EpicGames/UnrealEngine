// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Interactions/SocialInteractionHandle.h"
#include "Internationalization/Text.h"

DECLARE_DELEGATE_RetVal_OneParam(bool, FOnCustomIsInteractionAvailable, const USocialUser&);

/**
 * Link between the class-polymorphism-based interaction handle and the static template-polymorphism-based interactions.
 * Implementation detail, automatically set up and utilized in the DECLARE_SOCIAL_INTERACTION macros above
 */
class ISocialInteractionWrapper
{
public:
	virtual ~ISocialInteractionWrapper() = default;

	virtual FName GetInteractionName() const = 0;
	virtual FText GetDisplayName(const USocialUser& User) const = 0;
	virtual FString GetSlashCommandToken() const = 0;

	virtual bool IsAvailable(const USocialUser& User) const = 0;
	virtual void ExecuteInteraction(USocialUser& User) const = 0;
	virtual void ExecuteInteractionWithContext(USocialUser& User, const TMap<FString, FString>& AnalyticsContext) const = 0;
};

template <typename InteractionT>
class TSocialInteractionWrapper : public ISocialInteractionWrapper
{
public:
	virtual FName GetInteractionName() const override final { return InteractionT::GetInteractionName(); }
	virtual FText GetDisplayName(const USocialUser& User) const override final { return InteractionT::GetDisplayName(User); }
	virtual FString GetSlashCommandToken() const override final { return InteractionT::GetSlashCommandToken(); }

	virtual bool IsAvailable(const USocialUser& User) const override final { return InteractionT::IsAvailable(User); }
	virtual void ExecuteInteraction(USocialUser& User) const override final { InteractionT::ExecuteInteraction(User); }
	virtual void ExecuteInteractionWithContext(USocialUser& User, const TMap<FString, FString>& AnalyticsContext) const override final { InteractionT::ExecuteInteractionWithContext(User, AnalyticsContext); };

private:
	friend InteractionT;
	TSocialInteractionWrapper() {}

	FSocialInteractionHandle GetHandle() const { return FSocialInteractionHandle(*this); }
};

// Helper macros for declaring a social interaction class
// Establishes boilerplate behavior and declares all functions the user is required to provide


#define DECLARE_SOCIAL_INTERACTION_EXPORT(APIMacro, InteractionName)	\
	class FSocialInteraction_##InteractionName	\
	{	\
	public:	\
		static APIMacro FSocialInteractionHandle GetHandle();	\
		static FName GetInteractionName();  \
		static APIMacro FText GetDisplayName(const USocialUser& User);	\
		static APIMacro FString GetSlashCommandToken();	\
		static bool IsAvailable(const USocialUser& User);	\
		static APIMacro void ExecuteInteractionWithContext(USocialUser& User, const TMap<FString, FString>& AnalyticsContext);	\
		static APIMacro void ExecuteInteraction(USocialUser& User);	\
		static APIMacro FOnCustomIsInteractionAvailable& OnCustomIsInteractionAvailable();	\
	private: \
		static APIMacro bool CanExecute(const USocialUser& User); \
	}



#define DEFINE_SOCIAL_INTERACTION(InteractionName)	\
	FSocialInteractionHandle FSocialInteraction_##InteractionName::GetHandle()	\
	{	\
		static const TSocialInteractionWrapper<FSocialInteraction_##InteractionName> InteractionWrapper;	\
		return InteractionWrapper.GetHandle();	\
	}	\
	FName FSocialInteraction_##InteractionName::GetInteractionName()  \
	{ \
		return #InteractionName; \
	} \
	bool FSocialInteraction_##InteractionName::IsAvailable(const USocialUser& User)	\
	{	\
		if (CanExecute(User))	\
		{	\
			return OnCustomIsInteractionAvailable().IsBound() ? OnCustomIsInteractionAvailable().Execute(User) : true;	\
		}	\
		return false;	\
	}	\
	void FSocialInteraction_##InteractionName::ExecuteInteractionWithContext(USocialUser& User, const TMap<FString, FString>& AnalyticsContext)	\
	{	\
		User.WithContext(AnalyticsContext, [](USocialUser& InUser) {	\
			ExecuteInteraction(InUser);	\
		});	\
	}	\
	FOnCustomIsInteractionAvailable& FSocialInteraction_##InteractionName::OnCustomIsInteractionAvailable()	\
	{	\
		static FOnCustomIsInteractionAvailable CustomAvailabilityCheckDelegate;	\
		return CustomAvailabilityCheckDelegate;	\
	}	\


#define DECLARE_SOCIAL_INTERACTION(InteractionName) DECLARE_SOCIAL_INTERACTION_EXPORT(, InteractionName)
