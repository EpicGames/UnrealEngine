// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/AssetManager.h"
#include "Engine/StreamableManager.h"
#include "Misc/NotNull.h"
#include "Party/PartyTypes.h"
#include "PartyModule.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "UObject/GCObject.h"
#include "Containers/Ticker.h"
#include "Interfaces/OnlinePartyInterface.h"
#include "Templates/SharedPointer.h"

namespace UE::Party
{
	PARTY_API bool IsRepDataAsyncLoadingEnabled();
}

/** Util exclusively for use by TPartyDataReplicator to circumvent circular include header issues (we can't include SocialParty.h or PartyMember.h here) */
class FPartyDataReplicatorHelper
{
	template <typename, class> friend class TPartyDataReplicator;
	PARTY_API static void ReplicateDataToMembers(const FOnlinePartyRepDataBase& RepDataInstance, const UScriptStruct& RepDataType, const FOnlinePartyData& ReplicationPayload);
};

/** Base util class for dealing with data that is replicated to party members */
template <typename RepDataT, class OwningObjectT>
class TPartyDataReplicator : public FGCObject
{
	static_assert(TIsDerivedFrom<RepDataT, FOnlinePartyRepDataBase>::IsDerived, "TPartyDataReplicator is only intended to function with FOnlinePartyRepDataBase types.");
	friend OwningObjectT;

public:
	~TPartyDataReplicator()
	{
		Reset();
	}

	const RepDataT* operator->() const { check(RepDataPtr); return RepDataPtr; }
	RepDataT* operator->() { check(RepDataPtr); return RepDataPtr; }
	const RepDataT& operator*() const { return *RepDataPtr; }
	RepDataT& operator*() { return *RepDataPtr; }

	virtual void AddReferencedObjects(FReferenceCollector& Collector) override
	{
		Collector.AddReferencedObject(RepDataType);
	}

	virtual FString GetReferencerName() const override
	{
		return "TPartyDataReplicator";
	}

	bool IsValid() const { return RepDataType && RepDataPtr && RepDataCopy; }
	
	template <typename ChildRepDataT>
	void EstablishRepDataInstance(ChildRepDataT& RepDataInstance)
	{
		static_assert(TIsDerivedFrom<ChildRepDataT, RepDataT>::IsDerived, "Incompatible RepData child struct type");

		static_cast<FOnlinePartyRepDataBase*>(&RepDataInstance)->OnDataChanged.BindRaw(this, &TPartyDataReplicator::HandleRepDataChanged);

		RepDataPtr = &RepDataInstance;
		RepDataType = ChildRepDataT::StaticStruct();

		RepDataCopy = (ChildRepDataT*)FMemory::Malloc(RepDataType->GetCppStructOps()->GetSize(), RepDataType->GetCppStructOps()->GetAlignment());
		RepDataType->GetCppStructOps()->Construct(RepDataCopy);
		RepDataType->GetCppStructOps()->Copy(RepDataCopy, RepDataPtr, 1);
	}

	void Flush()
	{
		// If we had a scheduled update run it now.
		if (UpdateTickerHandle.IsValid())
		{
			// Running manually - unregister ticker.
			FTSTicker::RemoveTicker(UpdateTickerHandle);
			UpdateTickerHandle.Reset();
			DeferredHandleReplicateChanges(0.f);
		}
	}

protected:
	void ProcessReceivedData(TNotNull<OwningObjectT*> Owner, const FOnlinePartyData& IncomingPartyData, bool bCompareToPrevious = true)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE_STR("TPartyDataReplicator::ProcessReceivedData");

		// If the rep data can be edited locally, disregard any replication updates (they're the same at best or out of date at worst)
		if (!static_cast<FOnlinePartyRepDataBase*>(RepDataPtr)->CanEditData())
		{
			if (UE::Party::IsRepDataAsyncLoadingEnabled())
			{
				// Since IncomingPartyData will only contain modified fields, it's important that we apply each update (in order) 
				// that comes in or the local party data will get out of sync with the server.
				// This means that if we receive new data while still loading the previous request, rather than canceling the previous 
				// request, we start a new one to be applied after the previous has completed.

				// Lazily create once to avoid allocs each time data is received.
				if (!PendingRepData)
				{
					// Allocate a new object to write the new data into as we don't want to overwrite the actual data until loading has completed.
					PendingRepData = (RepDataT*)FMemory::Malloc(RepDataType->GetCppStructOps()->GetSize(), RepDataType->GetCppStructOps()->GetAlignment());
					RepDataType->GetCppStructOps()->Construct(PendingRepData);
				}

				// We want this to match the current rep data so it stays in sync and we don't miss any assets in GetAssetsToAsyncLoad.
				// Any incoming updates will be applied via IncomingPartyData, but that doesn't account for local changes which is why the copy is needed each time.
				RepDataType->GetCppStructOps()->Copy(PendingRepData, RepDataPtr, 1);

				// Deserialize the incoming data into the scratch copy so we can extract any assets that need to be loaded.
				if (FVariantDataConverter::VariantMapToUStruct(IncomingPartyData.GetKeyValAttrs(), RepDataType, PendingRepData, 0, CPF_Transient | CPF_RepSkip))
				{
					FStreamableAsyncLoadParams LoadParams;
					// We always request all assets so the new load handle has everything that needs to stay in memory.
					static_cast<FOnlinePartyRepDataBase*>(PendingRepData)->GetAssetsToAsyncLoad(LoadParams.TargetsToStream);

					TSharedPtr<FStreamableHandle> LoadHandle = UAssetManager::GetStreamableManager().RequestAsyncLoad(MoveTemp(LoadParams));

					static int32 NextId = 0;
					const int32 RequestId = ++NextId;

					FQueuedRepData QueuedData;
					QueuedData.LoadHandle = LoadHandle; // Intentionally not moved into QueuedData as we use the handle after moving QueuedData into the Queue.
					QueuedData.KeyValueAttrs = IncomingPartyData.GetKeyValAttrs();
					QueuedData.Id = RequestId;
					QueuedData.bCompareToPrevious = bCompareToPrevious;

					// We always queue the data even if there is nothing to load to ensure updates are applied in the correct order.
					RepDataLoadQueue.Push(MoveTemp(QueuedData));

					if (LoadHandle.IsValid() && LoadHandle->IsLoadingInProgress())
					{
						UE_LOGF(LogParty, Verbose, "Queueing RepData load request %d for %ls. Loading %d assets.", RequestId, *(Owner->GetName()), LoadParams.TargetsToStream.Num());

						// Our lifetime is tied to our owner, so use it to prevent the callback from firing after we've been destroyed.
						LoadHandle->BindCompleteDelegate(
							FStreamableDelegate::CreateWeakLambda(Owner, [this]
							{
								ProcessLoadQueue();
							}));
					}
					else
					{
						UE_LOGF(LogParty, Verbose, "Queueing RepData request %d for %ls. All required assets already loaded.", RequestId, *(Owner->GetName()));

						// Try pumping the queue immediately since there's no load to wait on.
						ProcessLoadQueue();
					}
				}
				else
				{
					UE_LOGF(LogParty, Error, "Failed to serialize received party data!");
				}
			}
			else
			{
				PostReceiveData(IncomingPartyData.GetKeyValAttrs(), bCompareToPrevious);
			}
		}
	}

	void Reset()
	{
		if (RepDataPtr)
		{
			static_cast<FOnlinePartyRepDataBase*>(RepDataPtr)->OnDataChanged.Unbind();
			RepDataPtr = nullptr;
		}
		if (RepDataType)
		{
			if (PendingRepData)
			{
				RepDataType->GetCppStructOps()->Destruct(PendingRepData);
				FMemory::Free(PendingRepData);
				PendingRepData = nullptr;
			}
			if (RepDataCopy)
			{
				RepDataType->GetCppStructOps()->Destruct(RepDataCopy);
				FMemory::Free(RepDataCopy);
				RepDataCopy = nullptr;
			}
			RepDataType = nullptr;
		}
		if (UpdateTickerHandle.IsValid())
		{
			FTSTicker::RemoveTicker(UpdateTickerHandle);
			UpdateTickerHandle.Reset();
		}

		for (FQueuedRepData& Request : RepDataLoadQueue)
		{
			if (Request.LoadHandle.IsValid())
			{
				Request.LoadHandle->CancelHandle();
			}
		}
		RepDataLoadQueue.Empty();
	}

private:
	void HandleRepDataChanged()
	{
		if (!UpdateTickerHandle.IsValid())
		{
			UpdateTickerHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateRaw(this, &TPartyDataReplicator::DeferredHandleReplicateChanges));
		}
	}

	bool DeferredHandleReplicateChanges(float)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_TPartyDataReplicator_DeferredHandleReplicateChanges);

		// Reset ticker handle so that new data changed events will schedule a new ticker.
		UpdateTickerHandle.Reset();

		FOnlinePartyData OnlinePartyData;
		if (FVariantDataConverter::UStructToVariantMap(RepDataType, RepDataPtr, OnlinePartyData.GetKeyValAttrs(), 0, CPF_Transient | CPF_RepSkip))
		{
			FPartyDataReplicatorHelper::ReplicateDataToMembers(*RepDataPtr, *RepDataType, OnlinePartyData);
			
			// Make sure the local copy lines up with whatever has been sent most recently
			ensure(RepDataType->GetCppStructOps()->Copy(RepDataCopy, RepDataPtr, 1));
		}
		return false;
	}

	void PostReceiveData(const FOnlineKeyValuePairs<FString, FVariantData>& KeyValueAttrs, bool bCompareToPrevious, TSharedPtr<FStreamableHandle> ResourceHandle = nullptr)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE_STR("TPartyDataReplicator::PostReceiveData");

		bool bRepDataUpdated = false;
		if (UE::Party::IsRepDataAsyncLoadingEnabled())
		{
			// Only apply the values to fields whose value matches between RepDataPtr and RepDataCopy.
			// This prevents stomping locally set fields between when the load was queued and now.
			bRepDataUpdated = FVariantDataConverter::ApplyVariantMapToUnchangedProperties(KeyValueAttrs, RepDataType, RepDataPtr, RepDataCopy, 0, CPF_Transient | CPF_RepSkip);
		}
		else
		{
			bRepDataUpdated = FVariantDataConverter::VariantMapToUStruct(KeyValueAttrs, RepDataType, RepDataPtr, 0, CPF_Transient | CPF_RepSkip);
		}

		if (bRepDataUpdated)
		{
			static_cast<FOnlinePartyRepDataBase*>(RepDataPtr)->PostReplication(*RepDataCopy, MoveTemp(ResourceHandle));
			if (bCompareToPrevious)
			{
				static_cast<FOnlinePartyRepDataBase*>(RepDataPtr)->CompareAgainst(*RepDataCopy);
			}
			ensure(RepDataType->GetCppStructOps()->Copy(RepDataCopy, RepDataPtr, 1));
		}
		else
		{
			UE_LOGF(LogParty, Error, "Failed to serialize received party data!");
		}
	}

	void ProcessLoadQueue()
	{
		int32 Index = 0;

		// Apply queued updates in order provided they're done loading.
		while (Index < RepDataLoadQueue.Num())
		{
			FQueuedRepData& Request = RepDataLoadQueue[Index];

			// We won't always be waiting on a load as some updates don't contain assets that needed to be loaded first.
			if (Request.LoadHandle.IsValid() && Request.LoadHandle->IsLoadingInProgress())
			{
				UE_LOGF(LogParty, Verbose, "Waiting for RepData request %d to finish loading. %d requests in the queue.", Request.Id, RepDataLoadQueue.Num());
				break;
			}

			UE_LOGF(LogParty, Verbose, "Applying loaded RepData request %d.", Request.Id);
			PostReceiveData(Request.KeyValueAttrs, Request.bCompareToPrevious, MoveTemp(Request.LoadHandle));

			++Index;
		}

		if (Index > 0)
		{
			// Remove processed
			RepDataLoadQueue.RemoveAt(0, Index, EAllowShrinking::No);
		}
	}

	struct FQueuedRepData
	{
		TSharedPtr<FStreamableHandle> LoadHandle;
		FOnlineKeyValuePairs<FString, FVariantData> KeyValueAttrs;
		int32 Id = 0;
		bool bCompareToPrevious = false;
	};

	/** New rep data that is currently async loading any required assets before it can be applied. */
	TArray<FQueuedRepData> RepDataLoadQueue;

	/** Reflection data for child USTRUCT */
	TObjectPtr<const UScriptStruct> RepDataType = nullptr;

	/**
	 * Pointer to child UStruct that holds the current state of the party. Only modifiable by party leader.
	 * To establish a custom state struct, call EstablishPartyState<T> with the desired type within the child class's constructor
	 */
	RepDataT* RepDataPtr = nullptr;

	/** Scratch copy of child UStruct for handling replication comparisons */
	RepDataT* RepDataCopy = nullptr;

	/** Scratch copy of child UStruct for storing incoming rep data to query it prior to replication. */
	RepDataT* PendingRepData = nullptr;

	FTSTicker::FDelegateHandle UpdateTickerHandle;
};
