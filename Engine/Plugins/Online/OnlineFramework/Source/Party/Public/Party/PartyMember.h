// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Party/PartyDataReplicator.h"

#include "SocialTypes.h"
#include "PartyMember.generated.h"

#define UE_API PARTY_API

class ULocalPlayer;
class USocialUser;
class USocialToolkit;
class FOnlinePartyMember;
class FOnlinePartyData;
enum class EMemberExitedReason : uint8;
enum class EMemberConnectionStatus : uint8;

/** Platform data fields for party replication */
USTRUCT()
struct FPartyMemberPlatformData
{
	GENERATED_BODY()

public:
	bool operator==(const FPartyMemberPlatformData& Other) const { return Platform == Other.Platform && UniqueId == Other.UniqueId && SessionId == Other.SessionId; }
	bool operator!=(const FPartyMemberPlatformData& Other) const { return !operator==(Other); }

	/** Native platform on which this party member is playing. */
	UPROPERTY()
	FUserPlatform Platform;

	/** Net ID for this party member on their native platform. Blank if this member has no Platform SocialSubsystem. */
	UPROPERTY()
	FUniqueNetIdRepl UniqueId;

	/**
	 * The platform session this member is in. Can be blank for a bit while creating/joining.
	 * Only relevant when this member is on a platform that requires a session backing the party.
	 */
	UPROPERTY()
	FString SessionId;
};

/** Join in progress request. Represents a request from a local party member to a remote party member to acquire a reservation for the session the remote party member is in. */
USTRUCT()
struct FPartyMemberJoinInProgressRequest
{
	GENERATED_BODY()

public:
	bool operator==(const FPartyMemberJoinInProgressRequest& Other) const { return Target == Other.Target && Time == Other.Time; }
	bool operator!=(const FPartyMemberJoinInProgressRequest& Other) const { return !operator==(Other); }

	/** Remote member we want to join. */
	UPROPERTY()
	FUniqueNetIdRepl Target;

	/** Time the request was made. */
	UPROPERTY()
	int64 Time = 0;
};

/** Join in progress response. Represents a response from a local party member to a remote party member that requested to join in progress. */
USTRUCT()
struct FPartyMemberJoinInProgressResponse
{
	GENERATED_BODY()

public:
	bool operator==(const FPartyMemberJoinInProgressResponse& Other) const
	{
		return Requester == Other.Requester &&
			RequestTime == Other.RequestTime &&
			ResponseTime == Other.ResponseTime &&
			DenialReason == Other.DenialReason;
	}
	bool operator!=(const FPartyMemberJoinInProgressResponse& Other) const { return !operator==(Other); }

	/** Remote member that this response is for. */
	UPROPERTY()
	FUniqueNetIdRepl Requester;

	/** Time the request was made. Matches FPartyMemberJoinInProgressRequest::Time */
	UPROPERTY()
	int64 RequestTime = 0;

	/** Time the response was made. */
	UPROPERTY()
	int64 ResponseTime = 0;

	/**
	 * Result of session reservation attempt.
	 * @see EPartyJoinDenialReason
	 */
	UPROPERTY()
	uint8 DenialReason = 0;
};

/** Join in progress data. Holds the current request and any responses. Requests and responses are expected to be cleared in a short amount of time. Combined into one field to reduce field count. */
USTRUCT()
struct FPartyMemberJoinInProgressData
{
	GENERATED_BODY()

public:

	bool operator==(const FPartyMemberJoinInProgressData& Other) const { return Request == Other.Request && Responses == Other.Responses; }
	bool operator!=(const FPartyMemberJoinInProgressData& Other) const { return !operator==(Other); }

	/** Current request for the local member. */
	UPROPERTY()
	FPartyMemberJoinInProgressRequest Request;

	/** List of responses for other members who requested a reservation. */
	UPROPERTY()
	TArray<FPartyMemberJoinInProgressResponse> Responses;
};

/** Base struct used to replicate data about the state of a single party member to all members. */
USTRUCT()
struct FPartyMemberRepData : public FOnlinePartyRepDataBase
{
	GENERATED_BODY()

public:
	FPartyMemberRepData() {}
	FPartyMemberRepData& operator=(const FPartyMemberRepData& Other)
	{
		FOnlinePartyRepDataBase::operator=(Other);

		// Do not copy multicasts, because this makes it very hard to track the lifetime of registered delegates

		OwnerMember = Other.OwnerMember;
		bAllowOwnerless = Other.bAllowOwnerless;
		PlatformData = Other.PlatformData;
		CrossplayPreference = Other.CrossplayPreference;
		JoinMethod = Other.JoinMethod;
		JoinInProgressData = Other.JoinInProgressData;

		return *this;
	}
	UE_API virtual void SetOwningMember(const class UPartyMember& InOwnerMember);
	/** Mark the party data as ownerless. This will bypass any "CanEdit" checks. Useful for using this object in a test context. */
	UE_API void MarkOwnerless();

protected:
	UE_API virtual bool CanEditData() const override;
	UE_API virtual void CompareAgainst(const FOnlinePartyRepDataBase& OldData) const override;
	UE_API virtual const USocialParty* GetOwnerParty() const override;
	UE_API virtual const UPartyMember* GetOwningMember() const;

private:
	TWeakObjectPtr<const UPartyMember> OwnerMember;
	bool bAllowOwnerless = false;

	/** Platform data fields for party replication */
	UPROPERTY()
	FPartyMemberPlatformData PlatformData;
	EXPOSE_REVISED_USTRUCT_REP_DATA_PROPERTY(FPartyMemberRepData, FUserPlatform, PlatformData, Platform, Platform, 4.27);
	EXPOSE_REVISED_USTRUCT_REP_DATA_PROPERTY(FPartyMemberRepData, FUniqueNetIdRepl, PlatformData, UniqueId, PlatformUniqueId, 4.27);
	EXPOSE_REVISED_USTRUCT_REP_DATA_PROPERTY(FPartyMemberRepData, FSessionId, PlatformData, SessionId, PlatformSessionId, 4.27);

	/** The crossplay preference of this user. Only relevant to crossplay party scenarios. */
	UPROPERTY()
	ECrossplayPreference CrossplayPreference = ECrossplayPreference::NoSelection;
	EXPOSE_REP_DATA_PROPERTY(FPartyMemberRepData, ECrossplayPreference, CrossplayPreference);

	/** Method used to join the party */
	UPROPERTY()
	FString JoinMethod;
	EXPOSE_REP_DATA_PROPERTY(FPartyMemberRepData, FString, JoinMethod);

	/** Data used for join in progress flow. */
	UPROPERTY()
	FPartyMemberJoinInProgressData JoinInProgressData;
	EXPOSE_USTRUCT_REP_DATA_PROPERTY(FPartyMemberRepData, FPartyMemberJoinInProgressRequest, JoinInProgressData, Request);
	EXPOSE_USTRUCT_REP_DATA_PROPERTY(FPartyMemberRepData, TArray<FPartyMemberJoinInProgressResponse>, JoinInProgressData, Responses);
};

using FPartyMemberDataReplicator = TPartyDataReplicator<FPartyMemberRepData, UPartyMember>;

UCLASS(MinimalAPI, Abstract, config = Game, Within = SocialParty, Transient)
class UPartyMember : public UObject
{
	GENERATED_BODY()

	friend class FPartyPlatformSessionMonitor;
	friend class USocialManager;
	friend USocialParty;
public:
	UE_API UPartyMember();

	UE_API virtual void BeginDestroy() override;

	UE_API bool CanPromoteToLeader(const ULocalPlayer& PerformingPlayer) const;
	UE_API bool PromoteToPartyLeader(const ULocalPlayer& PerformingPlayer);
	UE_API bool CanKickFromParty(const ULocalPlayer& PerformingPlayer) const;
	UE_API bool KickFromParty(const ULocalPlayer& PerformingPlayer);

	UE_API bool IsInitialized() const;
	UE_API bool IsPartyLeader() const;
	UE_API bool IsLocalPlayer() const;
	
	UE_API USocialParty& GetParty() const;
	UE_API FUniqueNetIdRepl GetPrimaryNetId() const;
	const FPartyMemberRepData& GetRepData() const { return *MemberDataReplicator; }
	/** Get the default social user. NOTE: This method will be deprecated in the future. Prefer GetSocialUser(InLocalUserId). */
	UE_API USocialUser& GetSocialUser() const;
	/**
	 * Get the social user for a local player
	 * @param InLocalUserId the primary user id of the local user to get the social user for.
	 * @return the social user registered for this party member and local user. May be null if InLocalUserId does not map to a social toolkit, otherwise expected to be non-null.
	 */
	UE_API USocialUser* GetSocialUser(const FUniqueNetIdRepl& InLocalUserId) const;

	UE_API EMemberConnectionStatus GetMemberConnectionStatus() const;

	UE_API FString GetDisplayName() const;
	UE_API FName GetPlatformOssName() const;

	DECLARE_EVENT(UPartyMember, FOnPartyMemberStateChanged);
	FOnPartyMemberStateChanged& OnInitializationComplete() const { return OnMemberInitializedEvent; }
	FOnPartyMemberStateChanged& OnPromotedToLeader() const { return OnPromotedToLeaderEvent; }
	FOnPartyMemberStateChanged& OnDemoted() const { return OnDemotedEvent; }
	FOnPartyMemberStateChanged& OnMemberConnectionStatusChanged() const { return OnMemberConnectionStatusChangedEvent; }
	FOnPartyMemberStateChanged& OnDisplayNameChanged() const { return OnDisplayNameChangedEvent; }

	DECLARE_EVENT_OneParam(UPartyMember, FOnPartyMemberLeft, EMemberExitedReason)
	FOnPartyMemberLeft& OnLeftParty() const { return OnLeftPartyEvent; }

	UE_API FString ToDebugString(bool bIncludePartyId = true) const;

protected:
	UE_API void InitializePartyMember(const FOnlinePartyMemberConstRef& OssMember, FSimpleDelegate&& OnInitComplete);

	FPartyMemberRepData& GetMutableRepData() { return *MemberDataReplicator; }
	UE_API void NotifyMemberDataReceived(const FOnlinePartyData& MemberData);
	UE_API void NotifyMemberPromoted();
	UE_API void NotifyMemberDemoted();
	UE_API void NotifyRemovedFromParty(EMemberExitedReason ExitReason);

protected:
	UE_API virtual void FinishInitializing();
	UE_API virtual void InitializeLocalMemberRepData();

	UE_API virtual void OnMemberPromotedInternal();
	UE_API virtual void OnMemberDemotedInternal();
	UE_API virtual void OnRemovedFromPartyInternal(EMemberExitedReason ExitReason);
	UE_API virtual void Shutdown();

	FPartyMemberDataReplicator MemberDataReplicator;

	TSharedPtr<const FOnlinePartyMember> GetOSSPartyMember() const { return OssPartyMember; }

private:
	UE_API void InitializeSocialUserForToolkit(USocialToolkit& Toolkit);
	UE_API void HandleSocialUserInitialized(USocialUser& InitializedUser);
	UE_API void HandleMemberConnectionStatusChanged(const FUniqueNetId& ChangedUserId, const EMemberConnectionStatus NewMemberConnectionStatus, const EMemberConnectionStatus PreviousMemberConnectionStatus);
	UE_API void HandleMemberAttributeChanged(const FUniqueNetId& ChangedUserId, const FString& Attribute, const FString& NewValue, const FString& OldValue);
	UE_API void OnSocialToolkitCreated(USocialToolkit& Toolkit);
	UE_API void OnSocialToolkitDestroyed(USocialToolkit& Toolkit);
	UE_API void OnSocialToolkitLoggedIn(USocialToolkit& Toolkit);

	FOnlinePartyMemberConstPtr OssPartyMember;

	UPROPERTY()
	TObjectPtr<USocialUser> DefaultSocialUser = nullptr;

	// Initializing status
	enum class EInitializingFlags : uint8
	{
		Done = 0, // Done initializing
		SocialUsers = 1<<0, // Waiting for all social users to initialize
		InitialMemberData = 1<<1, // Waiting to receive initial member data
	};
	FRIEND_ENUM_CLASS_FLAGS(EInitializingFlags);
	EInitializingFlags InitializingFlags = EInitializingFlags::Done;

	UPROPERTY(config)
	bool bEnableDebugInitializer = true;

	// Debug info for initializing party members
	class FDebugInitializer;
	friend class FDebugInitializer;
	TUniquePtr<FDebugInitializer> DebugInitializer;

	mutable FOnPartyMemberStateChanged OnMemberConnectionStatusChangedEvent;
	mutable FOnPartyMemberStateChanged OnDisplayNameChangedEvent;
	mutable FOnPartyMemberStateChanged OnMemberInitializedEvent;
	mutable FOnPartyMemberStateChanged OnPromotedToLeaderEvent;
	mutable FOnPartyMemberStateChanged OnDemotedEvent;
	mutable FOnPartyMemberLeft OnLeftPartyEvent;
};

namespace UE::OnlineFramework
{
/**
 * Utility method to trigger a delegate when a party member is initialized, or trigger immediately if already initialized.
 * Avoids needing to use the pattern 'if (Member->IsInitialized()) { DoWork(); } else { Member->OnInitializationComplete().Add...'
 * @param InPartyMember the party member
 * @param InDelegate the delegate to trigger when initialization is complete (or trigger immediately if already initialized)
 */
PARTY_API void OnPartyMemberInitializeComplete(UPartyMember& InPartyMember, FSimpleDelegate&& InDelegate);
}

#undef UE_API
