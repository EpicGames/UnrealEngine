// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/OnlineReplStructs.h"
#include "Templates/SharedPointer.h"
#include "PartyTypes.generated.h"

#define UE_API PARTY_API

class USocialParty;
class UPartyMember;

class IOnlinePartySystem;
class FOnlinePartyData;

struct FStreamableHandle;

enum class EJoinPartyCompletionResult : int8;

UENUM()
enum class EPartyType : uint8
{
	/** This party is public (not really supported right now) */
	Public,
	/** This party is joinable by friends */
	FriendsOnly,
	/** This party requires an invite from someone within the party */
	Private
};

UENUM()
enum class EPartyInviteRestriction : uint8
{
	/** Any party member can send invites */
	AnyMember,
	/** Only the leader can send invites */
	LeaderOnly,
	/** Nobody can invite anyone to this party */
	NoInvites
};

UENUM()
enum class EPartyJoinDenialReason : uint8
{
	/** Framework-level denial reasons */

	/** No denial, matches success internally */
	NoReason = 0,
	/** The local player aborted the join attempt */
	JoinAttemptAborted,
	/** Party leader is busy or at inopportune time to allow joins - to be used as a fallback when there isn't a more specific reason (more specific reasons are preferred) */
	Busy,
	/** Either the necessary OSS itself or critical element thereof (PartyInterface, SessionInterface, etc.) is missing. */
	OssUnavailable,
	/** Party is full */
	PartyFull,
	/** Game is full, but not party */
	GameFull,
	/** Asked a non party leader to join game, shouldn't happen */
	NotPartyLeader,
	/** Party has been marked as private and the join request is revoked */
	PartyPrivate,
	/** Player has crossplay restriction that would be violated */
	JoinerCrossplayRestricted,
	/** Party member has crossplay restriction that would be violated */
	MemberCrossplayRestricted,
	/** Player is in a game mode that restricts joining */
	GameModeRestricted,
	/** Player is currently banned */
	Banned,
	/** Player is not yet logged in */
	NotLoggedIn,
	/** Unable to start joining - we are checking for a session to rejoin */
	CheckingForRejoin,
	/** The target user is missing presence info */
	TargetUserMissingPresence,
	/** The target user's presence says the user is unjoinable */
	TargetUserUnjoinable,
	/** The target user is currently Away */
	TargetUserAway,
	/** We found ourself to be the leader of the friend's party according to the console session */
	AlreadyLeaderInPlatformSession,
	/** The target user is not playing the same game as us */
	TargetUserPlayingDifferentGame,
	/** The target user's presence does not have any information about their party session (platform friends only) */
	TargetUserMissingPlatformSession,
	/** There is no party join info available in the target user's platform session */
	PlatformSessionMissingJoinInfo,
	/** We were unable to launch the query to find the platform friend's session (platform friends only) */
	FailedToStartFindConsoleSession,
	/** The party is of a type that the game does not support (it specified nullptr for the USocialParty class) */
	MissingPartyClassForTypeId,
	/** The target user is blocked by the local user on one or more of the active subsystems */
	TargetUserBlocked,
	/** Failed to get party join info from target user */
	InvalidJoinInfo,
	/** The join method doesn't support non-friends */
	NotFriends,

	/**
	* Customizable denial reasons.
	* Expected usage is to assign the entries in the custom enum to the arbitrary custom entry placeholders below.
	* App level users of the system can then cast to/from their custom enum as desired.
	*/
	CustomReason0,
	CustomReason1,
	CustomReason2,
	CustomReason3,
	CustomReason4,
	CustomReason5,
	CustomReason6,
	CustomReason7,
	CustomReason8,
	CustomReason9,
	CustomReason10,
	CustomReason11,
	CustomReason12,
	CustomReason13,
	CustomReason14,
	CustomReason15,
	CustomReason16,
	CustomReason17,
	CustomReason18,
	CustomReason19,
	CustomReason20,
	CustomReason21,
	CustomReason22,
	CustomReason23,
	CustomReason24,
	CustomReason25,
	CustomReason26,
	CustomReason27,
	CustomReason28,
	CustomReason29,
	CustomReason30,
	CustomReason31,
	CustomReason32,
	CustomReason33,
	CustomReason34,
	CustomReason35,
	CustomReason36,
	CustomReason37,
	CustomReason38,
	CustomReason39,

	MAX
};

PARTY_API const TCHAR* ToString(EPartyJoinDenialReason Type);

UENUM()
enum class EApprovalAction : uint8
{
	Approve,
	Enqueue,
	EnqueueAndStartBeacon,
	Deny
};

UENUM()
enum class ESocialPartyInviteMethod : uint8 
{
	/** Default value for try invite */
	Other = 0,
	/** Invite was sent from a toast */
	Notification,
	/** Invite was sent via accepting "Request To Join" */
	AcceptRequestToJoin,
	/** Custom methods */
	Custom0,
	Custom1,
	Custom2,
	Custom3,
	Custom4,
	Custom5,
	Custom6,
	Custom7,
	Custom8,
	Custom9,

	MAX
};

UENUM()
enum class ESocialPartyInviteFailureReason : uint8 
{
	Success = 0,
	NotOnline,
	NotAcceptingMembers,
	NotFriends,
	AlreadyInParty,
	OssValidationFailed,
	PlatformInviteFailed,
	PartyInviteFailed,
	InviteRateLimitExceeded
};


PARTY_API const TCHAR* LexToString(ESocialPartyInviteFailureReason Type);

// Gives a smidge more meaning to the intended use for the string. These should just be UniqueId's (and are), but not reliably allocated as shared ptrs, so they cannot be replicated via FUniqueNetIdRepl.
using FSessionId = FString;

USTRUCT()
struct FPartyPlatformSessionInfo
{
	GENERATED_BODY();

public:

	/** The platform session type (because in a crossplay party, members can be in different session types) */
	UPROPERTY()
	FString SessionType;

	/** The platform session id. Will be unset if it is not yet available to be joined. */
	UPROPERTY()
	FString SessionId;

	/** Primary OSS ID of the player that owns this console session */
	UPROPERTY()
	FUniqueNetIdRepl OwnerPrimaryId;

	bool operator==(const FString& InSessionType) const;
	bool operator==(const FPartyPlatformSessionInfo& Other) const;
	bool operator!=(const FPartyPlatformSessionInfo& Other) const { return !(*this == Other); }

	FString ToDebugString() const;
	bool IsSessionOwner(const UPartyMember& PartyMember) const;
	bool IsInSession(const UPartyMember& PartyMember) const;
};

USTRUCT()
struct FPartyPrivacySettings
{
	GENERATED_BODY()

public:
	/** The type of party in terms of advertised joinability restrictions */
	UPROPERTY()
	EPartyType PartyType = EPartyType::Private;

	/** Who is allowed to send invitataions to the party? */
	UPROPERTY()
	EPartyInviteRestriction PartyInviteRestriction = EPartyInviteRestriction::NoInvites;

	/** True to restrict the party exclusively to friends of the party leader */
	UPROPERTY()
	bool bOnlyLeaderFriendsCanJoin = true;

	bool PARTY_API operator==(const FPartyPrivacySettings& Other) const;
	bool operator!=(const FPartyPrivacySettings& Other) const { return !operator==(Other); }

	FPartyPrivacySettings() {}
};


/** Companion to EPartyJoinDenialReason to lessen the hassle of working with a "customized" enum */
struct FPartyJoinDenialReason
{
public:
	FPartyJoinDenialReason() {}
	FPartyJoinDenialReason(int32 DenialReasonCode)
	{
		if (ensure(DenialReasonCode >= 0 && DenialReasonCode < (uint8)EPartyJoinDenialReason::MAX))
		{
			DenialReason = (EPartyJoinDenialReason)DenialReasonCode;
		}
	}

	template <typename CustomReasonEnumT>
	FPartyJoinDenialReason(CustomReasonEnumT CustomReason)
		: FPartyJoinDenialReason((int32)CustomReason)
	{
	}

	bool operator==(const FPartyJoinDenialReason& Other) const { return DenialReason == Other.DenialReason; }
	bool operator==(int32 ReasonCode) const { return (int32)DenialReason == ReasonCode; }

	template <typename CustomReasonEnumT>
	bool operator==(CustomReasonEnumT CustomReason) const { return (uint8)DenialReason == (uint8)CustomReason; }

	operator int32() const { return (int32)DenialReason; }

	bool HasAnyReason() const { return DenialReason != EPartyJoinDenialReason::NoReason; }
	bool HasCustomReason() const { return DenialReason >= EPartyJoinDenialReason::CustomReason0; }
	EPartyJoinDenialReason GetReason() const { return DenialReason; }

	template <typename CustomReasonEnumT>
	CustomReasonEnumT GetCustomReason() const
	{
		checkf(HasCustomReason(), TEXT("Cannot convert a non-custom disapproval reason to a custom one. Always check HasCustomReason() first."));
		return static_cast<CustomReasonEnumT>((uint8)DenialReason);
	}

	template <typename CustomReasonEnumT>
	const TCHAR* AsString() const
	{
		if (HasCustomReason())
		{
			return ToString(GetCustomReason<CustomReasonEnumT>());
		}
		return ToString(DenialReason);
	}

private:
	EPartyJoinDenialReason DenialReason = EPartyJoinDenialReason::NoReason;
};

struct FPartyJoinApproval
{
	FPartyJoinApproval() {}
	
	void SetDenialReason(FPartyJoinDenialReason InDenialReason)
	{
		DenialReason = InDenialReason;
		if (DenialReason.HasAnyReason())
		{
			ApprovalAction = EApprovalAction::Deny;
		}
	}

	void SetApprovalAction(EApprovalAction InApprovalAction)
	{
		ApprovalAction = InApprovalAction;
		if (ApprovalAction != EApprovalAction::Deny)
		{
			DenialReason = EPartyJoinDenialReason::NoReason;
		}
	}

	EApprovalAction GetApprovalAction() const { return ApprovalAction; }
	FPartyJoinDenialReason GetDenialReason() const { return DenialReason; }

	bool CanJoin() const { return ApprovalAction != EApprovalAction::Deny && ensure(!DenialReason.HasAnyReason()); }

private:
	FPartyJoinDenialReason DenialReason;
	EApprovalAction ApprovalAction = EApprovalAction::Approve;
};

struct FJoinPartyResult
{
public:
	UE_API FJoinPartyResult(); 
	UE_API FJoinPartyResult(FPartyJoinDenialReason InDenialReason);
	UE_API FJoinPartyResult(EJoinPartyCompletionResult InResult);
	UE_API FJoinPartyResult(EJoinPartyCompletionResult InResult, FPartyJoinDenialReason InDenialReason);
	UE_API FJoinPartyResult(EJoinPartyCompletionResult InResult, int32 InResultSubCode);
	
	UE_API void SetDenialReason(FPartyJoinDenialReason InDenialReason);
	UE_API void SetResult(EJoinPartyCompletionResult InResult);
	
	UE_API bool WasSuccessful() const;

	EJoinPartyCompletionResult GetResult() const { return Result; }
	FPartyJoinDenialReason GetDenialReason() const { return DenialReason; }
	int32 GetResultSubCode() const { return ResultSubCode; }

private:
	EJoinPartyCompletionResult Result;
	/** Denial reason - used if Result is NotApproved */
	FPartyJoinDenialReason DenialReason;
	/** Result sub code - used for any other Result type */
	int32 ResultSubCode = 0;
};

/** Base for all rep data structs */
USTRUCT()
struct FOnlinePartyRepDataBase
{
	GENERATED_USTRUCT_BODY()
	virtual ~FOnlinePartyRepDataBase() = default;

protected:
	/**
	 * Gets any assets that should be async loaded before PostReplication is called.
	 * This is only used if Party.ReplicationAsyncLoadingEnabled is enabled.
	 */
	virtual void GetAssetsToAsyncLoad(TArray<FSoftObjectPath>& OutAssets) const {}

	/**
	 * Compare this data against the given old data, triggering delegates as appropriate.
	 * Intended to be used in concert with EXPOSE_REP_DATA_PROPERTY so that all you need to do is run through each property's CompareX function.
	 * If you need to do more complex comparison logic, you're welcome to do that as well/instead.
	 * @see FPartyRepData::CompareAgainst for an example
	 */
	virtual void CompareAgainst(const FOnlinePartyRepDataBase& OldData) const {}

	/**
	 * Called directly after an updated member state is received and copied into the local state.
	 * InResourceHandle will contain any resources returned by GetAssetsToAsyncLoad that were loaded.
	 * This handle should be retained to keep these resources in memory.
	 */
	virtual void PostReplication(const FOnlinePartyRepDataBase& OldData, TSharedPtr<FStreamableHandle> InResourceHandle) {}
	UE_DEPRECATED(5.8, "Please use version taking InResourceHandle")
	virtual void PostReplication(const FOnlinePartyRepDataBase& OldData) {}
	virtual bool CanEditData() const { return false; }
	virtual const USocialParty* GetOwnerParty() const { return nullptr; }
	virtual const UPartyMember* GetOwningMember() const { return nullptr; }

	UE_API void LogSetPropertyFailure(const TCHAR* OwningStructTypeName, const TCHAR* PropertyName) const;
	UE_API void LogPropertyChanged(const TCHAR* OwningStructTypeName, const TCHAR* PropertyName, bool bFromReplication) const;

	friend class FPartyDataReplicatorHelper;
	template <typename, class> friend class TPartyDataReplicator;
	FSimpleDelegate OnDataChanged;
};

//////////////////////////////////////////////////////////////////////////
// Boilerplate for exposing RepData properties
//////////////////////////////////////////////////////////////////////////

#define EXPOSE_REP_DATA_PROPERTY_NO_SETTER_IMPL(Owner, PropertyType, PropertyName, PropertyAccess, GetterAccess, OnChangedAccess)	\
public:	\
	/** If the property is a POD or ptr type, we'll work with it by copy. Otherwise, by const ref */	\
	using Mutable##PropertyName##Type = std::remove_const_t<PropertyType>;	\
	using PropertyName##ArgType = std::conditional_t<TOr<TIsPODType<PropertyType>, TIsPointer<PropertyType>>::Value, PropertyType, const Mutable##PropertyName##Type&>;	\
	\
private:	\
	/** Bummer to have two signatures, but cases that want both the old and new values are much rarer, so most don't want to bother with a handler that takes an extra unused param */	\
	DECLARE_MULTICAST_DELEGATE_OneParam(FOn##PropertyName##Changed, PropertyName##ArgType /*NewValue*/);	\
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOn##PropertyName##ChangedDif, PropertyName##ArgType /*NewValue*/, PropertyName##ArgType /*OldValue*/);	\
OnChangedAccess:	\
	/** Bind to receive the new property value only on changes */	\
	FOn##PropertyName##Changed& On##PropertyName##Changed() const { return On##PropertyName##ChangedEvent; }	\
	/** Bind to receive both the new and old property value on changes */	\
	FOn##PropertyName##ChangedDif& On##PropertyName##ChangedDif() const { return On##PropertyName##ChangedDifEvent; }	\
GetterAccess: \
	PropertyName##ArgType Get##PropertyName() const { return PropertyAccess; }	\
private:	\
	void Compare##PropertyName(const Owner& OldData) const	\
	{	\
		Compare##PropertyName(OldData.PropertyAccess); \
	}	\
	void Compare##PropertyName(const std::remove_const_t<PropertyType>& OldData) const	\
	{	\
		if (PropertyAccess != OldData)	\
		{	\
			LogPropertyChanged(TEXT(#Owner), TEXT(#PropertyName), true);	\
			On##PropertyName##ChangedDif().Broadcast(PropertyAccess, OldData);	\
			On##PropertyName##Changed().Broadcast(PropertyAccess);	\
		}	\
	}	\
	mutable FOn##PropertyName##Changed On##PropertyName##ChangedEvent;	\
	mutable FOn##PropertyName##ChangedDif On##PropertyName##ChangedDifEvent

/** Simplest option - exposes getter and events, but no default setter */
#define EXPOSE_REP_DATA_PROPERTY_NO_SETTER(Owner, PropertyType, PropertyName, PropertyAccess)	\
	EXPOSE_REP_DATA_PROPERTY_NO_SETTER_IMPL(Owner, PropertyType, PropertyName, PropertyAccess, public, public)

#define EXPOSE_REP_DATA_PROPERTY_SETTER_ONLY(Owner, PropertyType, PropertyName, PropertyAccess, SetterPrivacy)	\
SetterPrivacy:	\
	void Set##PropertyName(PropertyName##ArgType New##PropertyName)	\
	{	\
		if (CanEditData())	\
		{	\
			if (PropertyAccess != New##PropertyName)	\
			{	\
				LogPropertyChanged(TEXT(#Owner), TEXT(#PropertyName), false);	\
				if (On##PropertyName##ChangedDif().IsBound())	\
				{	\
					PropertyType OldValue = PropertyAccess;	\
					PropertyAccess = New##PropertyName;	\
					On##PropertyName##ChangedDif().Broadcast(PropertyAccess, OldValue);	\
				}	\
				else	\
				{	\
					PropertyAccess = New##PropertyName;	\
				}	\
				On##PropertyName##Changed().Broadcast(PropertyAccess);	\
				OnDataChanged.ExecuteIfBound();	\
			}	\
		}	\
		else	\
		{	\
			LogSetPropertyFailure(TEXT(#Owner), TEXT(#PropertyName));	\
		}	\
	}	\
private: //

/* Helpers for printing a user-friendly field deprecation warning for replicated data. */
#define STRINGIFY_REP_DATA_PROPERTY_REVISION_WARNING_TEXT(WarningText) #WarningText
#define REP_DATA_PROPERTY_REVISION_WARNING(DeprecatedName, RevisedName, DeprecatedVersion) UE_DEPRECATED(DeprecatedVersion, STRINGIFY_REP_DATA_PROPERTY_REVISION_WARNING_TEXT(DeprecatedName) " has been replaced by " STRINGIFY_REP_DATA_PROPERTY_REVISION_WARNING_TEXT(RevisedName) ".")

/**
 * Exposes a rep data property and provides a default property setter.
 * Awkwardly named util - don't bother using directly. Opt for EXPOSE_PRIVATE_REP_DATA_PROPERTY or EXPOSE_REP_DATA_PROPERTY
 */
#define EXPOSE_REP_DATA_PROPERTY_SETTER_ACCESS(Owner, PropertyType, PropertyName, PropertyAccess, SetterPrivacy)	\
EXPOSE_REP_DATA_PROPERTY_NO_SETTER(Owner, PropertyType, PropertyName, PropertyAccess);	\
EXPOSE_REP_DATA_PROPERTY_SETTER_ONLY(Owner, PropertyType, PropertyName, PropertyAccess, SetterPrivacy)

/** Expose ustruct property for access without using StructProperty in the property name. */
#define EXPOSE_PRIVATE_USTRUCT_REP_DATA_PROPERTY_DIRECT(Owner, PropertyType, StructProperty, ChildProperty)	\
EXPOSE_REP_DATA_PROPERTY_SETTER_ACCESS(Owner, PropertyType, ChildProperty, StructProperty.ChildProperty, private);

/** Expose ustruct property for access without using StructProperty in the property name. */
#define EXPOSE_USTRUCT_REP_DATA_PROPERTY_DIRECT(Owner, PropertyType, StructProperty, ChildProperty)	\
EXPOSE_REP_DATA_PROPERTY_SETTER_ACCESS(Owner, PropertyType, ChildProperty, StructProperty.ChildProperty, public);

 /** 
 * Simplest option - exposes getter and events, but no default setter
 * Revised version - allows the preservation of a previous property name for making delegates and accessors which will work along-side the new names.
 *                   A compile-time deprecation warning will be printed when the old fields are accessed.
 */
#define EXPOSE_REVISED_REP_DATA_PROPERTY_NO_SETTER(Owner, PropertyType, PropertyName, PropertyAccess, DeprecatedPropertyName, DeprecatedVersion)	\
EXPOSE_REP_DATA_PROPERTY_NO_SETTER(Owner, PropertyType, PropertyName, PropertyAccess); \
public:	\
	/** Bind to receive the new property value only on changes */	\
	REP_DATA_PROPERTY_REVISION_WARNING(On##DeprecatedPropertyName##Changed, On##PropertyName##Changed, DeprecatedVersion)	\
	FOn##PropertyName##Changed& On##DeprecatedPropertyName##Changed() const	{ return On##PropertyName##ChangedEvent; }	\
	/** Bind to receive both the new and old property value on changes */	\
	REP_DATA_PROPERTY_REVISION_WARNING(On##DeprecatedPropertyName##ChangedDif, On##PropertyName##ChangedDif, DeprecatedVersion)	\
	FOn##PropertyName##ChangedDif& On##DeprecatedPropertyName##ChangedDif() const { return On##PropertyName##ChangedDifEvent; }	\
	\
	REP_DATA_PROPERTY_REVISION_WARNING(Get##DeprecatedPropertyName, Get##PropertyName, DeprecatedVersion)	\
	PropertyName##ArgType Get##DeprecatedPropertyName() const { return PropertyAccess; }

/**
 * Exposes a rep data property and provides a default property setter.
 * Awkwardly named util - don't bother using directly. Opt for EXPOSE_PRIVATE_REP_DATA_PROPERTY or EXPOSE_REP_DATA_PROPERTY
 * Revised version - allows the preservation of a previous property name for making delegates and accessors which will work along-side the new names.
 *                   A compile-time deprecation warning will be printed when the old fields are accessed.
 */
#define EXPOSE_REVISED_REP_DATA_PROPERTY_SETTER_ACCESS(Owner, PropertyType, PropertyName, PropertyAccess, SetterPrivacy, DeprecatedPropertyName, DeprecatedVersion)	\
EXPOSE_REVISED_REP_DATA_PROPERTY_NO_SETTER(Owner, PropertyType, PropertyName, PropertyAccess, DeprecatedPropertyName, DeprecatedVersion);	\
EXPOSE_REP_DATA_PROPERTY_SETTER_ONLY(Owner, PropertyType, PropertyName, PropertyAccess, SetterPrivacy)	\
SetterPrivacy:	\
	REP_DATA_PROPERTY_REVISION_WARNING(Set##DeprecatedPropertyName, Set##PropertyName, DeprecatedVersion)	\
	void Set##DeprecatedPropertyName(PropertyName##ArgType New##PropertyName)	\
	{	\
		Set##PropertyName(New##PropertyName);	\
	}	\
private: //

/**
 * Exposes a rep data property with a private setter.
 * The setter only allows modification if the local user has authority to alter the property and will automatically trigger delegates appropriately.
 */
#define EXPOSE_PRIVATE_REP_DATA_PROPERTY(Owner, PropertyType, Property)	\
EXPOSE_REP_DATA_PROPERTY_SETTER_ACCESS(Owner, PropertyType, Property, Property, private);

/**
 * Fully exposes the rep data property with a public setter.
 * The setter only allows modification if the local user has authority to alter the property and will automatically trigger delegates appropriately
 */
#define EXPOSE_REP_DATA_PROPERTY(Owner, PropertyType, Property)	\
EXPOSE_REP_DATA_PROPERTY_SETTER_ACCESS(Owner, PropertyType, Property, Property, public);

/**
* Exposes a rep data property with a private setter for a member within a USTRUCT parameter.
* The setter only allows modification if the local user has authority to alter the property and will automatically trigger delegates appropriately.
*/
#define EXPOSE_PRIVATE_USTRUCT_REP_DATA_PROPERTY(Owner, PropertyType, StructProperty, ChildProperty)	\
EXPOSE_REP_DATA_PROPERTY_SETTER_ACCESS(Owner, PropertyType, StructProperty##ChildProperty, StructProperty.ChildProperty, private);

/**
* Fully exposes the rep data property with a public setter for a member within a USTRUCT parameter.
* The setter only allows modification if the local user has authority to alter the property and will automatically trigger delegates appropriately
*/
#define EXPOSE_USTRUCT_REP_DATA_PROPERTY(Owner, PropertyType, StructProperty, ChildProperty)	\
EXPOSE_REP_DATA_PROPERTY_SETTER_ACCESS(Owner, PropertyType, StructProperty##ChildProperty, StructProperty.ChildProperty, public);

/**
* Exposes a rep data property with a private setter for a member within a USTRUCT parameter.
* The setter only allows modification if the local user has authority to alter the property and will automatically trigger delegates appropriately.
* The deprecated name is used to preserve existing delegates and accessors until a later time.
*/
#define EXPOSE_REVISED_PRIVATE_USTRUCT_REP_DATA_PROPERTY(Owner, PropertyType, StructProperty, ChildProperty, DeprecatedPropertyName, DeprecatedVersion)	\
EXPOSE_REVISED_REP_DATA_PROPERTY_SETTER_ACCESS(Owner, PropertyType, StructProperty##ChildProperty, StructProperty.ChildProperty, private, DeprecatedPropertyName, DeprecatedVersion);

/**
* Fully exposes the rep data property with a public setter for a member within a USTRUCT parameter.
* The setter only allows modification if the local user has authority to alter the property and will automatically trigger delegates appropriately
* The deprecated name is used to preserve existing delegates and accessors until a later time.
*/
#define EXPOSE_REVISED_USTRUCT_REP_DATA_PROPERTY(Owner, PropertyType, StructProperty, ChildProperty, DeprecatedPropertyName, DeprecatedVersion)	\
EXPOSE_REVISED_REP_DATA_PROPERTY_SETTER_ACCESS(Owner, PropertyType, StructProperty##ChildProperty, StructProperty.ChildProperty, public, DeprecatedPropertyName, DeprecatedVersion);

PARTY_API const TCHAR* ToString(EPartyType Type);
PARTY_API const TCHAR* ToString(EApprovalAction Type);

/** Defines the means used to join a party*/
namespace PartyJoinMethod
{
	// User has created the party
	extern PARTY_API const FName Creation;
	// User has joined the party via invitation
	extern PARTY_API const FName Invitation;
	// user has joined after requesting access
	extern PARTY_API const FName RequestToJoin;
	// User has joined the party via presence
	extern PARTY_API const FName Presence;
	// User has joined the party using a platform option
	extern PARTY_API const FName PlatformSession;
	// User has joined the party via command line
	extern PARTY_API const FName CommandLineJoin;
	// User has joined via unknown/undocumented process
	extern PARTY_API const FName Unspecified;
}

#undef UE_API
