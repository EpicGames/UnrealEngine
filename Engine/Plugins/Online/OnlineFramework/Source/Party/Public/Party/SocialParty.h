// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Party/PartyDataReplicator.h"

#include "SocialTypes.h"
#include "SpectatorBeaconClient.h"
#include "Interfaces/OnlineChatInterface.h"
#include "Containers/Queue.h"
#include "SocialParty.generated.h"

#define UE_API PARTY_API

class AOnlineBeaconClient;
class FOnlinePartyId;
class FOnlinePartyTypeId;
enum ETravelType : int;
namespace EPartyReservationResult { enum Type : int; }

class APartyBeaconClient;
class UNetDriver;

class ULocalPlayer;
class USocialManager;
class USocialUser;

class FOnlineSessionSettings;
class FOnlineSessionSearchResult;
enum class EMemberExitedReason : uint8;
struct FPartyMemberJoinInProgressRequest;
struct FPartyMemberJoinInProgressResponse;

/** Base struct used to replicate data about the state of the party to all members. */
USTRUCT()
struct FPartyRepData : public FOnlinePartyRepDataBase
{
	GENERATED_BODY();

public:
	FPartyRepData() {}
	FPartyRepData& operator=(const FPartyRepData& Other)
	{
		FOnlinePartyRepDataBase::operator=(Other);
		
		// Do not copy multicasts, because this makes it very hard to track the lifetime of registered delegates
		
		OwnerParty = Other.OwnerParty;
		bAllowOwnerless = Other.bAllowOwnerless;
		PrivacySettings = Other.PrivacySettings;
		PlatformSessions = Other.PlatformSessions;

		return *this;
	}

	UE_API virtual void SetOwningParty(const class USocialParty& InOwnerParty);
	/** Mark the party data as ownerless. This will bypass any "CanEdit" checks. Useful for using this object in a test context. */
	UE_API void MarkOwnerless();

	UE_API const FPartyPlatformSessionInfo* FindSessionInfo(const FString& SessionType) const;
	const TArray<FPartyPlatformSessionInfo>& GetPlatformSessions() const { return PlatformSessions; }
	FSimpleMulticastDelegate& OnPlatformSessionsChanged() const { return OnPlatformSessionsChangedEvent; } 

	UE_API void UpdatePlatformSessionInfo(FPartyPlatformSessionInfo&& SessionInfo);
	UE_API void ClearPlatformSessionInfo(const FString& SessionType);

	/** Fires after received rep data has been fully applied */
	FSimpleMulticastDelegate& OnPostReplicationProcessed() const { return OnPostReplicationProcessedEvent; }

protected:
	UE_API virtual bool CanEditData() const override;
	UE_API virtual void CompareAgainst(const FOnlinePartyRepDataBase& OldData) const override;
	UE_API virtual const USocialParty* GetOwnerParty() const override;
	UE_API virtual void PostReplication(const FOnlinePartyRepDataBase& OldData, TSharedPtr<FStreamableHandle> InResourceHandle) override;

	TWeakObjectPtr<const USocialParty> OwnerParty;
	bool bAllowOwnerless = false;

	//@todo DanH Party: Isn't this redundant with the party config itself? Why bother putting it here too when the config replicates to everyone already? #suggested
	/** The privacy settings for the party */
	UPROPERTY()
	FPartyPrivacySettings PrivacySettings;
	EXPOSE_REP_DATA_PROPERTY(FPartyRepData, FPartyPrivacySettings, PrivacySettings);

	/** List of platform sessions for the party. Includes one entry per platform that needs a session and has a member of that session. */
	UPROPERTY()
	TArray<FPartyPlatformSessionInfo> PlatformSessions;

private:
	mutable FSimpleMulticastDelegate OnPlatformSessionsChangedEvent;
	mutable FSimpleMulticastDelegate OnPostReplicationProcessedEvent;
};

using FPartyDataReplicator = TPartyDataReplicator<FPartyRepData, USocialParty>;

/**
 * Party game state that contains all information relevant to the communication within a party
 * Keeps all players in sync with the state of the party and its individual members
 */
UCLASS(MinimalAPI, Abstract, Within=SocialManager, config=Game, Transient)
class USocialParty : public UObject
{
	GENERATED_BODY()

	friend class FPartyPlatformSessionMonitor;
	friend UPartyMember;
	friend USocialManager;
	friend USocialUser;
public:
	static UE_API bool IsJoiningDuringLoadEnabled();

	UE_API USocialParty();

	/** Re-evaluates whether this party is joinable by anyone and, if not, establishes the reason why */
	UE_API void RefreshPublicJoinability();

	DECLARE_DELEGATE_OneParam(FOnLeavePartyAttemptComplete, ELeavePartyCompletionResult)
	UE_API virtual void LeaveParty(const FOnLeavePartyAttemptComplete& OnLeaveAttemptComplete = FOnLeavePartyAttemptComplete());
	UE_API virtual void RemoveLocalMember(const FUniqueNetIdRepl& LocalUserId, const FOnLeavePartyAttemptComplete& OnLeaveAttemptComplete = FOnLeavePartyAttemptComplete());

	const FPartyRepData& GetRepData() const { return *PartyDataReplicator; }

	template <typename SocialManagerT = USocialManager>
	SocialManagerT& GetSocialManager() const
	{
		SocialManagerT* ManagerOuter = GetTypedOuter<SocialManagerT>();
		check(ManagerOuter);
		return *ManagerOuter;
	}
	
	template <typename MemberT = UPartyMember>
	MemberT& GetOwningLocalMember() const
	{
		MemberT* LocalMember = GetPartyMember<MemberT>(OwningLocalUserId);
		check(LocalMember);
		return *LocalMember;
	}

	template <typename MemberT = UPartyMember>
	MemberT* GetPartyLeader() const
	{
		return GetPartyMember<MemberT>(CurrentLeaderId);
	}

	template <typename MemberT = UPartyMember>
	MemberT* GetPartyMember(const FUniqueNetIdRepl& MemberId) const
	{
		return Cast<MemberT>(GetMemberInternal(MemberId));
	}

	UE_API bool ContainsUser(const USocialUser& User) const;

	UE_API ULocalPlayer* GetOwningLocalPlayerPtr() const;
	const FUniqueNetIdRepl& GetOwningLocalUserId() const { return OwningLocalUserId; }
	const FUniqueNetIdRepl& GetPartyLeaderId() const { return CurrentLeaderId; }
	UE_API bool IsLocalPlayerPartyLeader() const;
	UE_API bool IsPartyLeader(const ULocalPlayer& LocalPlayer) const;
	UE_API bool IsPartyLeaderLocal() const;

	UE_API FChatRoomId GetChatRoomId() const;
	UE_API bool IsPersistentParty() const;
	UE_API const FOnlinePartyTypeId& GetPartyTypeId() const;
	UE_API const FOnlinePartyId& GetPartyId() const;

	UE_API EPartyState GetOssPartyState() const;
	UE_API EPartyState GetOssPartyPreviousState() const;

	UE_API bool IsCurrentlyCrossplaying() const;
	UE_API bool IsPartyFunctionalityDegraded() const;

	UE_API bool IsPartyFull() const;
	UE_API int32 GetNumPartyMembers() const;
	UE_API void SetPartyMaxSize(int32 NewSize);
	UE_API int32 GetPartyMaxSize() const;
	UE_API FPartyJoinDenialReason GetPublicJoinability() const;
	bool IsLeavingParty() const { return bIsLeavingParty; }

	/** Is the specified net driver for our reservation beacon? */
	UE_API bool IsNetDriverFromReservationBeacon(const UNetDriver* InNetDriver) const;

	UE_API virtual void DisconnectParty();

	template <typename MemberT = UPartyMember>
	TArray<MemberT*> GetPartyMembers() const
	{
		TArray<MemberT*> PartyMembers;
		PartyMembers.Reserve(PartyMembersById.Num());
		for (const auto& IdMemberPair : PartyMembersById)
		{
			if (MemberT* PartyMember = Cast<MemberT>(IdMemberPair.Value))
			{
				PartyMembers.Add(PartyMember);
			}
		}
		return PartyMembers;
	}

	UE_API FString ToDebugString() const;

	DECLARE_EVENT_OneParam(USocialParty, FLeavePartyEvent, EMemberExitedReason);
	FLeavePartyEvent& OnPartyLeaveBegin() const { return OnPartyLeaveBeginEvent; }
	FLeavePartyEvent& OnPartyLeft() const { return OnPartyLeftEvent; }

	DECLARE_EVENT(USocialParty, FDisconnectPartyEvent);
	FDisconnectPartyEvent& OnPartyDisconnected() const { return OnPartyDisconnectedEvent; }

	DECLARE_EVENT_OneParam(USocialParty, FOnPartyMemberCreated, UPartyMember&);
	FOnPartyMemberCreated& OnPartyMemberCreated() const { return OnPartyMemberCreatedEvent; }

	DECLARE_EVENT_TwoParams(USocialParty, FOnPartyMemberLeft, UPartyMember*, const EMemberExitedReason);
	FOnPartyMemberLeft& OnPartyMemberLeft() const { return OnPartyMemberLeftEvent; }

	/**
	 * Event triggered when the party leader changes.
	 * @param NewLeader the new leader. Generally non-null, but there may be race conditions where we get the leader id before we get the leader's data.
	 */
	DECLARE_EVENT_OneParam(USocialParty, FOnPartyLeaderChanged, UPartyMember* /*NewLeader*/);
	FOnPartyLeaderChanged& OnPartyLeaderChanged() const { return OnPartyLeaderChangedEvent; }

	DECLARE_EVENT_OneParam(USocialParty, FOnPartyConfigurationChanged, const FPartyConfiguration&);
	FOnPartyConfigurationChanged& OnPartyConfigurationChanged() const { return OnPartyConfigurationChangedEvent; }

	DECLARE_EVENT_TwoParams(USocialParty, FOnPartyStateChanged, EPartyState, EPartyState);
	FOnPartyStateChanged& OnPartyStateChanged() const { return OnPartyStateChangedEvent; }

	DECLARE_EVENT_OneParam(USocialParty, FOnPartyFunctionalityDegradedChanged, bool /*bFunctionalityDegraded*/);
	FOnPartyFunctionalityDegradedChanged& OnPartyFunctionalityDegradedChanged() const { return OnPartyFunctionalityDegradedChangedEvent; }

	DECLARE_EVENT_OneParam(USocialParty, FOnInviteSent, const USocialUser&);
	FOnInviteSent& OnInviteSent() const { return OnInviteSentEvent; }

	DECLARE_EVENT_TwoParams(USocialParty, FOnPartyMemberConnectionStatusChanged, UPartyMember&, EMemberConnectionStatus);
	FOnPartyMemberConnectionStatusChanged& OnPartyMemberConnectionStatusChanged() const { return OnPartyMemberConnectionStatusChangedEvent; }

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnInitializationCompletePreNotify, USocialParty&);
	FOnInitializationCompletePreNotify& OnInitializationCompletePreNotify() const { return OnInitializationCompletePreNotifyEvent; }

	UE_API void ResetPrivacySettings();
	UE_API const FPartyPrivacySettings& GetPrivacySettings() const;

	UE_API virtual bool ShouldAlwaysJoinPlatformSession(const FSessionId& SessionId) const;

	/** Hook for subclasses to add game-specific keys to NAME_PartySession (default: no-op). */
	UE_API virtual void AugmentPlatformSessionSettings(FOnlineSessionSettings& SessionSettings) const;

	/** Fired by subclasses to ask FPartyPlatformSessionMonitor to re-advertise the platform session. */
	DECLARE_EVENT(USocialParty, FOnPlatformSessionUpdateRequested);
	FOnPlatformSessionUpdateRequested& OnPlatformSessionUpdateRequested() const { return OnPlatformSessionUpdateRequestedEvent; }

	UE_API virtual void JoinSessionCompleteAnalytics(const FSessionId& SessionId, const FString& JoinBootableGroupSessionResult);
	UE_API bool IsCurrentlyLeaving() const;

	DECLARE_DELEGATE_OneParam(FOnRequestJoinInProgressComplete, const EPartyJoinDenialReason /*DenialReason*/);
	UE_API void RequestJoinInProgress(const UPartyMember& TargetMember, const FOnRequestJoinInProgressComplete& CompletionDelegate);
	UE_API void CancelJoinInProgressRequest();
	UE_API bool IsJoinInProgeressRequestActive() const;

	UE_API bool CanInviteUser(const USocialUser& User, const ESocialPartyInviteMethod InviteMethod = ESocialPartyInviteMethod::Other) const;

	UE_API bool IsInitialized() const;

protected:
	UE_API void InitializeParty(const TSharedRef<const FOnlineParty>& InOssParty);
	UE_API void TryFinishInitialization();
	UE_API void NotifyRepDataProcessed();

	UE_API void SetIsMissingPlatformSession(bool bInIsMissingPlatformSession);
	bool IsMissingPlatformSession() { return bIsMissingPlatformSession; }

	FPartyRepData& GetMutableRepData() { return *PartyDataReplicator; }

	//--------------------------
	// User/member-specific actions that are best exposed on the individuals themselves, but best handled by the actual party
	UE_API bool HasUserBeenInvited(const USocialUser& User) const;

	UE_API bool CanPromoteMember(const ULocalPlayer& PerformingPlayer, const UPartyMember& PartyMember) const;
	UE_API virtual bool CanKickMember(const ULocalPlayer& PerformingPlayer, const UPartyMember& PartyMember) const;
	UE_API bool TryPromoteMember(const ULocalPlayer& PerformingPlayer, const UPartyMember& PartyMember);
	UE_API virtual bool TryKickMember(const ULocalPlayer& PerformingPlayer, const UPartyMember& PartyMember);
	
	UE_API bool TryInviteUser(const USocialUser& UserToInvite, const ESocialPartyInviteMethod InviteMethod = ESocialPartyInviteMethod::Other, const FString& MetaData = FString());
	//--------------------------

	UE_API virtual bool AllowJoinInProgressToMember();

protected:
	UE_API virtual void InitializePartyInternal();

	FPartyConfiguration& GetCurrentConfiguration() { return CurrentConfig; }

	/** Only called when a new party is being created by the local player and they are responsible for the rep data. Otherwise we just wait to receive it from the leader. */
	UE_API virtual void InitializePartyRepData();
	UE_API virtual FPartyPrivacySettings GetDesiredPrivacySettings() const;
	static UE_API FPartyPrivacySettings GetPrivacySettingsForConfig(const FPartyConfiguration& PartyConfig);
	UE_API virtual void OnLocalPlayerIsLeaderChanged(bool bIsLeader);
	UE_API virtual void HandlePrivacySettingsChanged(const FPartyPrivacySettings& NewPrivacySettings);
	UE_API virtual void OnMemberCreatedInternal(UPartyMember& NewMember);
	UE_API virtual void OnLeftPartyInternal(EMemberExitedReason Reason);

	/** Virtual versions of the package-scoped "CanX" methods above, as a virtual declared within package scoping cannot link (exported public, imported protected) */
	UE_API virtual ESocialPartyInviteFailureReason CanInviteUserInternal(const USocialUser& User, const ESocialPartyInviteMethod InviteMethod) const;

	UE_API virtual bool CanPromoteMemberInternal(const ULocalPlayer& PerformingPlayer, const UPartyMember& PartyMember) const;
	UE_API virtual bool CanKickMemberInternal(const ULocalPlayer& PerformingPlayer, const UPartyMember& PartyMember) const;

	UE_API virtual void OnInviteSentInternal(ESocialSubsystem SubsystemType, const USocialUser& InvitedUser, bool bWasSuccessful, const ESocialPartyInviteFailureReason FailureReason, const ESocialPartyInviteMethod InviteMethod, const FString& MetaData);

	UE_DEPRECATED(5.6, "Override 'void OnInviteSentInternal(ESocialSubsystem SubsystemType, const USocialUser& InvitedUser, bool bWasSuccessful, const ESocialPartyInviteFailureReason FailureReason, const ESocialPartyInviteMethod InviteMethod, const FString& MetaData)' instead")
	virtual void OnInviteSentInternal(ESocialSubsystem SubsystemType, const USocialUser& InvitedUser, bool bWasSuccessful, const ESocialPartyInviteFailureReason FailureReason, const ESocialPartyInviteMethod InviteMethod) {};
	
	UE_DEPRECATED(5.6, "Override 'void OnInviteSentInternal(ESocialSubsystem SubsystemType, const USocialUser& InvitedUser, bool bWasSuccessful, const ESocialPartyInviteFailureReason FailureReason, const ESocialPartyInviteMethod InviteMethod, const FString& MetaData)' instead")
	virtual void OnInviteSentInternal(ESocialSubsystem SubsystemType, const USocialUser& InvitedUser, bool bWasSuccessful) {};

	UE_API virtual void HandlePartySystemStateChange(EPartySystemState NewState);

	/** Determines the joinability of this party for a group of users requesting to join */
	UE_API virtual FPartyJoinApproval EvaluateJoinRequest(const TArray<IOnlinePartyUserPendingJoinRequestInfoConstRef>& Players, bool bFromJoinRequest) const;
	/**
	 * Perform the Away status check for an incoming join request
	 * @return true if the join request passes the away check
	 */
	UE_API virtual bool EvaluateJoinRequestAwayCheck(const TArray<IOnlinePartyUserPendingJoinRequestInfoConstRef>& Players, bool bFromJoinRequest) const;

	/** Determines the reason why, if at all, this party is currently flat-out unjoinable  */
	UE_API virtual FPartyJoinDenialReason DetermineCurrentJoinability() const;

	/** Override in child classes to specify the type of UPartyMember to create */
	UE_API virtual TSubclassOf<UPartyMember> GetDesiredMemberClass(bool bLocalPlayer) const;

	/** Override in child classes to provide encryption data for party beacon connections. */
	UE_API virtual bool InitializeBeaconEncryptionData(AOnlineBeaconClient& BeaconClient, const FString& SessionId);

	/** The list of party members to send the request for joining in progress. */
	UE_API virtual TArray<UPartyMember*> GetLocalPartyMembersForJoinInProgress() const;

	/** Override in child classes to provide extra invite metadata. */
	UE_API virtual FString GetInviteMetadata(const FString& ExistingMetadata);

	UE_API bool IsInviteRateLimited(const USocialUser& User, ESocialSubsystem SubsystemType) const;

	UE_API bool ApplyCrossplayRestriction(FPartyJoinApproval& JoinApproval, const FUserPlatform& Platform, const FOnlinePartyData& JoinData) const;
	UE_API FName GetGameSessionName() const;
	UE_API bool IsInRestrictedGameSession() const;

	/**
	 * Create a reservation beacon and connect to the server to get approval for new party members
	 * Only relevant while in an active game, not required while pre lobby / game
	 */
	UE_API void ConnectToReservationBeacon();
	UE_API void CleanupReservationBeacon();
	UE_API APartyBeaconClient* CreateReservationBeaconClient();

	APartyBeaconClient* GetReservationBeaconClient() const { return ReservationBeaconClient.Get(); }

	/**
	* Create a spectator beacon and connect to the server to get approval for new spectators
	*/
	UE_API void CleanupSpectatorBeacon();
	UE_API ASpectatorBeaconClient* CreateSpectatorBeaconClient();

	ASpectatorBeaconClient* GetSpectatorBeaconClient() const { return SpectatorBeaconClient.Get(); }

	/** Child classes MUST call EstablishRepDataInstance() on this using their member rep data struct instance */
	FPartyDataReplicator PartyDataReplicator;

	/** Reservation beacon class for getting server approval for new party members while in a game */
	UPROPERTY()
	TSubclassOf<APartyBeaconClient> ReservationBeaconClientClass;

	/** Spectator beacon class for getting server approval for new spectators while in a game */
	UPROPERTY()
	TSubclassOf<ASpectatorBeaconClient> SpectatorBeaconClientClass;

	/** Apply local party configuration to the OSS party, optionally resetting the access key to the party in the process */
	UE_API void UpdatePartyConfig(bool bResetAccessKey = false);

private:
	UE_API UPartyMember* GetOrCreatePartyMember(const FUniqueNetId& MemberId);
	UE_API void PumpApprovalQueue();
	UE_API void RejectAllPendingJoinRequests();
	UE_API void SetIsMissingXmppConnection(bool bInMissingXmppConnection);
	UE_API void BeginLeavingParty(EMemberExitedReason Reason);
	UE_API void FinalizePartyLeave(EMemberExitedReason Reason);

	UE_API void SetIsRequestingShutdown(bool bInRequestingShutdown);

	UE_API void CreatePlatformSession(const FString& SessionType);
	UE_API void UpdatePlatformSessionLeader(const FString& SessionType);

	UE_API void HandlePreClientTravel(const FString& PendingURL, ETravelType TravelType, bool bIsSeamlessTravel);

	UE_API UPartyMember* GetMemberInternal(const FUniqueNetIdRepl& MemberId) const;

protected:  // Handlers
	UE_API virtual void HandlePartyStateChanged(const FUniqueNetId& LocalUserId, const FOnlinePartyId& PartyId, EPartyState PartyState, EPartyState PreviousPartyState);
private:
	UE_API void HandlePartyConfigChanged(const FUniqueNetId& LocalUserId, const FOnlinePartyId& PartyId, const FPartyConfiguration& PartyConfig);
	UE_API void HandleUpdatePartyConfigComplete(const FUniqueNetId& LocalUserId, const FOnlinePartyId& PartyId, EUpdateConfigCompletionResult Result);
	UE_API void HandlePartyDataReceived(const FUniqueNetId& LocalUserId, const FOnlinePartyId& PartyId, const FName& Namespace, const FOnlinePartyData& PartyData);
	UE_API void HandleJoinabilityQueryReceived(const FUniqueNetId& LocalUserId, const FOnlinePartyId& PartyId, const IOnlinePartyPendingJoinRequestInfo& JoinRequestInfo);
	UE_API void HandlePartyJoinRequestReceived(const FUniqueNetId& LocalUserId, const FOnlinePartyId& PartyId, const IOnlinePartyPendingJoinRequestInfo& JoinRequestInfo);
	UE_API void HandlePartyLeft(const FUniqueNetId& LocalUserId, const FOnlinePartyId& PartyId);
	UE_API void HandlePartyMemberExited(const FUniqueNetId& LocalUserId, const FOnlinePartyId& PartyId, const FUniqueNetId& MemberId, EMemberExitedReason ExitReason);
	UE_API void HandlePartyMemberDataReceived(const FUniqueNetId& LocalUserId, const FOnlinePartyId& PartyId, const FUniqueNetId& MemberId, const FName& Namespace, const FOnlinePartyData& PartyMemberData);
	UE_API void HandlePartyMemberJoined(const FUniqueNetId& LocalUserId, const FOnlinePartyId& PartyId, const FUniqueNetId& MemberId);
	UE_API void HandlePartyMemberPromoted(const FUniqueNetId& LocalUserId, const FOnlinePartyId& PartyId, const FUniqueNetId& NewLeaderId);
	UE_API void HandlePartyPromotionLockoutChanged(const FUniqueNetId& LocalUserId, const FOnlinePartyId& PartyId, bool bArePromotionsLocked);

	UE_API void HandleMemberInitialized(UPartyMember* Member);
	UE_API void HandleMemberPlatformUniqueIdChanged(const FUniqueNetIdRepl& NewPlatformUniqueId, UPartyMember* Member);
	UE_API void HandleMemberSessionIdChanged(const FSessionId& NewSessionId, UPartyMember* Member);

	UE_API void HandleBeaconHostConnectionFailed();
	UE_API void HandleReservationRequestComplete(EPartyReservationResult::Type ReservationResponse);

	UE_API void HandleLeavePartyComplete(const FUniqueNetId& LocalUserId, const FOnlinePartyId& PartyId, ELeavePartyCompletionResult LeaveResult, FOnLeavePartyAttemptComplete OnAttemptComplete);
	UE_API void HandleRemoveLocalPlayerComplete(const FUniqueNetId& LocalUserId, const FOnlinePartyId& PartyId, ELeavePartyCompletionResult LeaveResult, FOnLeavePartyAttemptComplete OnAttemptComplete);

	UE_API void RemovePlayerFromReservationBeacon(const FUniqueNetId& LocalUserId, const FUniqueNetId& PlayerToRemove);

	UE_API void HandleJoinInProgressDataRequestChanged(const FPartyMemberJoinInProgressRequest& Request, UPartyMember* Member);
	UE_API void HandleJoinInProgressDataResponsesChanged(const TArray<FPartyMemberJoinInProgressResponse>& Responses, UPartyMember* Member);

private:
	TSharedPtr<const FOnlineParty> OssParty;

	UPROPERTY()
	FUniqueNetIdRepl OwningLocalUserId;

	/** Tracked explicitly so we know which player was demoted whenever the leader changes */
	UPROPERTY()
	FUniqueNetIdRepl CurrentLeaderId;

	UPROPERTY()
	TMap<FUniqueNetIdRepl, TObjectPtr<UPartyMember>> PartyMembersById;

	TMap<FUniqueNetIdRepl, double> LastInviteSentById;

	UPROPERTY(config)
	double PlatformUserInviteCooldown = 10.f;

	UPROPERTY(config)
	double PrimaryUserInviteCooldown = 0.f;

	FPartyConfiguration CurrentConfig;

	//@todo DanH Party: Rename/reorg this to more clearly call out that this is specific to lobby beacon stuff #suggested
	struct FPendingMemberApproval
	{
		struct FMemberInfo
		{
			FMemberInfo(FUniqueNetIdRepl InMemberId, FUserPlatform InPlatform, TSharedPtr<const FOnlinePartyData> InJoinData = TSharedPtr<const FOnlinePartyData>())
				: MemberId(InMemberId)
				, Platform(MoveTemp(InPlatform))
				, JoinData(InJoinData)
			{}

			FUniqueNetIdRepl MemberId;
			FUserPlatform Platform;
			TSharedPtr<const FOnlinePartyData> JoinData;
		};

		FUniqueNetIdRepl RecipientId;
		TArray<FMemberInfo> Members;
		bool bIsJIPApproval;
		int64 JoinInProgressRequestTime = 0;
		bool bIsPlayerRemoval = false;
	};
	TQueue<FPendingMemberApproval> PendingApprovals;

	bool bStayWithPartyOnDisconnect = false;
	bool bIsMemberPromotionPossible = true;
	
	/**
	 * Last known reservation beacon client net driver name
	 * Intended to be used to detect network errors related to our current or last reservation beacon client's net driver.
	 * Some network error handlers may be called after we cleanup our beacon connection.
	 */
	FName LastReservationBeaconClientNetDriverName;
	
	/** Reservation beacon client instance while getting approval for new party members*/
	UPROPERTY()
	TWeakObjectPtr<APartyBeaconClient> ReservationBeaconClient = nullptr;
	
	/**
	* Last known spectator beacon client net driver name
	* Intended to be used to detect network errors related to our current or last spectator beacon client's net driver.
	* Some network error handlers may be called after we cleanup our beacon connection.
	*/
	FName LastSpectatorBeaconClientNetDriverName;
	
	/** Spectator beacon client instance while getting approval for spectator*/
	UPROPERTY()
	TWeakObjectPtr<ASpectatorBeaconClient> SpectatorBeaconClient = nullptr;

	/**
	 * True when we have limited functionality due to lacking an xmpp connection.
	 * Don't set directly, use the private setter to trigger events appropriately.
	 */
	TOptional<bool> bIsMissingXmppConnection;
	bool bIsMissingPlatformSession = false;

	bool bIsLeavingParty = false;
	bool bIsInitialized = false;
	bool bHasReceivedRepData = false;
	TOptional<bool> bIsRequestingShutdown;

	UE_API void RespondToJoinInProgressRequest(const FPendingMemberApproval& PendingApproval, const EPartyJoinDenialReason DenialReason);
	UE_API void CallJoinInProgressComplete(const EPartyJoinDenialReason DenialReason);
	UE_API void RunJoinInProgressTimer();

	/** Complete delegate for join in progress requests. This should only have one at a time. */
	TOptional<FOnRequestJoinInProgressComplete> RequestJoinInProgressComplete;

	FTimerHandle JoinInProgressTimerHandle;

	/** How often the timer should check in seconds for stale data when running. */
	UPROPERTY(config)
	float JoinInProgressTimerRate = 5.f;
	
	/** How long in seconds before join in progress requests timeout and are cleared from member data. */
	UPROPERTY(config)
	int32 JoinInProgressRequestTimeout = 30;

	/** How long in seconds before join in progress responses are cleared from member data. */
	UPROPERTY(config)
	int32 JoinInProgressResponseTimeout = 60;

	/** Maximum number of join in progress responses to store in member rep data. Oldest responses are removed first. */
	UPROPERTY(config)
	int32 MaxJoinInProgressResponses = 12;

	/** Maximum number of join in progress responses to store per requesting member. Oldest responses are removed first. */
	UPROPERTY(config)
	int32 MaxJoinInProgressResponsesPerMember = 5;

	mutable FLeavePartyEvent OnPartyLeaveBeginEvent;
	mutable FLeavePartyEvent OnPartyLeftEvent;
	mutable FDisconnectPartyEvent OnPartyDisconnectedEvent;
	mutable FOnPartyMemberCreated OnPartyMemberCreatedEvent;
	mutable FOnPartyMemberLeft OnPartyMemberLeftEvent;
	mutable FOnPartyLeaderChanged OnPartyLeaderChangedEvent;
	mutable FOnPartyConfigurationChanged OnPartyConfigurationChangedEvent;
	mutable FOnPartyStateChanged OnPartyStateChangedEvent;
	mutable FOnPartyMemberConnectionStatusChanged OnPartyMemberConnectionStatusChangedEvent;
	mutable FOnPartyFunctionalityDegradedChanged OnPartyFunctionalityDegradedChangedEvent;
	mutable FOnInviteSent OnInviteSentEvent;
	mutable FOnInitializationCompletePreNotify OnInitializationCompletePreNotifyEvent;
	mutable FOnPlatformSessionUpdateRequested OnPlatformSessionUpdateRequestedEvent;
};

namespace UE::OnlineFramework
{
PARTY_API TArray<FUniqueNetIdRepl> GetPartyMemberIds(const USocialParty& SocialParty);
PARTY_API TArray<USocialToolkit*> GetLocalPartyMemberToolkits(const USocialParty& SocialParty);
} // UE::OnlineFramework::Party

#undef UE_API
