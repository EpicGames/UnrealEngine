// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Stats/Stats.h"
#include "Modules/ModuleManager.h"

/** Logging related to parties */
PARTY_API DECLARE_LOG_CATEGORY_EXTERN(LogParty, Display, All);

/** Party module stats */
DECLARE_STATS_GROUP(TEXT("Party"), STATGROUP_Party, STATCAT_Advanced);
/** Total async thread time */
DECLARE_CYCLE_STAT_EXTERN(TEXT("PartyStat1"), STAT_PartyStat1, STATGROUP_Party, PARTY_API);

/**
 * Module for the party framework that builds on top of IOnlineParty
 */
class FPartyModule : 
	public IModuleInterface
{

public:

	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static inline FPartyModule& Get()
	{
		return FModuleManager::LoadModuleChecked<FPartyModule>("Party");
	}

	/**
	 * Checks to see if this module is loaded and ready.  It is only valid to call Get() if IsAvailable() returns true.
	 *
	 * @return True if the module is loaded and ready to use
	 */
	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded("Party");
	}

private:

	// IModuleInterface

	/**
	 * Called when party module is loaded
	 * Initialize platform specific parts of voice handling
	 */
	virtual void StartupModule() override;
	
	/**
	 * Called when party module is unloaded
	 * Shutdown platform specific parts of voice handling
	 */
	virtual void ShutdownModule() override;
};
