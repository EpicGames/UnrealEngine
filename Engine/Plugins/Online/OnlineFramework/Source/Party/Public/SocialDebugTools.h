// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/OnlineAsyncOpHandle.h"

#include "SocialDebugTools.generated.h"

#define UE_API PARTY_API

class FOnlinePartyId;
class FUniqueNetId;
class IOnlinePartyJoinInfo;
class USocialManager;

class IOnlineSubsystem;
class FOnlineAccountCredentials;
class IOnlinePartyPendingJoinRequestInfo;
struct FPartyMemberJoinInProgressRequest;
typedef TSharedPtr<const class IOnlinePartyJoinInfo> IOnlinePartyJoinInfoConstPtr;
typedef TSharedPtr<class FOnlinePartyData> FOnlinePartyDataPtr;
typedef TSharedPtr<const class FOnlinePartyData> FOnlinePartyDataConstPtr;
using FUniqueNetIdPtr = TSharedPtr<const FUniqueNetId>;
enum class EPartyJoinDenialReason : uint8;

UCLASS(MinimalAPI, Within = SocialManager, Config = Game)
class USocialDebugTools : public UObject, public FExec
{
	GENERATED_BODY()

	static constexpr const int32 LocalUserNum = 0;

public:
	UE_API USocialManager& GetSocialManager() const;

	// FExec
#if UE_ALLOW_EXEC_COMMANDS
	UE_API virtual bool Exec(class UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Out) override;
#endif

	// USocialDebugTools

	UE_API USocialDebugTools();
	UE_API virtual void Shutdown();

	DECLARE_DELEGATE_OneParam(FLoginComplete, bool);
	UE_API virtual void Login(const FString& Instance, const FOnlineAccountCredentials& Credentials, const FLoginComplete& OnComplete);

	DECLARE_DELEGATE_OneParam(FLogoutComplete, bool);
	UE_API virtual void Logout(const FString& Instance, const FLogoutComplete& OnComplete);

	DECLARE_DELEGATE_OneParam(FJoinPartyComplete, bool);
	UE_API virtual void JoinParty(const FString& Instance, const FString& FriendName, const FJoinPartyComplete& OnComplete);

	DECLARE_DELEGATE_OneParam(FJoinInProgressComplete, EPartyJoinDenialReason);
	UE_API virtual void JoinInProgress(const FString& Instance, const FJoinInProgressComplete& OnComplete);

	DECLARE_DELEGATE_OneParam(FLeavePartyComplete, bool);
	UE_API virtual void LeaveParty(const FString& Instance, const FLeavePartyComplete& OnComplete);

	DECLARE_DELEGATE_OneParam(FCleanupPartiesComplete, bool);
	UE_API virtual void CleanupParties(const FString& Instance, const FCleanupPartiesComplete& OnComplete);

	DECLARE_DELEGATE_OneParam(FSetPartyMemberDataComplete, bool);
	UE_API virtual void SetPartyMemberData(const FString& Instance, const UStruct* StructType, const void* StructData, const FSetPartyMemberDataComplete& OnComplete);
	UE_API virtual void SetPartyMemberDataJson(const FString& Instance, const FString& JsonStr, const FSetPartyMemberDataComplete& OnComplete);

	virtual void GetContextNames(TArray<FString>& OutContextNames) const { Contexts.GenerateKeyArray(OutContextNames); }

	struct FInstanceContext
	{
		FInstanceContext(const FString& InstanceName, USocialDebugTools& SocialDebugTools)
			: Name(InstanceName)
			, OnlineSub(nullptr)
			, Owner(SocialDebugTools)
		{}

		void Init();
		void Shutdown();
		IOnlineSubsystem* GetOSS() const { return OnlineSub; }
		FOnlinePartyDataPtr GetPartyMemberData() const { return PartyMemberData; }
		FUniqueNetIdPtr GetLocalUserId() const;
		void ModifyPartyField(const FString& FieldName, const class FVariantData& FieldValue);

		bool SetJIPRequest(const FPartyMemberJoinInProgressRequest& InRequest);

		FString Name;
		IOnlineSubsystem* OnlineSub;
		USocialDebugTools& Owner;
		FOnlinePartyDataPtr PartyMemberData;

		// delegates
		FDelegateHandle LoginCompleteDelegateHandle;
		FDelegateHandle LogoutCompleteDelegateHandle;
		UE::Online::FOnlineEventDelegateHandle OnPresenceUpdatedHandle;
		FDelegateHandle PresenceReceivedDelegateHandle;
		FDelegateHandle FriendInviteReceivedDelegateHandle;
		FDelegateHandle PartyInviteReceivedDelegateHandle;
		FDelegateHandle PartyJoinRequestReceivedDelegateHandle;
	};

	UE_API FInstanceContext& GetContext(const FString& Instance);
	UE_API FInstanceContext* GetContextForUser(const FUniqueNetId& UserId);

protected:
	UE_API virtual void PrintExecCommands() const;
	UE_API virtual bool RunCommand(const TCHAR* Cmd, const TArray<FString>& TargetInstances);
	virtual void NotifyContextInitialized(const FInstanceContext& Context) { }

private:
	bool bAutoAcceptFriendInvites;
	bool bAutoAcceptPartyInvites;

	TMap<FString, FInstanceContext> Contexts;

	UE_API IOnlinePartyJoinInfoConstPtr GetDefaultPartyJoinInfo() const;
	UE_API IOnlineSubsystem* GetDefaultOSS() const;
	UE_API void PrintExecUsage() const;

	// OSS callback handlers
	UE_API void HandleFriendInviteReceived(const FUniqueNetId& LocalUserId, const FUniqueNetId& FriendId);
	UE_API void HandlePartyInviteReceived(const FUniqueNetId& LocalUserId, const IOnlinePartyJoinInfo& Invitation);
	UE_API void HandlePartyJoinRequestReceived(const FUniqueNetId& LocalUserId, const FOnlinePartyId& PartyId, const IOnlinePartyPendingJoinRequestInfo& JoinRequestInfo);
};

#undef UE_API
