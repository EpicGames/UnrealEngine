// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "Party/PartyTypes.h"
#include "Interfaces/OnlinePartyInterface.h"

#include "SocialTypes.h"
#include "Templates/SubclassOf.h"
#include "SocialManager.generated.h"

#define UE_API PARTY_API

class FSocialInteractionHandle;
class IOnlineSubsystem;
struct FOnlineError;

class ULocalPlayer;
class USocialUser;
class USocialParty;
class USocialToolkit;
class UGameInstance;
class FOnlineSessionSearchResult;
class FPartyPlatformSessionManager;
class USocialDebugTools;

enum ETravelType : int;

#define ABORT_DURING_SHUTDOWN() if (IsEngineExitRequested() || bShutdownPending) { UE_LOGF(LogParty, Log, "%ls - Received callback during shutdown: IsEngineExitRequested=%ls, bShutdownPending=%ls.", ANSI_TO_TCHAR(__FUNCTION__), *LexToString(IsEngineExitRequested()), *LexToString(bShutdownPending)); return; }

/** Singleton manager at the top of the social framework */
UCLASS(MinimalAPI, Within = GameInstance, Config = Game)
class USocialManager : public UObject, public FExec
{
	GENERATED_BODY()

	friend class FPartyPlatformSessionManager;
	friend USocialUser;

public:
	// FExec
#if UE_ALLOW_EXEC_COMMANDS
	UE_API virtual bool Exec(class UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Out) override;
#endif

	static UE_API bool IsSocialSubsystemEnabled(ESocialSubsystem SubsystemType);
	static UE_API FName GetSocialOssName(ESocialSubsystem SubsystemType);
	static UE_API FText GetSocialOssPlatformName(ESocialSubsystem SubsystemType);
	static UE_API IOnlineSubsystem* GetSocialOss(UWorld* World, ESocialSubsystem SubsystemType);
	static UE_API FUserPlatform GetLocalUserPlatform();
	static const TArray<ESocialSubsystem>& GetDefaultSubsystems() { return DefaultSubsystems; }
	static const TArray<FSocialInteractionHandle>& GetRegisteredInteractions() { return RegisteredInteractions; }

	UE_API USocialManager();
	static UE_API void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);

	UE_API static bool ShouldReadPresenceFromOnlineServices();
	UE_API static FName GetPresenceFrameworkInstance();
	UE_API static FName GetPresenceFrameworkAdapterInstance();

	/** Initializes the manager - call this right after creating the manager object during GameInstance initialization. */
	UE_API virtual void InitSocialManager();
	UE_API virtual void ShutdownSocialManager();

	UE_API USocialToolkit& GetSocialToolkit(const ULocalPlayer& LocalPlayer) const;
	UE_API USocialToolkit* GetSocialToolkit(int32 LocalPlayerNum) const;
	UE_API USocialToolkit* GetSocialToolkit(FUniqueNetIdRepl LocalUserId) const;
	UE_API USocialToolkit* GetFirstLocalUserToolkit() const;
	UE_API FUniqueNetIdRepl GetFirstLocalUserId(ESocialSubsystem SubsystemType) const;
	UE_API bool IsLocalUser(const FUniqueNetIdRepl& LocalUserId, ESocialSubsystem SubsystemType) const;
	UE_API int32 GetFirstLocalUserNum() const;
	UE_API USocialDebugTools* GetDebugTools() const;

	DECLARE_EVENT_OneParam(USocialManager, FOnSocialToolkitCreated, USocialToolkit&)
	FOnSocialToolkitCreated& OnSocialToolkitCreated() const { return OnSocialToolkitCreatedEvent; }
	DECLARE_EVENT_OneParam(USocialManager, FOnSocialToolkitDestroyed, USocialToolkit&)
	/** Event triggered when a social toolkit is destroyed. Triggered after it is no longer registered with this social manager. */
	FOnSocialToolkitDestroyed& OnSocialToolkitDestroyed() const { return OnSocialToolkitDestroyedEvent; }
	
	DECLARE_EVENT_OneParam(USocialManager, FOnPartyMembershipChanged, USocialParty&);
	FOnPartyMembershipChanged& OnPartyJoined() const { return OnPartyJoinedEvent; }

	DECLARE_DELEGATE_OneParam(FOnCreatePartyAttemptComplete, ECreatePartyCompletionResult);
	UE_API void CreateParty(const FOnlinePartyTypeId& PartyTypeId, const FPartyConfiguration& PartyConfig, const FOnCreatePartyAttemptComplete& OnCreatePartyComplete);
	UE_API void CreatePersistentParty(const FOnCreatePartyAttemptComplete& OnCreatePartyComplete = FOnCreatePartyAttemptComplete());

	/** Attempt to restore our party state from the party system */
	DECLARE_DELEGATE_OneParam(FOnRestorePartyStateFromPartySystemComplete, bool /*bSucceeded*/)
	UE_API void RestorePartyStateFromPartySystem(const FOnRestorePartyStateFromPartySystemComplete& OnRestoreComplete);

	UE_API bool IsPartyJoinInProgress(const FOnlinePartyTypeId& TypeId) const;
	UE_API bool IsPersistentPartyJoinInProgress() const;

	template <typename PartyT = USocialParty>
	PartyT* GetPersistentParty() const
	{
		return Cast<PartyT>(GetPersistentPartyInternal());
	}

	template <typename PartyT = USocialParty>
	PartyT* GetParty(const FOnlinePartyTypeId& PartyTypeId) const
	{
		return Cast<PartyT>(GetPartyInternal(PartyTypeId));
	}

	template <typename PartyT = USocialParty>
	PartyT* GetParty(const FOnlinePartyId& PartyId) const
	{
		return Cast<PartyT>(GetPartyInternal(PartyId));
	}

	UE_API bool IsConnectedToPartyService() const;

	UE_API void HandlePartyDisconnected(USocialParty* LeavingParty);

	/**
	 * Makes an attempt for the target local player to join the primary local player's party
	 * @param LocalPlayerNum - ControllerId of the Secondary player that wants to join the party
	 * @param Delegate - Delegate run when the join process is finished
	 */
	UE_API void RegisterSecondaryPlayer(int32 LocalPlayerNum, const FOnJoinPartyComplete& Delegate = FOnJoinPartyComplete());

	UE_API virtual void NotifyPartyInitialized(USocialParty& Party);

	UE_DEPRECATED(5.6, "ValidateJoinTarget has been deprecated, use the overloaded one instead")
	UE_API virtual FJoinPartyResult ValidateJoinTarget(const USocialUser& UserToJoin, const FOnlinePartyTypeId& PartyTypeId) const;
	UE_DEPRECATED(5.7, "Use ValidateJoinTarget that takes an options struct")
	UE_API virtual FJoinPartyResult ValidateJoinTarget(const USocialUser& UserToJoin, const FOnlinePartyTypeId& PartyTypeId, bool bCheckPlatformSession) const;
	
	struct FValidateJoinTargetOptions
	{
		/** Should we check the platform session for joinability info if the social user is on the same platform as us? */
		bool bCheckPlatformSession = true;
		/** Is this part of an active join flow? There may be stricter checks when this is false (eg, we don't want to show Join UI options), but if we get past that, we may want to continue with a best effort join attempt. */
		bool bIsActivelyJoining = true;
	};
	/** Validates that the target user has valid join info for us to use and that we can join any party of the given type */
	UE_API virtual FJoinPartyResult ValidateJoinTarget(const USocialUser& UserToJoin, const FOnlinePartyTypeId& PartyTypeId, const FValidateJoinTargetOptions& ValidateOptions) const;
	/** @return true if we can join an away user that has sent us an invitation */
	UE_API bool CanAcceptInvitationsFromAwayUsers() const;

protected:
	DECLARE_DELEGATE_OneParam(FOnJoinPartyAttemptComplete, const FJoinPartyResult&);
	UE_API void JoinParty(const USocialUser& UserToJoin, const FOnlinePartyTypeId& PartyTypeId, const FOnJoinPartyAttemptComplete& OnJoinPartyComplete, const FName& JoinMethod);

public:
	struct FJoinPartyAttempt
	{
		UE_API FJoinPartyAttempt(const USocialUser* InTargetUser, const FOnlinePartyTypeId& InPartyTypeId, const FName& InJoinMethod, const FOnJoinPartyAttemptComplete& InOnJoinComplete);

		UE_API FString ToDebugString() const;

		TWeakObjectPtr<const USocialUser> TargetUser;
		FOnlinePartyTypeId PartyTypeId;
		FName JoinMethod = PartyJoinMethod::Unspecified;
		FUniqueNetIdRepl TargetUserPlatformId;

		IOnlinePartyJoinInfoConstPtr JoinInfo;

		FOnJoinPartyAttemptComplete OnJoinComplete;

		static UE_API const FName Step_FindPlatformSession;
		static UE_API const FName Step_QueryJoinability;
		static UE_API const FName Step_LeaveCurrentParty;
		static UE_API const FName Step_JoinParty;
		static UE_API const FName Step_DeferredPartyCreation;
		static UE_API const FName Step_WaitForPersistentPartyCreation;

		FSocialActionTimeTracker ActionTimeTracker;

		TMap<FString, FString> AnalyticsContext;
	};
protected:

	UE_API virtual void RegisterSocialInteractions();

	/** Validate that we are clear to try joining a party of the given type. If not, gives the reason why. */
	UE_API virtual FJoinPartyResult ValidateJoinAttempt(const FOnlinePartyTypeId& PartyTypeId) const;
	
	/**
	 * Gives child classes a chance to append any additional data to a join request that's about to be sent to another party.
	 * This is where you'll add game-specific information that can affect whether you are eligible for the target party.
	 */
	UE_API virtual void FillOutJoinRequestData(const FOnlinePartyId& TargetParty, FOnlinePartyData& OutJoinRequestData) const;

	UE_API virtual TSubclassOf<USocialParty> GetPartyClassForType(const FOnlinePartyTypeId& PartyTypeId) const;

	//virtual void OnCreatePartyComplete(const TSharedPtr<const FOnlinePartyId>& PartyId, ECreatePartyCompletionResult Result, FOnlinePartyTypeId PartyTypeId) {}
	//virtual void OnQueryJoinabilityComplete(const FOnlinePartyId& PartyId, EJoinPartyCompletionResult Result, int32 DeniedResultCode, FOnlinePartyTypeId PartyTypeId) {}

	UE_API virtual void OnJoinPartyAttemptCompleteInternal(const FJoinPartyAttempt& JoinAttemptInfo, const FJoinPartyResult& Result);
	virtual void OnPartyLeftInternal(USocialParty& LeftParty, EMemberExitedReason Reason) {}
	UE_API virtual void OnToolkitCreatedInternal(USocialToolkit& NewToolkit);

	UE_API virtual bool CanCreateNewPartyObjects() const;

	/** Up to the game to decide whether it wants to allow crossplay (generally based on a user setting of some kind) */
	UE_API virtual ECrossplayPreference GetCrossplayPreference() const;

	template <typename InteractionT>
	void RegisterInteraction()
	{
		RegisteredInteractions.Add(InteractionT::GetHandle());
	}

	UE_API void RefreshCanCreatePartyObjects();

	UE_API USocialParty* GetPersistentPartyInternal(bool bEvenIfLeaving = false) const;

public:
	UE_API const FJoinPartyAttempt* GetJoinAttemptInProgress(const FOnlinePartyTypeId& PartyTypeId) const;

protected:
	UE_API TSharedPtr<const IOnlinePartyJoinInfo> GetJoinInfoFromSession(const FOnlineSessionSearchResult& PlatformSession);
	UE_API void FinishJoinPartyAttempt(FJoinPartyAttempt& JoinAttemptToDestroy, const FJoinPartyResult& JoinResult);

	UE_API virtual TSubclassOf<USocialDebugTools> GetSocialDebugToolsClass() const;

	/** The desired type of SocialToolkit to create for each local player */
	TSubclassOf<USocialToolkit> ToolkitClass;

	// Set during shutdown, used to early-out of lingering OnlineSubsystem callbacks that are pending
	bool bShutdownPending = false;

	TMap<FOnlinePartyTypeId, FJoinPartyAttempt> JoinAttemptsByTypeId;

	UE_API void QueryPartyJoinabilityInternal(FJoinPartyAttempt& JoinAttempt);

	UE_API USocialParty* GetPartyInternal(const FOnlinePartyTypeId& PartyTypeId, bool bIncludeLeavingParties = false) const;

	UE_API virtual void HandleWorldEstablished(UWorld* World);
private:
	UE_API UGameInstance& GetGameInstance() const;
	UE_API USocialToolkit& CreateSocialToolkit(ULocalPlayer& OwningLocalPlayer, int32 LocalPlayerIndex);

	UE_API void JoinPartyInternal(FJoinPartyAttempt& JoinAttempt);
	
	UE_API USocialParty* EstablishNewParty(const FUniqueNetId& LocalUserId, const FOnlinePartyId& PartyId, const FOnlinePartyTypeId& PartyTypeId);

	UE_API USocialParty* GetPartyInternal(const FOnlinePartyId& PartyId, bool bIncludeLeavingParties = false) const;

	UE_API void OnCreatePersistentPartyCompleteInternal(ECreatePartyCompletionResult Result, FOnCreatePartyAttemptComplete OnCreatePartyComplete);
	bool bCreatingPersistentParty = false;

private:	// Handlers
	UE_API void HandleGameViewportInitialized();
	UE_API void HandlePreClientTravel(const FString& PendingURL, ETravelType TravelType, bool bIsSeamlessTravel);
	UE_API void HandleLocalPlayerAdded(int32 LocalUserNum);
	UE_API void HandleLocalPlayerRemoved(int32 LocalUserNum);
	UE_API void HandleToolkitReset(int32 LocalUserNum);
	
	UE_API void OnRestorePartiesComplete(const FUniqueNetId& LocalUserId, const FOnlineError& Result, const FOnRestorePartyStateFromPartySystemComplete OnRestoreComplete);
	UE_API void HandleCreatePartyComplete(const FUniqueNetId& LocalUserId, const TSharedPtr<const FOnlinePartyId>& PartyId, ECreatePartyCompletionResult Result, FOnlinePartyTypeId PartyTypeId, FOnCreatePartyAttemptComplete CompletionDelegate);
	UE_API void HandleJoinPartyComplete(const FUniqueNetId& LocalUserId, const FOnlinePartyId& PartyId, EJoinPartyCompletionResult Result, int32 NotApprovedReasonCode, FOnlinePartyTypeId PartyTypeId);
	
	UE_API void HandlePersistentPartyStateChanged(EPartyState NewState, EPartyState PreviousState, USocialParty* PersistentParty);
	UE_API void HandleLeavePartyForJoinComplete(ELeavePartyCompletionResult LeaveResult, USocialParty* LeftParty);
	UE_API void HandlePartyLeaveBegin(EMemberExitedReason Reason, USocialParty* LeavingParty);
	UE_API void HandlePartyLeft(EMemberExitedReason Reason, USocialParty* LeftParty);

	UE_API void HandleLeavePartyForMissingJoinAttempt(const FUniqueNetId& LocalUserId, const FOnlinePartyId& PartyId, ELeavePartyCompletionResult LeaveResult, FOnlinePartyTypeId PartyTypeId);

	UE_API void HandleFillPartyJoinRequestData(const FUniqueNetId& LocalUserId, const FOnlinePartyId& PartyId, FOnlinePartyData& PartyData);
	UE_API void HandleFindSessionForJoinComplete(bool bWasSuccessful, const FOnlineSessionSearchResult& FoundSession, FOnlinePartyTypeId PartyTypeId);

protected: // overridable handlers
	UE_API virtual void HandleQueryJoinabilityComplete(const FUniqueNetId& LocalUserId, const FOnlinePartyId& PartyId, const FQueryPartyJoinabilityResult& Result, FOnlinePartyTypeId PartyTypeId);

private:
	static UE_API TArray<ESocialSubsystem> DefaultSubsystems;
	static UE_API TArray<FSocialInteractionHandle> RegisteredInteractions;

	UPROPERTY()
	TArray<TObjectPtr<USocialToolkit>> SocialToolkits;

	UPROPERTY()
	TObjectPtr<USocialDebugTools> SocialDebugTools;

	/** Framework instance name to be used in online services presence functionality */
	UPROPERTY(Config)
	FName PresenceFrameworkInstance;

	/** Framework instance name to be used in online services presence functionality where adapter functionality is needed */
	UPROPERTY(Config)
	FName PresenceFrameworkAdapterInstance;

	/** Can we accept an invitation (ie, join party) from an Away user? */
	UPROPERTY(config)
	bool bAcceptInvitationsFromAwayUsers = true;

	bool bIsConnectedToPartyService = false;
	
	/**
	 * False during brief windows where the game isn't in a state conducive to creating a new party object and after the manager is completely shut down (prior to being GC'd)
	 * Tracked to allow OSS level party activity to execute immediately, but hold off on establishing our local (and replicated) awareness of the party until this client is ready.
	 */
	bool bCanCreatePartyObjects = false;

	TSharedPtr<FPartyPlatformSessionManager> PartySessionManager;

	TMap<FOnlinePartyTypeId, TObjectPtr<USocialParty>> JoinedPartiesByTypeId;
	TMap<FOnlinePartyTypeId, TObjectPtr<USocialParty>> LeavingPartiesByTypeId;

	FDelegateHandle OnFillJoinRequestInfoHandle;

	mutable FOnSocialToolkitCreated OnSocialToolkitCreatedEvent;
	mutable FOnSocialToolkitDestroyed OnSocialToolkitDestroyedEvent;
	mutable FOnPartyMembershipChanged OnPartyJoinedEvent;
};

#undef UE_API
