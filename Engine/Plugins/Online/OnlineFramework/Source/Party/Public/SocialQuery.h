// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "SocialToolkit.h"
#include "Misc/ConfigCacheIni.h"

DECLARE_DELEGATE_TwoParams(FOnQueryCompleted, FName, const TSharedRef<class FSocialQueryBase>&);

class FSocialQueryBase : public TSharedFromThis<FSocialQueryBase>
{
public:
	virtual ~FSocialQueryBase() = default;
	virtual void ExecuteQuery() = 0;

	bool HasExecuted() const { return bHasExecuted; }
	ESocialSubsystem GetSubsystemType() const { return SubsystemType; }
	const USocialToolkit* GetOwningToolkit() const { return Toolkit.Get(); }

protected:
	TWeakObjectPtr<const USocialToolkit> Toolkit;
	ESocialSubsystem SubsystemType;
	bool bHasExecuted = false;
	FOnQueryCompleted OnQueryCompleted;
};

template <typename QueryUserIdT, typename... CompletionCallbackArgs>
class TSocialQuery : public FSocialQueryBase
{
public:
	using FQueryId = QueryUserIdT;
	using FOnQueryComplete = TDelegate<void(ESocialSubsystem, bool, CompletionCallbackArgs...)>;

	// All subclasses of TSocialQuery must implement this static method
	// Intentionally not implemented here to catch errors at compile time
	static FName GetQueryId();

	virtual void AddUserId(const QueryUserIdT& UserId, const FOnQueryComplete& QueryCompleteHandler)
	{
		CompletionCallbacksByUserId.Add(UserId, QueryCompleteHandler);
	}

	template <typename OtherQueryT>
	bool operator==(const TSharedRef<OtherQueryT>& OtherQuery) const
	{
		return TIsDerivedFrom<OtherQueryT, TSocialQuery>::IsDerived &&
			OtherQuery->Subsystem == SubsystemType &&
			OtherQuery->Toolkit == Toolkit;
	}

protected:
	friend class FSocialQueryManager;
	TSocialQuery() {}
	void Initialize(const USocialToolkit& InToolkit, ESocialSubsystem InSubsystemType, const FOnQueryCompleted& InOnQueryCompleted)
	{
		Toolkit = &InToolkit;
		SubsystemType = InSubsystemType;
		OnQueryCompleted = InOnQueryCompleted;
	}

	inline IOnlineSubsystem* GetOSS() const
	{
		return Toolkit.IsValid() ? Toolkit->GetSocialOss(SubsystemType) : nullptr;
	}

	static TArray<TSharedRef<TSocialQuery>> CurrentQueries;
	TMap<QueryUserIdT, FOnQueryComplete> CompletionCallbacksByUserId;
};

class FSocialQueryManager
{
public:
	template <typename SocialQueryT>
	static TSharedRef<SocialQueryT> GetQuery(const USocialToolkit& Toolkit, ESocialSubsystem SubsystemType)
	{
		return Get().GetQueryInternal<SocialQueryT>(Toolkit, SubsystemType);
	}

	template <typename SocialQueryT>
	static void AddUserId(const USocialToolkit& Toolkit,
		ESocialSubsystem InSubsystem, 
		const typename SocialQueryT::FQueryId& InQueryId,
		const typename SocialQueryT::FOnQueryComplete& OnQueryCompleteHandler)
	{
		Get().GetQueryInternal<SocialQueryT>(Toolkit, InSubsystem)->AddUserId(InQueryId, OnQueryCompleteHandler);
	}

	bool HandleExecuteQueries(float)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FSocialQueryManager_HandleExecuteQueries);
		// Execute all pending queries
		TArray<TArray<TSharedRef<FSocialQueryBase>>> AllQueries;
		CurrentQueriesById.GenerateValueArray(AllQueries);
		
		for (const TArray<TSharedRef<FSocialQueryBase>>& Queries : AllQueries)
		{
			for (const TSharedRef<FSocialQueryBase>& Query : Queries)
			{
				if (!Query->HasExecuted())
				{
					Query->ExecuteQuery();
				}
			}
		}

		TickExecuteHandle.Reset();

		// Returning false ensures the ticker removes this delegate
		return false;
	}

private:
	FSocialQueryManager() {}

	static FSocialQueryManager& Get()
	{
		static FSocialQueryManager SingletonInstance;
		return SingletonInstance;
	}

	template <typename SocialQueryT>
	TSharedRef<SocialQueryT> GetQueryInternal(const USocialToolkit& Toolkit, ESocialSubsystem SubsystemType)
	{
		const FName QueryId = SocialQueryT::GetQueryId();

		TArray<TSharedRef<FSocialQueryBase>>& Queries = CurrentQueriesById.FindOrAdd(QueryId);
		for (const TSharedRef<FSocialQueryBase>& Query : Queries)
		{
			if (Query->GetSubsystemType() == SubsystemType && Query->GetOwningToolkit() == &Toolkit && !Query->HasExecuted())
			{
				return StaticCastSharedRef<SocialQueryT>(Query);
			}
		}

		// No matching query found, so make a new one
		TSharedRef<SocialQueryT> NewQuery = MakeShareable(new SocialQueryT);
		NewQuery->Initialize(Toolkit, SubsystemType, FOnQueryCompleted::CreateRaw(this, &FSocialQueryManager::HandleQueryComplete));
		Queries.Add(NewQuery);

		float UserInfoQueryAggregationTime = 0.0f;
		
		GConfig->GetFloat(TEXT("Social"), TEXT("UserInfoQueryAggregationTime"), UserInfoQueryAggregationTime, GGameIni);

		// If we aren't already registered to execute our queries next tick, do so now
		if (!TickExecuteHandle.IsValid())
		{
			TickExecuteHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateRaw(this, &FSocialQueryManager::HandleExecuteQueries), UserInfoQueryAggregationTime);
		}

		return NewQuery;
	}

	void HandleQueryComplete(FName QueryId, const TSharedRef<FSocialQueryBase>& Query)
	{
		if (TArray<TSharedRef<FSocialQueryBase>>* Queries = CurrentQueriesById.Find(QueryId))
		{
			Queries->Remove(Query);
		}
	}

	FTSTicker::FDelegateHandle TickExecuteHandle;
	TMap<FName, TArray<TSharedRef<FSocialQueryBase>>> CurrentQueriesById;
};
