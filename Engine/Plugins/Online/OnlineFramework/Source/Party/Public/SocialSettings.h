// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "SocialSettings.generated.h"

#define UE_API PARTY_API

enum class ESocialSubsystem : uint8;

USTRUCT()
struct FSocialPlatformDescription
{
	GENERATED_BODY()

	FSocialPlatformDescription() { };

	/**
	 * The name of this platform
	 * @see IOnlineSubsystem::GetLocalPlatformName
	 */
	UPROPERTY()
	FString Name;

	/** The type of this platform.  For example DESKTOP or MOBILE */
	UPROPERTY()
	FString PlatformType;

	/** The online subsystem this platform uses */
	UPROPERTY()
	FName OnlineSubsystem;

	/** The session type this platform uses */
	UPROPERTY()
	FString SessionType;

	/** The external association type for this platform */
	UPROPERTY()
	FString ExternalAccountType;

	/** The crossplay pool this platform belongs to */
	UPROPERTY()
	FString CrossplayPool;
};

/**
 * Config-driven settings object for the social framework.
 * Only the CDO is ever expected to be used, no instance is ever expected to be created.
 */
UCLASS(MinimalAPI, Config = Game)
class USocialSettings : public UObject
{
	GENERATED_BODY()

public:
	UE_API USocialSettings();

	static UE_API FString GetUniqueIdEnvironmentPrefix(ESocialSubsystem SubsystemType);
	static UE_API int32 GetDefaultMaxPartySize();
	static UE_API bool ShouldPreferPlatformInvites();
	static UE_API bool MustSendPrimaryInvites();
	static UE_API bool MustAlwaysSendPlatformSessionInvites();
	static UE_API bool ShouldLeavePartyOnDisconnect();
	static UE_API bool ShouldSetDesiredPrivacyOnLocalPlayerBecomesLeader();
	static UE_API float GetUserListAutoUpdateRate();
	static UE_API int32 GetMinNicknameLength();
	static UE_API int32 GetMaxNicknameLength();
	static UE_API const TArray<FSocialPlatformDescription>& GetSocialPlatformDescriptions();

	/** Checks if current OnlineSubsystem name is Sony specific */
	static UE_API bool IsSonyOSS(const FName& OSSName);

	/**
	 * Get a platform description (from GetSocialPlatformDescriptions) for a specific OnlineSubsystem.
	 * @param OnlineSubsystemName the online subsystem name to search for
	 * @return the social platform description for that online subsystem. May return null if it is not found.
	 */
	static UE_API const FSocialPlatformDescription* GetSocialPlatformDescriptionForOnlineSubsystem(const FName& OnlineSubsystemName);

private:
	/**
	 * The specific OSS' that have their IDs stored with an additional prefix for the environment to which they pertain.
	 * This is only necessary for OSS' (ex: Switch) that do not have separate environments, just one big pot with both dev and prod users/friendships/etc.
	 * For these cases, the linked account ID stored on the Primary UserInfo for this particular OSS will be prefixed with the specific environment in which the linkage exists.
	 * Additionally, the prefix must be prepended when mapping the external ID to a primary ID.
	 * Overall, it's a major hassle that can hopefully be done away with eventually, but for now is necessary to fake environmental behavior on OSS' without environments.
	 */
	UPROPERTY(config)
	TArray<FName> OssNamesWithEnvironmentIdPrefix;

	/** How many players are in a party by default */
	UPROPERTY(config)
	int32 DefaultMaxPartySize = 4;

	/** If true, prioritize the platform's social system over the publisher's */
	UPROPERTY(config)
	bool bPreferPlatformInvites = true;

	/** If true, always send invites using the publisher's system even if already sent via a platform system */
	UPROPERTY(config)
	bool bMustSendPrimaryInvites = false;
	
	/** Some platforms require a platform invite to be sent along with the in-game invitation, even if the players are not friends in the platform */
	UPROPERTY(config)
	bool bMustAlwaysSendPlatformInvites=true;

	/** Should we leave a party when it enters the disconnected state? */
	UPROPERTY(config)
	bool bLeavePartyOnDisconnect = true;

	/** How often the user list will update in seconds */
	UPROPERTY(config)
	bool bSetDesiredPrivacyOnLocalPlayerBecomesLeader = true;

	UPROPERTY(config)
	float UserListAutoUpdateRate = 0.5f;

	/** Shortest possible player nickname */
	UPROPERTY(Config)
	int32 MinNicknameLength = 3;

	/** Longest possible player nickname */
	UPROPERTY(Config)
	int32 MaxNicknameLength = 16;

	UPROPERTY(Config)
	TArray<FSocialPlatformDescription> SocialPlatformDescriptions;

	UPROPERTY(Config)
	TArray<FName> SonyOSSNames;
};

#undef UE_API
