// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Interfaces/OnlinePartyInterface.h"
#include "Interfaces/OnlinePresenceInterface.h"
#include "Containers/Ticker.h"

#include "Online/Presence.h"
#include "Online/OnlineAsyncOpHandle.h"
#include "PartyModule.h"
#include "Templates/SubclassOf.h"
#include "User/SocialUser.h"
#include "SocialToolkit.generated.h"

#define UE_API PARTY_API

class FSocialUserListConfig;
class IOnlineSubsystem;
class USocialManager;

class ULocalPlayer;
class USocialUser;
class FSocialFriendInvitation;
class FSocialPartyInvitation;
class USocialChatManager;

class FOnlinePartyId;
class FSocialUserList;

enum class EMemberExitedReason : uint8;

namespace FriendInviteFailureReason 
{
	extern PARTY_API const FString InviteFailReason_NotFound;
	extern PARTY_API const FString InviteFailReason_AddingSelfFail;
	extern PARTY_API const FString InviteFailReason_AddingBlockedFail;
	extern PARTY_API const FString InviteFailReason_AlreadyFriends;
}
namespace FriendAcceptFailureReason
{
	extern PARTY_API const FString AcceptFailReason_NotPendingInbound;
}

DECLARE_DELEGATE_OneParam(FUserDependentAction, USocialUser&);

/** Represents the full suite of social functionality available to a given LocalPlayer */
UCLASS(MinimalAPI, Within = SocialManager, Config = Game)
class USocialToolkit : public UObject, public FExec
{
	GENERATED_BODY()

	friend class FPartyPlatformSessionManager;
	friend USocialChatManager;
	friend USocialUser;

public:
	template <typename ToolkitT = USocialToolkit>
	static ToolkitT* GetToolkitForPlayer(const ULocalPlayer* LocalPlayer)
	{
		static_assert(TIsDerivedFrom<ToolkitT, USocialToolkit>::IsDerived, "GetToolkitForPlayer only supports getting USocialToolkit type objects");
		return Cast<ToolkitT>(GetToolkitForPlayerInternal(LocalPlayer));
	}

	UE_API USocialToolkit();

	UE_API virtual void InitializeToolkit(ULocalPlayer& InOwningLocalPlayer);
	UE_API virtual bool IsOwnerLoggedIn() const;
	
	UE_API USocialChatManager& GetChatManager() const;

	UE_API IOnlineSubsystem* GetSocialOss(ESocialSubsystem SubsystemType) const;

	UE_API TSharedRef<ISocialUserList> CreateUserList(const FSocialUserListConfig& ListConfig) const;

	UE_API USocialUser& GetLocalUser() const;

	UE_API FUniqueNetIdRepl GetLocalUserNetId(ESocialSubsystem SubsystemType) const;
	UE_API int32 GetLocalUserNum() const;

	UE_API const FOnlineUserPresence* GetPresenceInfo(ESocialSubsystem SubsystemType) const;
	UE_API const UE::Online::TOnlineResult<UE::Online::FGetCachedPresence> GetPresenceInfoV2(ESocialSubsystem SubsystemType) const;
	UE_API virtual void SetLocalUserOnlineState(EOnlinePresenceState::Type OnlineState);
	UE_API virtual void SetLocalUserOnlineStateV2(UE::Online::EUserPresenceStatus OnlineState);

	UE_DEPRECATED(5.6, "AddLocalUserOnlineProperties is deprecated, please use IOnlinePresence::SetPresence instead.")
	UE_API void AddLocalUserOnlineProperties(FPresenceProperties OnlineProperties);

	UE_API ULocalPlayer* GetOwningLocalPlayerPtr() const;
	UE_API USocialManager& GetSocialManager() const;

	const TArray<USocialUser*>& GetAllUsers() const { return AllUsers; }

	/** Finds a SocialUser given a unique net ID from any OSS */
	UE_API USocialUser* FindUser(const FUniqueNetIdRepl& UserId) const;

	/**
	 * Queues an action to be executed when a USocialUser corresponding to the given UniqueId has been established.
	 * The process of creating and initializing a user varies in complexity depending on the local user's platform and the OSS the given Id corresponds to.
	 *
	 * Up to the caller whether they want the action executed immediately upon creation of the USocialUser instance,
	 *	or after the instance is fully initialized (i.e. has finished discovering all available backend information)
	 *  Default is to execute after initialization and is generally more appropriate.
	 */
	UE_API void QueueUserDependentAction(const FUniqueNetIdRepl& UserId, TFunction<void(USocialUser&)>&& UserActionFunc, bool bExecutePostInit = true);
	UE_API void QueueUserDependentAction(const FUniqueNetIdRepl& SubsystemId, FUserDependentAction UserActionDelegate);
	
	/**
	 * Attempts to send a friend invite to another user based on display name or email.
	 * Only necessary to use this path when you do not have a known USocialUser for this user already.
	 */
	UE_API void TrySendFriendInvite(const FString& DisplayNameOrEmail) const;

	UE_API virtual bool IsFriendshipRestricted(const USocialUser& SocialUser, ESocialSubsystem SubsystemType) const;

	UE_API bool GetAuthAttribute(ESocialSubsystem SubsystemType, const FString& AttributeKey, FString& OutValue) const;

	const FString& GetRecentPlayerNamespaceToQuery() const { return RecentPlayerNamespaceToQuery; }
	
	/** Event triggered when the owning local player's login status changes */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnLoginChanged, bool /*bLoggedIn*/);
	FOnLoginChanged& OnLoginChanged() const { return OnLoginChangedEvent; }

	DECLARE_EVENT_OneParam(USocialToolkit, FPartyInviteEvent, USocialUser&);
	FPartyInviteEvent& OnPartyInviteReceived() const { return OnPartyInviteReceivedEvent; }
	FPartyInviteEvent& OnPartyInviteRemoved() const { return OnPartyInviteRemovedEvent; }

	DECLARE_EVENT_TwoParams(USocialToolkit, FFriendInviteEvent, USocialUser&, ESocialSubsystem);
	FFriendInviteEvent& OnFriendInviteSent() const { return OnFriendInviteSentEvent; }
	FFriendInviteEvent& OnFriendInviteReceived() const { return OnFriendInviteReceivedEvent; }
	
	DECLARE_EVENT_ThreeParams(USocialToolkit, FOnRelationshipEstablished, USocialUser&, ESocialSubsystem, bool /*bIsNewlyEstablished*/);
	FOnRelationshipEstablished& OnFriendshipEstablished() const { return OnFriendshipEstablishedEvent; }
	FOnRelationshipEstablished& OnUserBlocked() const { return OnUserBlockedEvent; }
	FOnRelationshipEstablished& OnRecentPlayerAdded() const { return OnRecentPlayerAddedEvent; }

	DECLARE_EVENT_OneParam(USocialToolkit, FOnKnownUserInitialized, USocialUser&);
	FOnKnownUserInitialized& OnKnownUserInitialized() { return OnKnownUserInitializedEvent; }

	DECLARE_EVENT_OneParam(USocialToolkit, FOnSocialUserInvalidated, const USocialUser&);
	FOnSocialUserInvalidated& OnSocialUserInvalidated() { return OnSocialUserInvalidatedEvent; }

	DECLARE_EVENT(USocialToolkit, FBasicToolkitEvent);
	FBasicToolkitEvent& OnToolkitReset() const { return OnToolkitResetEvent; }

	DECLARE_EVENT_OneParam(USocialToolkit, FOnPartyRequestToJoinSentEvent, const USocialUser&);
	FOnPartyRequestToJoinSentEvent& OnPartyRequestToJoinSent() const { return OnPartyRequestToJoinSentEvent; }

	DECLARE_EVENT_TwoParams(USocialToolkit, FOnPartyRequestToJoinReceivedEvent, USocialUser& /*SocialUser*/, IOnlinePartyRequestToJoinInfoConstRef /*Request*/);
	FOnPartyRequestToJoinReceivedEvent& OnPartyRequestToJoinReceived() const { return OnPartyRequestToJoinReceivedEvent; }

	DECLARE_EVENT_ThreeParams(USocialToolkit, FOnPartyRequestToJoinRemovedEvent, USocialUser& /*SocialUser*/, IOnlinePartyRequestToJoinInfoConstRef /*Request*/, EPartyRequestToJoinRemovedReason /*Reason*/);
	FOnPartyRequestToJoinRemovedEvent& OnPartyRequestToJoinRemoved() const { return OnPartyRequestToJoinRemovedEvent; }

	UE_API virtual void NotifyPartyInviteReceived(USocialUser& SocialUser, const IOnlinePartyJoinInfo& Invite);
	UE_API virtual void NotifyPartyInviteRemoved(USocialUser& SocialUser, const IOnlinePartyJoinInfo& Invite);

	/**
	 * Can we automatically re-create a persistent party, for example on party join failure or on party kick?
	 * Typically you will want to, so the user always has a party, but there may be scenarios where you want to delay
	 * creating a new party until the client is in a better state.
	 * @return true if we can automatically recreate a persistent party
	 */
	virtual bool CanAutoRecreatePersistentParty() const { return IsOwnerLoggedIn(); }

#if WITH_EDITOR || !UE_BUILD_SHIPPING
	bool Debug_IsRandomlyChangingPresence() const { return bDebug_IsRandomlyChangingUserPresence; }
#endif

protected:
	UE_API void NotifySubsystemIdEstablished(USocialUser& SocialUser, ESocialSubsystem SubsystemType, const FUniqueNetIdRepl& SubsystemId);
	TSubclassOf<USocialChatManager> GetChatManagerClass() { return ChatManagerClass; }
	
	UE_API bool TrySendFriendInvite(USocialUser& SocialUser, ESocialSubsystem SubsystemType) const;

	UE_API bool AcceptFriendInvite(const USocialUser& SocialUser, ESocialSubsystem SubsystemType) const;

	UE_API void HandleUserInvalidated(USocialUser& InvalidUser);

#if PARTY_PLATFORM_SESSIONS_PSN
	UE_API void NotifyPSNFriendsListRebuilt();
#endif

protected:
	/** only handles sending a friend request via the friends interface, assumes all checks are handled previously **/
	UE_API virtual bool SendFriendInviteInternal(USocialUser& SocialUser, ESocialSubsystem SubsystemType) const;
	/** only handles accepting a friend request via the friends interface, assumes all checks are handled previously **/
	UE_API virtual bool AcceptFriendInviteInternal(const USocialUser& SocialUser, ESocialSubsystem SubsystemType) const;

	UE_API virtual void OnOwnerLoggedIn();
	UE_API virtual void OnOwnerLoggedOut();
	
	virtual void OnSendFriendInviteComplete(const FUniqueNetId& InvitedUserId, const FString& InviteeName, bool bInviteSuceeded, const FString& InviteError) {}
	virtual void OnAcceptFriendInviteComplete(const FUniqueNetId& InviterUserId, bool bInviteSucceeded, const FString& InviteError) {}
	virtual void OnReadFriendsListComplete(int32 LocalUserNum, bool bWasSuccessful, const FString& ListName, const FString& ErrorStr, ESocialSubsystem SubsystemType) {}
	virtual void OnQueryBlockedPlayersComplete(const FUniqueNetId& UserId, bool bWasSuccessful, const FString& ErrorStr, ESocialSubsystem SubsystemType) {}
	virtual void OnQueryRecentPlayersComplete(const FUniqueNetId& UserId, const FString& Namespace, bool bWasSuccessful, const FString& ErrorStr, ESocialSubsystem SubsystemType) {}
	virtual void OnDeleteFriendComplete(int32 LocalPlayer, bool bWasSuccessful, const FUniqueNetId& FormerFriendId, const FString& ListName, const FString& ErrorStr, ESocialSubsystem SubsystemType) {}
	virtual void OnBlockPlayerComplete(int32 LocalUserNum, bool bWasSuccessful, const FUniqueNetId& BlockedPlayerID, const FString& ListName, const FString& ErrorStr, ESocialSubsystem SubsystemType) {}
	virtual void OnUnblockPlayerComplete(int32 LocalUserNum, bool bWasSuccessful, const FUniqueNetId& UnblockedPlayerID, const FString& ListName, const FString& ErrorStr, ESocialSubsystem SubsystemType) {}
	
protected:
	/** Called when a Friend's presence did change */
	virtual void OnFriendPresenceDidChange(const USocialUser& FriendSocialUser, const TSharedRef<FOnlineUserPresence>& NewPresence, ESocialSubsystem SubsystemType) {}
	virtual void OnFriendPresenceDidChangeV2(const USocialUser& FriendSocialUser, const TSharedRef<const UE::Online::FUserPresence>& NewPresence, ESocialSubsystem SubsystemType) {}

	UE_API void QueryFriendsLists();
	UE_API void QueryBlockedPlayers();
	UE_API void QueryRecentPlayers();

	virtual void OnQueryFriendsListSuccess(ESocialSubsystem SubsystemType, const TArray<TSharedRef<FOnlineFriend>>& FriendsList) {}
	virtual void OnQueryBlockedPlayersSuccess(ESocialSubsystem SubsystemType, const TArray<TSharedRef<FOnlineBlockedPlayer>>& BlockedPlayers) {}
	virtual void OnQueryRecentPlayersSuccess(ESocialSubsystem SubsystemType, const TArray<TSharedRef<FOnlineRecentPlayer>>& FriendsList) {}

	/** handle result of TrySendFriendInviteInternal **/
	UE_API void HandleSendFriendInviteComplete(int32 LocalUserNum, bool bWasSuccessful, const FUniqueNetId& InvitedUserId, const FString& ListName, const FString& ErrorStr, ESocialSubsystem SubsystemType, FString DisplayName);
	/** handle result of AcceptFriendInvite **/
	UE_API void HandleAcceptFriendInviteComplete(int32 LocalUserNum, bool bWasSuccessful, const FUniqueNetId& InviterUserId, const FString& ListName, const FString& ErrorStr);

	/** The type of SocialUser to create to represent known users */
	TSubclassOf<USocialUser> SocialUserClass;

	/** Class to use when creating the ChatManager */
	TSubclassOf<USocialChatManager> ChatManagerClass;

	/** The name of the friends list to query */
	FString FriendListToQuery = TEXT("default");

	/** The name of the recent player namespace to query */
	FString RecentPlayerNamespaceToQuery;

	//@todo DanH: Basic idea here is that the toolkit may not want to query all of these things up-front for scalability reasons, some it may want to wait on until some requirement is met
	//		Something nicer is definitely possible and worth revisiting
	bool bQueryFriendsOnStartup = true;
	bool bQueryBlockedPlayersOnStartup = true;
	bool bQueryRecentPlayersOnStartup = true;

	/** Whether we allow other users to send us invites */
	UE_API virtual const bool IsInviteAllowedFromUser(const USocialUser& User, const TSharedRef<const IOnlinePartyJoinInfo>& InviteRef) const;

private:
	UE_API void QueueUserDependentActionInternal(const FUniqueNetIdRepl& SubsystemId, ESocialSubsystem SubsystemType, TFunction<void(USocialUser&)>&& UserActionFunc, bool bExecutePostInit = true);

	template <typename OnlineUserT>
	void ProcessUserList(const TArray<TSharedRef<OnlineUserT>>& OssUserList, ESocialSubsystem SubsystemType, FOnRelationshipEstablished& RelationshipEvent)
	{
		UE_LOGF(LogParty, Verbose, "SocialToolkit [%d] processing queried list of [%d] users on subsystem [%ls]", GetLocalUserNum(), OssUserList.Num(), ToString(SubsystemType));
		for (const TSharedRef<OnlineUserT>& OssUserInfo : OssUserList)
		{
			QueueUserDependentActionInternal(OssUserInfo->GetUserId(), SubsystemType,
				[WeakUserInfo = TWeakPtr<OnlineUserT>(OssUserInfo), SubsystemType, &RelationshipEvent](USocialUser& SocialUser)
				{
					// Make sure the OSS info is still around - given the async nature of all this it's possible that it's been replaced or removed
					if (WeakUserInfo.IsValid())
					{
						SocialUser.EstablishOssInfo(WeakUserInfo.Pin().ToSharedRef(), SubsystemType);
						RelationshipEvent.Broadcast(SocialUser, SubsystemType, false);
					}
				});
		}
	}

private:	// Handlers
	UE_API void HandleControllerIdChanged(int32 NewId, int32 OldId);
	UE_API void HandlePlayerLoginStatusChanged(int32 LocalUserNum, ELoginStatus::Type OldStatus, ELoginStatus::Type NewStatus, const FUniqueNetId& NewId);
	
	UE_API void HandleReadFriendsListComplete(int32 LocalUserNum, bool bWasSuccessful, const FString& ListName, const FString& ErrorStr, ESocialSubsystem SubsystemType);
	UE_API void HandleQueryBlockedPlayersComplete(const FUniqueNetId& UserId, bool bWasSuccessful, const FString& ErrorStr, ESocialSubsystem SubsystemType);
	UE_API void HandleQueryRecentPlayersComplete(const FUniqueNetId& UserId, const FString& Namespace, bool bWasSuccessful, const FString& ErrorStr, ESocialSubsystem SubsystemType);

	UE_API void HandleRecentPlayersAdded(const FUniqueNetId& LocalUserId, const TArray<TSharedRef<FOnlineRecentPlayer>>& NewRecentPlayers, ESocialSubsystem SubsystemType);

	UE_API void HandleMapExternalIdComplete(ESocialSubsystem SubsystemType, bool bWasSuccessful, const FUniqueNetIdRepl& MappedPrimaryId, FUniqueNetIdRepl ExternalId, TFunction<void(USocialUser&)> UserActionFunc, bool bExecutePostInit);
	UE_API void HandlePresenceReceived(const FUniqueNetId& UserId, const TSharedRef<FOnlineUserPresence>& NewPresence, ESocialSubsystem SubsystemType);
	UE_API void HandleServicesPresenceUpdated(const UE::Online::FPresenceUpdated& Event, ESocialSubsystem SubsystemType);
	void HandlePresenceReadCVarChanged(IConsoleVariable*);

	UE_API void HandleFriendInviteReceived(const FUniqueNetId& LocalUserId, const FUniqueNetId& SenderId, ESocialSubsystem SubsystemType);
	UE_API void HandleFriendInviteAccepted(const FUniqueNetId& LocalUserId, const FUniqueNetId& NewFriendId, ESocialSubsystem SubsystemType);
	UE_API void HandleFriendInviteRejected(const FUniqueNetId& LocalUserId, const FUniqueNetId& RejecterId, ESocialSubsystem SubsystemType);
	UE_API void HandleFriendRemoved(const FUniqueNetId& LocalUserId, const FUniqueNetId& FormerFriendId, ESocialSubsystem SubsystemType);

	UE_API void HandleDeleteFriendComplete(int32 LocalPlayer, bool bWasSuccessful, const FUniqueNetId& FormerFriendId, const FString& ListName, const FString& ErrorStr, ESocialSubsystem SubsystemType);
	UE_API void HandleFriendsChanged(ESocialSubsystem SubsystemType);

	UE_API void HandlePartyInviteReceived(const FUniqueNetId& LocalUserId, const IOnlinePartyJoinInfo& Invite);
	UE_API void HandlePartyInviteRemoved(const FUniqueNetId& LocalUserId, const IOnlinePartyJoinInfo& Invite, EPartyInvitationRemovedReason Reason);
	UE_API void HandlePartyDataReceived(const FUniqueNetId& LocalUserId, const FOnlinePartyId& PartyId, const FName& Namespace, const FOnlinePartyData& PartyData);

#if PARTY_PLATFORM_INVITE_PERMISSIONS
	UE_API void CanReceiveInviteFrom(USocialUser& SocialUser, const IOnlinePartyJoinInfoConstRef& Invite, TFunction<void(const bool /*bResult*/)>&& CompletionFunc);
#endif

	UE_API void HandleBlockPlayerComplete(int32 LocalUserNum, bool bWasSuccessful, const FUniqueNetId& BlockedPlayerID, const FString& ListName, const FString& ErrorStr, ESocialSubsystem SubsystemType);
	UE_API void HandleUnblockPlayerComplete(int32 LocalUserNum, bool bWasSuccessful, const FUniqueNetId& UnblockedPlayerID, const FString& ListName, const FString& ErrorStr, ESocialSubsystem SubsystemType);

	UE_API void HandleBlockListChanged(int32 LocalUserNum, const FString& ListName, ESocialSubsystem SubsystemType);
	
	UE_API void HandleQueryPrimaryUserIdMappingComplete(bool bWasSuccessful, const FUniqueNetId& RequestingUserId, const FString& DisplayName, const FUniqueNetId& IdentifiedUserId, const FString& Error);

	UE_API void HandlePartyMemberExited(const FUniqueNetId& LocalUserId, const FOnlinePartyId& PartyId, const FUniqueNetId& MemberId, const EMemberExitedReason Reason);
	UE_API void HandleGameDestroyed(const FName SessionName, bool bWasSuccessful);

	UE_API void HandleExistingPartyInvites(ESocialSubsystem SubsystemType);

	UE_API void HandlePartyRequestToJoinReceived(const FUniqueNetId& LocalUserId, const FOnlinePartyId& PartyId, const FUniqueNetId& RequesterId, const IOnlinePartyRequestToJoinInfo& Request);
	UE_API void HandlePartyRequestToJoinRemoved(const FUniqueNetId& LocalUserId, const FOnlinePartyId& PartyId, const FUniqueNetId& RequesterId, const IOnlinePartyRequestToJoinInfo& Request, EPartyRequestToJoinRemovedReason Reason);

#if WITH_EDITOR || !UE_BUILD_SHIPPING
	UE_API void Debug_OnStartRandomizeUserPresence(uint8 NumRandomUser, float TickerTimer);
	UE_API void Debug_OnStopRandomizeUserPresence(bool bClearGeneratedPresence);
	UE_API bool Debug_HandleRandomizeUserPresenceTick(float DeltaTime, uint8 NumRandomUser);
	UE_API void Debug_ChangeRandomUserPresence(uint8 NumRandomUser);
	bool bDebug_IsRandomlyChangingUserPresence = false;
	FTSTicker::FDelegateHandle Debug_PresenceTickerHandle;
#endif

	TMap<ESocialSubsystem, UE::Online::FOnlineEventDelegateHandle> OnPresenceUpdatedHandles;
	FDelegateHandle OnReadPresenceV2CVarChanged;

protected:
	UPROPERTY()
	TObjectPtr<USocialChatManager> SocialChatManager;

private:
	static UE_API USocialToolkit* GetToolkitForPlayerInternal(const ULocalPlayer* LocalPlayer);
	static UE_API TMap<TWeakObjectPtr<const ULocalPlayer>, TWeakObjectPtr<USocialToolkit>> AllToolkitsByOwningPlayer;

	UPROPERTY()
	TObjectPtr<USocialUser> LocalUser;

	UPROPERTY()
	TArray<TObjectPtr<USocialUser>> AllUsers;

	void ProcessOSSBlockedPlayers(ESocialSubsystem SubsystemType, const TArray<TSharedRef<FOnlineBlockedPlayer>>& BlockedPlayers);
	TMap<ESocialSubsystem, TSet<FUniqueNetIdRepl>> BlockedUsers;

	TMap<FUniqueNetIdRepl, TWeakObjectPtr<USocialUser>> UsersBySubsystemIds;

	UPROPERTY(config)
	bool bRemoveInvalidatedUserFromMaps = true;

	UPROPERTY()
	TWeakObjectPtr<ULocalPlayer> LocalPlayerOwner = nullptr;

	TSet<UE::Online::FAccountId> NonReadPresencesReceivedV2;
	FUniqueNetIdSet NonReadPresencesReceivedV1;

	TSet<IOnlinePartyJoinInfoConstRef> PartyInvitations;
	mutable TArray<TWeakPtr<FSocialUserList>> CachedSocialUserLists;

	mutable FOnLoginChanged OnLoginChangedEvent;

	mutable FPartyInviteEvent OnPartyInviteReceivedEvent;
	mutable FPartyInviteEvent OnPartyInviteAcceptedEvent;
	mutable FPartyInviteEvent OnPartyInviteRejectedEvent;
	mutable FPartyInviteEvent OnPartyInviteRemovedEvent;

	mutable FFriendInviteEvent OnFriendInviteReceivedEvent;
	mutable FFriendInviteEvent OnFriendInviteSentEvent;
	
	mutable FOnRelationshipEstablished OnFriendshipEstablishedEvent;
	mutable FOnRelationshipEstablished OnUserBlockedEvent;
	mutable FOnRelationshipEstablished OnRecentPlayerAddedEvent;
	
	mutable FOnKnownUserInitialized OnKnownUserInitializedEvent;
	mutable FOnSocialUserInvalidated OnSocialUserInvalidatedEvent;
	mutable FBasicToolkitEvent OnToolkitResetEvent;

	mutable FOnPartyRequestToJoinSentEvent OnPartyRequestToJoinSentEvent;
	mutable FOnPartyRequestToJoinReceivedEvent OnPartyRequestToJoinReceivedEvent;
	mutable FOnPartyRequestToJoinRemovedEvent OnPartyRequestToJoinRemovedEvent;
};

#undef UE_API
