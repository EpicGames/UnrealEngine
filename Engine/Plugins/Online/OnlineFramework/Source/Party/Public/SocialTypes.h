// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SocialSettings.h"

#include "SocialTypes.generated.h"

#define UE_API PARTY_API

class USocialUser;
class USocialToolkit;
class USocialManager;

/** All supported subsystems  */
UENUM()
enum class ESocialSubsystem : uint8
{
	// Publisher-level cross-platform OSS
	Primary,
	// OSS specific to the platform on which we're running (PSN, XBL, GameCenter, etc.)
	Platform,

	// External OSS' that are always available and contain linkable user accounts
	/*Facebook,
	Google,
	Twitch,*/
	MAX
};

inline const TCHAR* LexToString(ESocialSubsystem InSubsystem)
{
	switch(InSubsystem)
	{
	case ESocialSubsystem::Primary:
		return TEXT("Primary");
	case ESocialSubsystem::Platform:
		return TEXT("Platform");
	}

	return TEXT("Unknown");
}

/** Per-OSS relationship types */
UENUM()
enum class ESocialRelationship : uint8
{
	Any,
	FriendInviteReceived,
	FriendInviteSent,
	PartyInvite,
	Friend,
	BlockedPlayer,
	SuggestedFriend,
	RecentPlayer,
	JoinRequest,
	AnyWithLocal,
	// Follower, (?)
};

inline const TCHAR* LexToString(ESocialRelationship Relationship)
{
	switch (Relationship)
	{
	case ESocialRelationship::Any: return TEXT("Any");
	case ESocialRelationship::AnyWithLocal: return TEXT("AnyWithLocal");
	case ESocialRelationship::FriendInviteReceived: return TEXT("FriendInviteReceived");
	case ESocialRelationship::FriendInviteSent: return TEXT("FriendInviteSent");
	case ESocialRelationship::PartyInvite: return TEXT("PartyInvite");
	case ESocialRelationship::Friend: return TEXT("Friend");
	case ESocialRelationship::BlockedPlayer: return TEXT("BlockedPlayer");
	case ESocialRelationship::SuggestedFriend: return TEXT("SuggestedFriend");
	case ESocialRelationship::RecentPlayer: return TEXT("RecentPlayer");
	case ESocialRelationship::JoinRequest: return TEXT("JoinRequest");
	}

	return TEXT("Unknown");
}

UENUM()
enum class ECrossplayPreference : uint8
{
	NoSelection,
	OptedIn,
	OptedOut,
	OptedOutRestricted
};

/** Thin wrapper to infuse a raw platform string with some meaning */
USTRUCT()
struct FUserPlatform
{
	GENERATED_BODY()

public:
	UE_API FUserPlatform();
	UE_API FUserPlatform(const FString& InPlatform);

	operator const FString&() const { return PlatformDescription.Name; }
	const FString& ToString() const { return PlatformDescription.Name; }
	const FString& GetTypeName() const { return PlatformDescription.PlatformType; }
	const FSocialPlatformDescription& GetPlatformDescription() const { return PlatformDescription; }

	UE_API bool operator==(const FString& OtherStr) const;
	bool operator!=(const FString& OtherStr) const { return !operator==(OtherStr); }
	UE_API bool operator==(const FUserPlatform& Other) const;
	bool operator!=(const FUserPlatform& Other) const { return !operator==(Other); }

	UE_API bool IsValid() const;
	UE_API bool IsDesktop() const;
	UE_API bool IsMobile() const;
	UE_API bool IsConsole() const;
	UE_API bool IsCrossplayWith(const FString& OtherPlatformStr) const;
	UE_API bool IsCrossplayWith(const FUserPlatform& OtherPlatform) const;
	UE_API bool IsCrossplayWithLocalPlatform() const;

private:
	UPROPERTY()
	FSocialPlatformDescription PlatformDescription;
};

inline bool OptedOutOfCrossplay(ECrossplayPreference InPreference)
{
	switch (InPreference)
	{
	case ECrossplayPreference::OptedOut:
	case ECrossplayPreference::OptedOutRestricted:
		return true;
	}
	return false;
}

struct FSocialActionTimeTracker
{
	UE_API void BeginStep(FName StepName);
	UE_API void CompleteStep(FName StepName);

	UE_API double GetActionStartTime() const;
	UE_API double GetTotalDurationMs() const;

	UE_API FName GetCurrentStepName() const;
	UE_API double GetStepDurationMs(FName StepName) const;

private:
	struct FSocialActionStep
	{
		FName StepName;
		double StartTime = FPlatformTime::Seconds();
		double EndTime = 0.0;

		double GetDurationMs() const;
		bool operator==(FName OtherStepName) const { return StepName == OtherStepName; }
	};

	TArray<FSocialActionStep> ActionSteps;
};

UENUM()
enum class EPlatformIconDisplayRule : uint8
{
	Always,									// always show the platform icon
	AlwaysIfDifferent,						// always show the icon if it's a different platform from my own
	AlwaysWhenInCrossplayParty,				// always show the icon if I'm in a crossplay party
	AlwaysIfDifferentWhenInCrossplayParty,	// only show the icon if it's different from my own and I'm in a crossplay party
	Never,									// never show the icon
};

#define DECLARE_SHARED_PTR_ALIASES(Type)	\
class Type;	\
using Type##Ptr = TSharedPtr<Type>;	\
using Type##PtrConst = TSharedPtr<const Type>;	\
using Type##Ref = TSharedRef<Type>;	\
using Type##RefConst = TSharedRef<const Type>	

DECLARE_SHARED_PTR_ALIASES(ISocialUserList);
DECLARE_SHARED_PTR_ALIASES(FSocialChatMessage);

inline const TCHAR* ToString(ESocialSubsystem SocialSubsystem)
{
	switch (SocialSubsystem)
	{
	case ESocialSubsystem::Primary: return TEXT("Primary");
	case ESocialSubsystem::Platform: return TEXT("Platform");
	}
	return TEXT("Unknown");
}

inline const TCHAR* LexToString(ECrossplayPreference Preference)
{
	switch (Preference)
	{
	case ECrossplayPreference::NoSelection: return TEXT("NoSelection");
	case ECrossplayPreference::OptedIn: return TEXT("OptedIn");
	case ECrossplayPreference::OptedOut: return TEXT("OptedOut");
	}
	return TEXT("Unknown");
}

#undef UE_API
