// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PartyModule.h"
#include "SocialTypes.h"

class USocialUser;

DECLARE_DELEGATE_RetVal_OneParam(bool, FOnCustomFilterUser, const USocialUser&);

/** OSS-agnostic user state filters (presence info generally required). Some of these do imply others and some conflict. Up to consumers to responsibly choose appropriate combinations. */
enum class ESocialUserStateFlags
{
	None = 0,
	Online = 1, 
	Joinable = 1 << 1,
	LookingForGroup = 1 << 2,
	SamePlatform = 1 << 3,
	InGame = 1 << 4,
	SameApp = 1 << 5,
	SameParty = 1 << 6,
};

ENUM_CLASS_FLAGS(ESocialUserStateFlags);

/** Configuration of ISocialUserList properties that are immutable once the list is created. */
class FSocialUserListConfig
{
public:
	FSocialUserListConfig() {}

	FString Name = TEXT("");
	ESocialRelationship RelationshipType = ESocialRelationship::Friend;
	TArray<ESocialSubsystem> RelevantSubsystems;
	TArray<ESocialSubsystem> ForbiddenSubsystems;
	ESocialUserStateFlags RequiredPresenceFlags = ESocialUserStateFlags::None;
	ESocialUserStateFlags ForbiddenPresenceFlags = ESocialUserStateFlags::None;

	// Delegate that is evaluated every time we test a user for inclusion/exclusion
	FOnCustomFilterUser OnCustomFilterUser;

	// These functions run whenever the User broadcasts OnGameSpecificStatusChanged and if they all return true will trigger a full re-evaluation of this user for list eligibility
	TArray<TFunction<bool(const USocialUser&)>> GameSpecificStatusFilters;

	// Whether or not the list should be polled regularly for updates (as opposed to manually having UpdateNow triggered)
	bool bAutoUpdate = false;

	bool bSortDuringUpdate = true;

	bool operator==(const FSocialUserListConfig& OtherConfig) const
	{
		bool bHasSameConfigs = RelationshipType == OtherConfig.RelationshipType &&
			RequiredPresenceFlags == OtherConfig.RequiredPresenceFlags &&
			ForbiddenPresenceFlags == OtherConfig.ForbiddenPresenceFlags &&
			CompoundSubsystemEnum(RelevantSubsystems) == CompoundSubsystemEnum(OtherConfig.RelevantSubsystems) &&
			CompoundSubsystemEnum(ForbiddenSubsystems) == CompoundSubsystemEnum(OtherConfig.ForbiddenSubsystems);

		if (bHasSameConfigs && (OnCustomFilterUser.IsBound() || OtherConfig.OnCustomFilterUser.IsBound() || GameSpecificStatusFilters.Num() > 0 || OtherConfig.GameSpecificStatusFilters.Num() > 0))
		{
			UE_LOGF(LogParty, Verbose, "Userlist %ls and Userlist %ls has the exact same list config, but one (or both) have custom filters or game specific filters so we are treating them as separate lists.", *Name, *OtherConfig.Name);
			return false;
		}

		return bHasSameConfigs;
	}

private:
	uint8 CompoundSubsystemEnum(const TArray<ESocialSubsystem>& Subsystems ) const
	{
		uint8 OutputValue = 0;
		for (ESocialSubsystem Subsystem : Subsystems)
		{
			OutputValue |= 1 << (uint8)Subsystem;
		}
		return OutputValue;
	}
};

class ISocialUserList
{
public:
	virtual ~ISocialUserList() = default;
	
	DECLARE_EVENT_OneParam(ISocialUserList, FOnUserAdded, USocialUser&)
	virtual FOnUserAdded& OnUserAdded() const = 0;

	DECLARE_EVENT_OneParam(ISocialUserList, FOnUserRemoved, const USocialUser&)
	virtual FOnUserRemoved& OnUserRemoved() const = 0;

	/** Fires one time whenever an update results in some kind of change */
	DECLARE_EVENT(ISocialUserList, FOnUpdateComplete)
	virtual FOnUpdateComplete& OnUpdateComplete() const = 0;

	virtual const TArray<TWeakObjectPtr<USocialUser>>& GetUsers() const = 0;
	virtual FString GetListName() const = 0;

	/** Trigger an update of the list immediately, regardless of auto update period */
	virtual void UpdateNow() = 0;

	/** Give external overwrite to disable list auto update for perf */
	virtual void SetAllowAutoUpdate(bool bIsEnabled) = 0;

	virtual void SetAllowSortDuringUpdate(bool bIsEnabled) = 0;

	virtual void MarkUserAsDirty(USocialUser& User) = 0;
};
