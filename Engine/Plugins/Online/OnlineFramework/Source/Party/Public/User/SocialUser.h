// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/Future.h"
#include "Online/OnlineResult.h"
#include "Online/Presence.h"
#include "Online/UserInfo.h"
#include "OnlineSubsystemTypes.h"
#include "Party/PartyTypes.h"
#include "SocialTypes.h"
#include "Templates/ValueOrError.h"

#include "SocialUser.generated.h"

#define UE_API PARTY_API

class FSocialInteractionHandle;
class USocialToolkit;
class USocialUser;

class IOnlinePartyJoinInfo;
class FOnlineUserPresence;
class UPartyMember;
struct FOnlineError;
class IOnlinePartyRequestToJoinInfo;
enum class EPartyInvitationRemovedReason : uint8;
enum class EPlatformIconDisplayRule : uint8;
enum class ERequestToJoinPartyCompletionResult : int8;
enum class EPartyRequestToJoinRemovedReason : uint8;
typedef TSharedRef<const IOnlinePartyJoinInfo> IOnlinePartyJoinInfoConstRef;
typedef TSharedPtr<const IOnlinePartyJoinInfo> IOnlinePartyJoinInfoConstPtr;

namespace EOnlinePresenceState { enum Type : uint8; }

DECLARE_DELEGATE_OneParam(FOnNewSocialUserInitialized, USocialUser&);

UCLASS(MinimalAPI, Within = SocialToolkit)
class USocialUser : public UObject
{
	GENERATED_BODY()

	friend USocialToolkit;

public:
	UE_API USocialUser();
	UE_API ~USocialUser();

	UE_API void RegisterInitCompleteHandler(const FOnNewSocialUserInitialized& OnInitializationComplete);
	bool IsInitialized() const { return bIsInitialized; }

	UE_API void ValidateFriendInfo(ESocialSubsystem SubsystemType);
	UE_API TArray<ESocialSubsystem> GetRelationshipSubsystems(ESocialRelationship Relationship) const;
	UE_API TArray<ESocialSubsystem> GetRelevantSubsystems() const;
	UE_API bool HasSubsystemInfo(ESocialSubsystem Subsystem) const;
	UE_API bool HasSubsystemInfo(const TSet<ESocialSubsystem>& SubsystemTypes, bool bRequireAll = false);

	UE_API bool IsLocalUser() const;
	UE_API bool HasNetId(const FUniqueNetIdRepl& UniqueId) const;
	UE_API USocialToolkit& GetOwningToolkit() const;
	UE_API EOnlinePresenceState::Type GetOnlineStatus() const;
	UE_API UE::Online::EUserPresenceStatus GetOnlineStatusV2() const;

	UE_API FUniqueNetIdRepl GetUserId(ESocialSubsystem SubsystemType) const;
	UE_API FString GetDisplayName() const;
	UE_API FString GetDisplayName(ESocialSubsystem SubsystemType) const;
	UE_API virtual FString GetNickname() const;
	UE_API virtual bool SetNickname(const FString& InNickName);

	UE_API EInviteStatus::Type GetFriendInviteStatus(ESocialSubsystem SubsystemType) const;
	UE_API bool IsFriend() const;
	UE_API bool IsFriend(ESocialSubsystem SubsystemType) const;
	UE_API bool IsFriendshipPending(ESocialSubsystem SubsystemType) const;
	UE_API bool IsAnyInboundFriendshipPending() const;
	UE_API const FOnlineUserPresence* GetFriendPresenceInfo(ESocialSubsystem SubsystemType) const;
	UE_API UE::Online::TOnlineResult<UE::Online::FGetCachedPresence> GetFriendPresenceInfoV2(ESocialSubsystem SubsystemType) const;
	UE_API FDateTime GetFriendshipCreationDate() const;
	UE_API virtual FDateTime GetLastOnlineDate() const;
	UE_API FText GetSocialName() const;
	UE_API virtual FUserPlatform GetCurrentPlatform() const;

	UE_API FString GetPlatformIconMarkupTag(EPlatformIconDisplayRule DisplayRule) const;
	UE_API virtual FString GetPlatformIconMarkupTag(EPlatformIconDisplayRule DisplayRule, FString& OutLegacyString) const;
	virtual FString GetMarkupTagForPlatform(const FUserPlatform& RemoteUserPlatform) const { return RemoteUserPlatform; }

	UE_API virtual void GetRichPresenceText(FText& OutRichPresence) const;

	UE_API bool IsRecentPlayer() const;
	UE_API bool IsRecentPlayer(ESocialSubsystem SubsystemType) const;
	
	UE_API bool IsBlocked() const;
	UE_API bool IsBlocked(ESocialSubsystem SubsystemType) const;

	UE_API bool IsOnline() const;
	UE_API bool IsPlayingThisGame() const;
	
	virtual bool CanReceiveOfflineInvite() const { return false; }
	virtual int64 GetInteractionScore() const { return 0;  }
	virtual int64 GetCustomSortValuePrimary() const { return 0; }
	virtual int64 GetCustomSortValueSecondary() const { return 0; }
	virtual int64 GetCustomSortValueTertiary() const { return 0; }

	/** Populate list with sort values in order of priority */
	UE_API virtual void PopulateSortParameterList(TArray<int64>& OutSortParams) const;

	UE_API bool SetUserLocalAttribute(ESocialSubsystem SubsystemType, const FString& AttrName, const FString& AttrValue);
	UE_API bool GetUserAttribute(ESocialSubsystem SubsystemType, const FString& AttrName, FString& OutAttrValue) const;

	UE_API virtual bool HasAnyInteractionsAvailable() const;
	UE_API virtual TArray<FSocialInteractionHandle> GetAllAvailableInteractions() const;

	UE_API virtual bool CanSendFriendInvite(ESocialSubsystem SubsystemType) const;
	UE_API virtual bool SendFriendInvite(ESocialSubsystem SubsystemType);
	UE_API virtual bool AcceptFriendInvite(ESocialSubsystem SocialSubsystem) const;
	UE_API virtual bool RejectFriendInvite(ESocialSubsystem SocialSubsystem) const;
	UE_API virtual bool EndFriendship(ESocialSubsystem SocialSubsystem) const;

	TMap<FString, FString> GetAnalyticsContext() const { return AnalyticsContext; }
	UE_API void WithContext(const TMap<FString, FString>& InAnalyticsContext, void(*Func)(USocialUser&));

	UE_API bool ShowPlatformProfile();

	UE_API void HandlePartyInviteReceived(const IOnlinePartyJoinInfo& Invite);
	UE_API void HandlePartyInviteRemoved(const IOnlinePartyJoinInfo& Invite, EPartyInvitationRemovedReason Reason);

	virtual bool CanRequestToJoin() const { return false; }
	virtual bool HasRequestedToJoinUs() const { return false; }
	UE_API void HandleRequestToJoinReceived(const IOnlinePartyRequestToJoinInfo& Request);
	UE_API void HandleRequestToJoinRemoved(const IOnlinePartyRequestToJoinInfo& Request, EPartyRequestToJoinRemovedReason Reason);

	UE_API virtual void RequestToJoinParty(const FName& JoinMethod);
	UE_API void AcceptRequestToJoinParty() const;
	UE_API void DismissRequestToJoinParty() const;

	UE_API virtual void HandlePartyRequestToJoinSent(const FUniqueNetId& LocalUserId, const FUniqueNetId& PartyLeaderId, const FDateTime& ExpiresAt, const ERequestToJoinPartyCompletionResult Result, FName JoinMethod, FString Metadata);

	UE_DEPRECATED(5.6, "Use the overload above that also receives metadata")
	UE_API virtual void HandlePartyRequestToJoinSent(const FUniqueNetId& LocalUserId, const FUniqueNetId& PartyLeaderId, const FDateTime& ExpiresAt, const ERequestToJoinPartyCompletionResult Result, FName JoinMethod);

	UE_API virtual IOnlinePartyJoinInfoConstPtr GetPartyJoinInfo(const FOnlinePartyTypeId& PartyTypeId) const;

	UE_API bool HasSentPartyInvite(const FOnlinePartyTypeId& PartyTypeId) const;
	UE_API FJoinPartyResult CheckPartyJoinability(const FOnlinePartyTypeId& PartyTypeId, bool bCheckPlatformSession = true) const;

	UE_API virtual void JoinParty(const FOnlinePartyTypeId& PartyTypeId, const FName& JoinMethod) const;
	UE_API virtual void RejectPartyInvite(const FOnlinePartyTypeId& PartyTypeId);

	UE_API virtual bool HasBeenInvitedToParty(const FOnlinePartyTypeId& PartyTypeId) const;
	UE_API bool CanInviteToParty(const FOnlinePartyTypeId& PartyTypeId) const;
	UE_API virtual bool InviteToParty(const FOnlinePartyTypeId& PartyTypeId, const ESocialPartyInviteMethod InviteMethod = ESocialPartyInviteMethod::Other, const FString& MetaData = FString()) const;

	UE_API virtual bool BlockUser(ESocialSubsystem Subsystem) const;
	UE_API virtual bool UnblockUser(ESocialSubsystem Subsystem) const;

	UE_API UPartyMember* GetPartyMember(const FOnlinePartyTypeId& PartyTypeId) const;

	DECLARE_EVENT_OneParam(USocialUser, FOnNicknameChanged, const FText&);
	FOnNicknameChanged& OnSetNicknameCompleted() const { return OnSetNicknameCompletedEvent; }

	DECLARE_EVENT(USocialUser, FPartyInviteResponseEvent);
	FPartyInviteResponseEvent& OnPartyInviteAccepted() const { return OnPartyInviteAcceptedEvent; }
	FPartyInviteResponseEvent& OnPartyInviteRejected() const { return OnPartyInviteRejectedEvent; }

	DECLARE_EVENT_OneParam(USocialUser, FOnUserPresenceChanged, ESocialSubsystem)
	FOnUserPresenceChanged& OnUserPresenceChanged() const { return OnUserPresenceChangedEvent; }

	// provided so that lists with custom game-specific filtering (and any other listeners) can potentially re-evaluate a user
	// the pattern here is similar to OnUserPresenceChanged but not subsystem-specific
	DECLARE_EVENT(USocialUser, FOnUserGameSpecificStatusChanged)
	FOnUserGameSpecificStatusChanged& OnUserGameSpecificStatusChanged() const { return OnUserGameSpecificStatusChangedEvent; }

	DECLARE_EVENT_OneParam(USocialUser, FOnFriendRemoved, ESocialSubsystem)
	FOnFriendRemoved& OnFriendRemoved() const { return OnFriendRemovedEvent; }
	FOnFriendRemoved& OnFriendInviteRemoved() const { return OnFriendInviteRemovedEvent; }

	DECLARE_EVENT_TwoParams(USocialUser, FOnBlockedStatusChanged, ESocialSubsystem, bool)
	FOnBlockedStatusChanged& OnBlockedStatusChanged() const { return OnBlockedStatusChangedEvent; }

	DECLARE_EVENT_ThreeParams(USocialUser, FOnSubsystemIdEstablished, USocialUser&, ESocialSubsystem, const FUniqueNetIdRepl&)
	FOnSubsystemIdEstablished& OnSubsystemIdEstablished() const { return OnSubsystemIdEstablishedEvent; }

	//void ClearPopulateInfoDelegateForSubsystem(ESocialSubsystem SubsystemType);

	UE_API FString ToDebugString() const;

	UE_API void EstablishOssInfo(const TSharedRef<FOnlineFriend>& FriendInfo, ESocialSubsystem SubsystemType);
	UE_API void EstablishOssInfo(const TSharedRef<FOnlineBlockedPlayer>& BlockedPlayerInfo, ESocialSubsystem SubsystemType);
	UE_API void EstablishOssInfo(const TSharedRef<FOnlineRecentPlayer>& RecentPlayerInfo, ESocialSubsystem SubsystemType);

protected:
	UE_API void InitLocalUser();
	UE_API void Initialize(const FUniqueNetIdRepl& PrimaryId);

	UE_API void NotifyPresenceChanged(ESocialSubsystem SubsystemType);
	UE_API void NotifyUserUnblocked(ESocialSubsystem SubsystemType);
	UE_API void NotifyFriendInviteRemoved(ESocialSubsystem SubsystemType);
	UE_API void NotifyUserUnfriended(ESocialSubsystem SubsystemType);

#if WITH_EDITOR || !UE_BUILD_SHIPPING
	UE_API void Debug_RandomizePresence();
	bool bDebug_IsPresenceArtificial = false;
	EOnlinePresenceState::Type Debug_RandomPresence;
	UE::Online::EUserPresenceStatus Debug_RandomPresenceV2;
#endif

protected:
	UE_API virtual void OnPresenceChangedInternalV2(ESocialSubsystem SubsystemType);
	UE_API virtual void OnPresenceChangedInternal(ESocialSubsystem SubsystemType);
	UE_API virtual void OnPartyInviteAcceptedInternal(const FOnlinePartyTypeId& PartyTypeId, const IOnlinePartyJoinInfo& Invite) const;
	UE_API virtual void OnPartyInviteRejectedInternal(const FOnlinePartyTypeId& PartyTypeId) const;
	UE_API virtual void HandleSetNicknameComplete(int32 LocalUserNum, const FUniqueNetId& FriendId, const FString& ListName, const FOnlineError& Error);
	UE_API virtual void SetSubsystemId(ESocialSubsystem SubsystemType, const FUniqueNetIdRepl& SubsystemId);

	UE_DEPRECATED(5.6, "Override 'void OnPartyInviteAcceptedInternal(const FOnlinePartyTypeId& PartyTypeId, const FString& InviteMetadata) const' instead")
	virtual void OnPartyInviteAcceptedInternal(const FOnlinePartyTypeId& PartyTypeId) const {};

	UE_API virtual FString GetRequestToJoinMetadata(const FString& ExistingMetadata);
	virtual void NotifyRequestToJoinReceived(const IOnlinePartyRequestToJoinInfo& Request) {}
	virtual void NotifyRequestToJoinRemoved(const IOnlinePartyRequestToJoinInfo& Request, EPartyRequestToJoinRemovedReason Reason) {}
	int32 NumPendingQueries = 0;
	TMap<FString, FString> AnalyticsContext;

	UE_API IOnlinePartyJoinInfoConstPtr GetSentPartyInvite(const FOnlinePartyTypeId& PartyTypeId) const;
	UE_API void TryBroadcastInitializationComplete();

	struct FSubsystemUserInfo
	{
		FSubsystemUserInfo(const FUniqueNetIdRepl& InUserId)
			: UserId(InUserId)
		{}

		bool IsValid() const;
		const FUniqueNetIdRepl& GetUserId() const { return UserId; }
		FString GetDisplayName() const { return UserInfo.IsValid() ? UserInfo.Pin()->GetDisplayName() : TEXT(""); }
		bool IsFriend() const { return GetFriendInviteStatus() == EInviteStatus::Accepted; }
		bool IsBlocked() const { return BlockedPlayerInfo.IsValid() || GetFriendInviteStatus() == EInviteStatus::Blocked; }
		EInviteStatus::Type GetFriendInviteStatus() const { return FriendInfo.IsValid() ? FriendInfo.Pin()->GetInviteStatus() : EInviteStatus::Unknown; }
		bool HasValidPresenceInfo() const { return IsFriend(); }
		const FOnlineUserPresence* GetPresenceInfo() const;

		// On the fence about caching this locally. We don't care about where it came from if we do, and we can cache it independent from any of the info structs (which will play nice with external mapping queries before querying the user info itself)
		FUniqueNetIdRepl UserId;

		TWeakPtr<FOnlineUser> UserInfo;
		TWeakPtr<FOnlineFriend> FriendInfo;
		TWeakPtr<FOnlineRecentPlayer> RecentPlayerInfo;
		TWeakPtr<FOnlineBlockedPlayer> BlockedPlayerInfo;
	};
	const FSubsystemUserInfo* GetSubsystemUserInfo(ESocialSubsystem Subsystem) const { return SubsystemInfoByType.Find(Subsystem); }

	UE_API void SetUserInfo(ESocialSubsystem SubsystemType, const TSharedRef<FOnlineUser>& UserInfo);

	// This method should be extended in derived classes to asynchronously set up the correct CommonAccount state for the given social user.
	UE_API virtual TFuture<void> SetupCommonAccount();
	// This method should be extended in derived classes to make any additional subsystem user info checks needed that might modify the services provider used in GetDisplayName
	UE_API virtual bool ShouldSpecifyDisplayNameServicesProvider(const FSubsystemUserInfo* SubsystemInfo) const;
	UE_API virtual UE::Online::EOnlinePlatformType GetDisplayNameOnlinePlatformTypeFromUserPlatform(const FString& UserCurrentPlatformStr) const;

	UE_API bool ShouldReadUserInfoFromOnlineServices() const;

private:
	UE_API void HandleQueryUserInfoComplete(ESocialSubsystem SubsystemType, bool bWasSuccessful, const TSharedPtr<FOnlineUser>& UserInfo);
	UE_API void HandleQueryUserInfoV2Complete(ESocialSubsystem SubsystemType, bool bWasSuccessful, const TValueOrError<TSharedRef<UE::Online::FUserInfo>, UE::Online::FOnlineError>& Result);

	UE_API virtual FString SanitizePresenceString(FString InString) const;

private:

	UE_API FSubsystemUserInfo& FindOrCreateSubsystemInfo(const FUniqueNetIdRepl& SubsystemId, ESocialSubsystem SubsystemType);

	bool bIsInitialized = false;
	UE_API void FinishInitialization();

	UE_API void OnJoinPartyComplete(const FJoinPartyResult& JoinPartyResult, FOnlinePartyTypeId PartyTypeId, IOnlinePartyJoinInfoConstPtr SentInvite) const;

#if !UE_BUILD_SHIPPING
	// Debug info for initializing social user
	class FDebugInitializer;
	friend class FDebugInitializer;
	TUniquePtr<FDebugInitializer> DebugInitializer;
#endif // !UE_BUILD_SHIPPING

	TMap<ESocialSubsystem, FSubsystemUserInfo> SubsystemInfoByType;

	TArray<IOnlinePartyJoinInfoConstRef> ReceivedPartyInvites;

	FDelegateHandle OnReadUserInfoV2CVarChanged;

	// Initialization delegates that fire only when a specific user has finishing initializing
	TArray<FOnNewSocialUserInitialized> UserInitializedEvents;

	mutable FOnNicknameChanged OnSetNicknameCompletedEvent;
	mutable FPartyInviteResponseEvent OnPartyInviteAcceptedEvent;
	mutable FPartyInviteResponseEvent OnPartyInviteRejectedEvent;
	mutable FOnUserPresenceChanged OnUserPresenceChangedEvent;
	mutable FOnFriendRemoved OnFriendRemovedEvent;
	mutable FOnFriendRemoved OnFriendInviteRemovedEvent;
	mutable FOnBlockedStatusChanged OnBlockedStatusChangedEvent;
	mutable FOnSubsystemIdEstablished OnSubsystemIdEstablishedEvent;
	mutable FOnUserGameSpecificStatusChanged OnUserGameSpecificStatusChangedEvent;
};

#undef UE_API
