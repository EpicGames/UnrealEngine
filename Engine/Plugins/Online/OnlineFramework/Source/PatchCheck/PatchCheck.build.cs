// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using UnrealBuildTool;

public class PatchCheck : ModuleRules
{
	public PatchCheck(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"OnlineSubsystem",
			}
		);

		PublicIncludePathModuleNames.AddRange(
			new string[] {
				"OnlineSubsystem",
			}
		);

		PublicDefinitions.Add("PATCH_CHECK_PLATFORM_ENVIRONMENT_DETECTION=" + (bPlatformEnvironmentDetection ? "1" : "0"));
		PrivateDefinitions.Add("PATCH_CHECK_PRIVILEGE_MUST_BE_LOGGED_IN=" + (bCanCheckPlayOnlinePrivilegeMustBeLoggedIn ? "1" : "0"));
	}

	[Obsolete("bFailOnGenericFailure deprecated since UE 5.6 - Patch checks always fail open when there are service issues.")]
	protected virtual bool bFailOnGenericFailure { get { return true; } }
	protected virtual bool bPlatformEnvironmentDetection { get { return false; } }
	protected virtual bool bCanCheckPlayOnlinePrivilegeMustBeLoggedIn { get { return true; } }
}
