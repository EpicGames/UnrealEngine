// Copyright Epic Games, Inc. All Rights Reserved.

#include "PatchCheck.h"
#include "PatchCheckModule.h"

#include "HAL/IConsoleManager.h"
#include "Misc/App.h"
#include "Online.h"

DEFINE_LOG_CATEGORY(LogPatchCheck);

static TAutoConsoleVariable<bool> CVarPatchCheckFailOpenOnError(
	TEXT("PatchCheck.FailOpenOnError"),
	true,
	TEXT("Whether a service failure when running a patch check should fail the operation."));

const TCHAR* LexToString(EPatchCheckResult Value)
{
	switch (Value)
	{
	case EPatchCheckResult::NoPatchRequired:	return TEXT("NoPatchRequired");
	case EPatchCheckResult::PatchRequired:		return TEXT("PatchRequired");
	case EPatchCheckResult::NoLoggedInUser:		return TEXT("NoLoggedInUser");
	default:									checkNoEntry(); // Intentional fallthrough
	case EPatchCheckResult::PatchCheckFailure:	return TEXT("PatchCheckFailure");
	}
}

FPatchCheck& FPatchCheck::Get()
{
	static IPatchCheckModule* ConfiguredModule = nullptr;

	if (ConfiguredModule && ConfiguredModule->GetPatchCheck())
	{
		return *ConfiguredModule->GetPatchCheck();
	}

	FString ModuleName;
	if (GConfig->GetString(TEXT("PatchCheck"), TEXT("ModuleName"), ModuleName, GEngineIni))
	{
		if (FModuleManager::Get().ModuleExists(*ModuleName))
		{
			ConfiguredModule = FModuleManager::LoadModulePtr<IPatchCheckModule>(*ModuleName);
		}

		if (ConfiguredModule && ConfiguredModule->GetPatchCheck())
		{
			return *ConfiguredModule->GetPatchCheck();
		}
		else
		{
			// Couldn't find the configured module, fallback to the default
			ensureMsgf(false, TEXT("FPatchCheck: Couldn't find module with Name %s, using default"), ModuleName.IsEmpty() ? TEXT("None") : *ModuleName);
		}
	}

	// No override module configured, use default
	ConfiguredModule = &FModuleManager::LoadModuleChecked<IPatchCheckModule>(TEXT("PatchCheck"));
	return *ConfiguredModule->MakePatchCheck();
}

FPatchCheck::~FPatchCheck()
{

}

FPatchCheck::FPatchCheck()
{
	RefreshConfig();
}

void FPatchCheck::RefreshConfig()
{
	if (!GConfig->GetBool(TEXT("PatchCheck"), TEXT("bCheckPlatformOSSForUpdate"), bCheckPlatformOSSForUpdate, GEngineIni))
	{
		/** For backwards compatibility with UUpdateManager */
		if (GConfig->GetBool(TEXT("/Script/Hotfix.UpdateManager"), TEXT("bCheckPlatformOSSForUpdate"), bCheckPlatformOSSForUpdate, GEngineIni))
		{
			ensureMsgf(false, TEXT("UpdateManager::bCheckPlatformOSSForUpdate is deprecated, Set FPatchCheck::bCheckPlatformOSSForUpdate using section [PatchCheck] instead."));
		}
	}

	if (!GConfig->GetBool(TEXT("PatchCheck"), TEXT("bCheckOSSForUpdate"), bCheckOSSForUpdate, GEngineIni))
	{
		/** For backwards compatibility with UUpdateManager */
		if (GConfig->GetBool(TEXT("/Script/Hotfix.UpdateManager"), TEXT("bCheckOSSForUpdate"), bCheckOSSForUpdate, GEngineIni))
		{
			ensureMsgf(false, TEXT("UpdateManager::bCheckOSSForUpdate is deprecated, Set FPatchCheck::bCheckOSSForUpdate using section [PatchCheck] instead."));
		}
	}

	GConfig->GetBool(TEXT("PatchCheck"), TEXT("bPlatformEnvironmentDetectionEnabled"), bPlatformEnvironmentDetectionEnabled, GEngineIni);

	UE_LOGF(LogPatchCheck, VeryVerbose, "[%s] [bCheckPlatformOSSForUpdate=%ls], [bCheckOSSForUpdate=%ls], [bPlatformEnvironmentDetectionEnabled=%ls]",
		__FUNCTION__, *LexToString(bCheckPlatformOSSForUpdate), *LexToString(bCheckOSSForUpdate), *LexToString(bPlatformEnvironmentDetectionEnabled));
}

void FPatchCheck::OnDetectPlatformEnvironmentComplete(const FOnlineError& Result)
{
	UE_LOGF(LogPatchCheck, VeryVerbose, "[%s] [Result=%ls]", __FUNCTION__, *Result.ToLogString());
	if (TSharedPtr<IPatchCheckStatsCollector> StrongStats = Stats.Pin())
	{
		StrongStats->OnPatchCheckStep_DetectEnvironmentComplete(Result.WasSuccessful(), Result.ToLogString());
	}

	if (Result.WasSuccessful())
	{
		bPlatformEnvironmentDetected = true;
		HandleOSSPatchCheck();
	}
	else
	{
		if (Result.GetErrorCode().Contains(TEXT("getUserAccessCode failed : 0x8055000f"), ESearchCase::IgnoreCase))
		{
			UE_LOGF(LogPatchCheck, Warning, "[%s] Failed to complete login because patch is required", __FUNCTION__);
			PatchCheckComplete(EPatchCheckResult::PatchRequired);
		}
		else
		{
			if (Result.GetErrorCode().Contains(TEXT("com.epicgames.identity.notloggedin"), ESearchCase::IgnoreCase))
			{
				UE_LOGF(LogPatchCheck, Warning, "[%s] Failed to detect online environment for the platform, no user signed in", __FUNCTION__);
				PatchCheckComplete(EPatchCheckResult::NoLoggedInUser);
			}
			else
			{
				// just a platform env error, assume production and keep going
				UE_LOGF(LogPatchCheck, Warning, "[%s] Failed to detect online environment for the platform", __FUNCTION__);
				bPlatformEnvironmentDetected = true;
				HandleOSSPatchCheck();
			}
		}
	}
}

void FPatchCheck::StartPatchCheck()
{
	UE_LOGF(LogPatchCheck, VeryVerbose, "[%s] [bIsCheckInProgress=%ls], [bPlatformEnvironmentDetectionEnabled=%ls], [bPlatformEnvironmentDetected=%ls]",
		__FUNCTION__, *LexToString(bIsCheckInProgress), *LexToString(bPlatformEnvironmentDetectionEnabled), *LexToString(bPlatformEnvironmentDetected));
	if (bIsCheckInProgress)
		return;

	RefreshConfig();

	TSharedPtr<IPatchCheckStatsCollector> StrongStats = Stats.Pin();
	if (StrongStats)
	{
		StrongStats->OnPatchCheckStarted();
	}

	if (bPlatformEnvironmentDetectionEnabled && !bPlatformEnvironmentDetected)
	{
		if (StrongStats)
		{
			StrongStats->OnPatchCheckStep_DetectEnvironmentStarted();
		}

#if PATCH_CHECK_PLATFORM_ENVIRONMENT_DETECTION
		if (DetectPlatformEnvironment())
		{
			return;
		}
#endif
	}

	HandleOSSPatchCheck();
}

void FPatchCheck::HandleOSSPatchCheck()
{
	UE_LOGF(LogPatchCheck, VeryVerbose, "[%s]", __FUNCTION__);
	if (bCheckPlatformOSSForUpdate && IOnlineSubsystem::GetByPlatform() != nullptr)
	{
		bIsCheckInProgress = true;
		StartPlatformOSSPatchCheck();
	}
	else if (bCheckOSSForUpdate)
	{
		bIsCheckInProgress = true;
		StartOSSPatchCheck();
	}
	else
	{
		UE_LOGF(LogPatchCheck, Warning, "Patch check disabled for both Platform and Default OSS");
	}
}

void FPatchCheck::AddEnvironmentWantsPatchCheckBackCompatDelegate(FName Tag, FEnvironmentWantsPatchCheck Delegate)
{
	BackCompatEnvironmentWantsPatchCheckDelegates.Emplace(Tag, MoveTemp(Delegate));
}

void FPatchCheck::RemoveEnvironmentWantsPatchCheckBackCompatDelegate(FName Tag)
{
	BackCompatEnvironmentWantsPatchCheckDelegates.Remove(Tag);
}

void FPatchCheck::RegisterStatsCollector(const TSharedPtr<IPatchCheckStatsCollector>& InStats)
{
	Stats = InStats;
}

void FPatchCheck::StartPlatformOSSPatchCheck()
{
	UE_LOGF(LogPatchCheck, VeryVerbose, "[%s]", __FUNCTION__);
	if (TSharedPtr<IPatchCheckStatsCollector> StrongStats = Stats.Pin())
	{
		StrongStats->OnPatchCheckStep_CheckPlatformPatchStarted();
	}

	EPatchCheckResult PatchResult = EPatchCheckResult::PatchCheckFailure;
	bool bStarted = false;

	IOnlineSubsystem* PlatformOnlineSub = IOnlineSubsystem::GetByPlatform();
	check(PlatformOnlineSub);
	IOnlineIdentityPtr PlatformOnlineIdentity = PlatformOnlineSub->GetIdentityInterface();
	if (PlatformOnlineIdentity.IsValid())
	{
		FUniqueNetIdPtr UserId = GetFirstSignedInUser(PlatformOnlineIdentity);
#if !PATCH_CHECK_PRIVILEGE_MUST_BE_LOGGED_IN
		// some platforms will log the user in if required in all but the NotLoggedIn state
		const bool bCanCheckPlayOnlinePrivilege = UserId.IsValid() && (PlatformOnlineIdentity->GetLoginStatus(*UserId) != ELoginStatus::NotLoggedIn);
#else
		const bool bCanCheckPlayOnlinePrivilege = UserId.IsValid() && (PlatformOnlineIdentity->GetLoginStatus(*UserId) == ELoginStatus::LoggedIn);
#endif
		if (bCanCheckPlayOnlinePrivilege)
		{
			bStarted = true;
			PlatformOnlineIdentity->GetUserPrivilege(*UserId,
				EUserPrivileges::CanPlayOnline, IOnlineIdentity::FOnGetUserPrivilegeCompleteDelegate::CreateRaw(this, &FPatchCheck::OnCheckForPatchComplete, true));
		}
		else
		{
			UE_LOGF(LogPatchCheck, Warning, "No valid platform user id when starting patch check!");
			PatchResult = EPatchCheckResult::NoLoggedInUser;
		}
	}

	if (!bStarted)
	{
		// Any failure to call GetUserPrivilege will result in completing the flow via this path
		PatchCheckComplete(PatchResult);
	}
}

void FPatchCheck::StartOSSPatchCheck()
{
	const bool bSkipPatchCheck = SkipPatchCheck();
	UE_LOGF(LogPatchCheck, Verbose, "[%s] [bSkipPatchCheck=%ls]", __FUNCTION__, *LexToString(bSkipPatchCheck));
	if (TSharedPtr<IPatchCheckStatsCollector> StrongStats = Stats.Pin())
	{
		StrongStats->OnPatchCheckStep_CheckOnlineServicePatchStarted();
	}

	if (bSkipPatchCheck)
	{
		// Trigger completion if check is skipped.
		PatchCheckComplete(EPatchCheckResult::NoPatchRequired);
	}
	else
	{
		EPatchCheckResult PatchResult = EPatchCheckResult::PatchCheckFailure;
		bool bStarted = false;

		// Online::GetIdentityInterface() can take a UWorld for correctness, but that only matters in PIE right now
		// and update checks should never happen in PIE currently.
		IOnlineIdentityPtr IdentityInt = Online::GetIdentityInterface();
		if (IdentityInt.IsValid())
		{
			// User could be invalid for "before title/login" check, underlying code doesn't need a valid user currently
			FUniqueNetIdPtr UserId = IdentityInt->CreateUniquePlayerId(TEXT("InvalidUser"));
			if (UserId.IsValid())
			{
				bStarted = true;
				IdentityInt->GetUserPrivilege(*UserId,
					EUserPrivileges::CanPlayOnline, IOnlineIdentity::FOnGetUserPrivilegeCompleteDelegate::CreateRaw(this, &FPatchCheck::OnCheckForPatchComplete, false));
			}
		}

		if (!bStarted)
		{
			// Any failure to call GetUserPrivilege will result in completing the flow via this path
			PatchCheckComplete(PatchResult);
		}
	}
}

bool FPatchCheck::EnvironmentWantsPatchCheck() const
{
	if (BackCompatEnvironmentWantsPatchCheckDelegates.Num() > 0)
	{
		for (const TPair<FName, FEnvironmentWantsPatchCheck>& Pair : BackCompatEnvironmentWantsPatchCheckDelegates)
		{
			if (Pair.Value.IsBound() && Pair.Value.Execute())
			{
				return true;
			}
		}
	}

	return false;
}

bool FPatchCheck::EditorWantsPatchCheck() const
{
	return false;
}

bool FPatchCheck::SkipPatchCheck() const
{
	// Does the environment care about patch checks (LIVE, STAGE, etc)
	const bool bEnvironmentWantsPatchCheck = EnvironmentWantsPatchCheck();

	// Can always opt in to a check
	const bool bForcePatchCheck = FParse::Param(FCommandLine::Get(), TEXT("ForcePatchCheck"));

	// Check whether editor needs a patch check
	const bool bEditorWantsPatchCheck = EditorWantsPatchCheck();
	const bool bSkipDueToEditor = UE_EDITOR && !bEditorWantsPatchCheck;

	// Prevent a patch check on dedicated server. UpdateManager also doesn't do a patch check on dedicated server.
	const bool bSkipDueToDedicatedServer = IsRunningDedicatedServer();

	// prevent a check when running unattended
	const bool bSkipDueToUnattended = FApp::IsUnattended();

	// Explicitly skipping the check
	const bool bForceSkipCheck = FParse::Param(FCommandLine::Get(), TEXT("SkipPatchCheck"));
	const bool bSkipPatchCheck = !bForcePatchCheck && (!bEnvironmentWantsPatchCheck || bSkipDueToEditor || bSkipDueToDedicatedServer || bForceSkipCheck || bSkipDueToUnattended);

	UE_LOGF(LogPatchCheck, VeryVerbose, "[%s] [bSkipPatchCheck=%ls], [bForcePatchCheck=%ls], [bEnvironmentWantsPatchCheck=%ls], [bSkipDueToEditor=%ls], [bSkipDueToDedicatedServer=%ls], [bForceSkipCheck=%ls], [bSkipDueToUnattended=%ls]",
		__FUNCTION__, *LexToString(bSkipPatchCheck), *LexToString(bForcePatchCheck), *LexToString(bEnvironmentWantsPatchCheck), *LexToString(bSkipDueToEditor), *LexToString(bSkipDueToDedicatedServer), *LexToString(bForceSkipCheck), *LexToString(bSkipDueToUnattended));

	return bSkipPatchCheck;
}

inline EPatchCheckResult TranslatePatchCheckResult(uint32 PrivilegeResult)
{
	EPatchCheckResult Result = EPatchCheckResult::NoPatchRequired;

	if (PrivilegeResult & (uint32)IOnlineIdentity::EPrivilegeResults::RequiredSystemUpdate)
	{
		Result = EPatchCheckResult::PatchRequired;
	}
	else if (PrivilegeResult & (uint32)IOnlineIdentity::EPrivilegeResults::RequiredPatchAvailable)
	{
		Result = EPatchCheckResult::PatchRequired;
	}
	else if (PrivilegeResult & ((uint32)IOnlineIdentity::EPrivilegeResults::UserNotLoggedIn | (uint32)IOnlineIdentity::EPrivilegeResults::UserNotFound))
	{
		Result = EPatchCheckResult::NoLoggedInUser;
	}
	else if (PrivilegeResult & (uint32)IOnlineIdentity::EPrivilegeResults::GenericFailure)
	{
		Result = EPatchCheckResult::PatchCheckFailure;
	}

	return Result;
}

void FPatchCheck::OnCheckForPatchComplete(const FUniqueNetId& UniqueId, EUserPrivileges::Type Privilege, uint32 PrivilegeResult, bool bConsoleCheck)
{
	EPatchCheckResult Result = Privilege == EUserPrivileges::CanPlayOnline ? TranslatePatchCheckResult(PrivilegeResult) : EPatchCheckResult::NoPatchRequired;

#if !UE_BUILD_SHIPPING
	// Set failure if requested.
	static bool bPatchCheckMockFailure = FParse::Param(FCommandLine::Get(), TEXT("PatchCheckMockFailure"));
	if (bPatchCheckMockFailure)
	{
		UE_LOGF(LogPatchCheck, Log, "[%s] Simulating patch check failure.", __FUNCTION__);
		Result = EPatchCheckResult::PatchCheckFailure;
	}

	// Set patch required if requested.
	static bool bPatchCheckMockPatchRequired = FParse::Param(FCommandLine::Get(), TEXT("PatchCheckMockPatchRequired"));
	if (bPatchCheckMockPatchRequired)
	{
		UE_LOGF(LogPatchCheck, Log, "[%s] Simulating required patch available.", __FUNCTION__);
		Result = EPatchCheckResult::PatchRequired;
	}
#endif

	UE_LOGF(LogPatchCheck, Verbose, "[%s] [Type=%ls], [Privilege=%ls], [PrivilegeResult=%ls], [PrivilegeResultValue=%d], [PatchCheckResult=%ls]",
		__FUNCTION__, bConsoleCheck ? TEXT("PlatformOSS") : TEXT("DefaultOSS"), *ToDebugString(Privilege), *ToDebugString(static_cast<IOnlineIdentity::EPrivilegeResults>(PrivilegeResult)), PrivilegeResult, LexToString(Result));

	// Publish stats.
	if (TSharedPtr<IPatchCheckStatsCollector> StrongStats = Stats.Pin())
	{
		const bool bSucceeded = Result == EPatchCheckResult::NoPatchRequired;
		if (bConsoleCheck)
		{
			StrongStats->OnPatchCheckStep_CheckPlatformPatchComplete(bSucceeded, LexToString(Result));
		}
		else
		{
			StrongStats->OnPatchCheckStep_CheckOnlineServicePatchComplete(bSucceeded, LexToString(Result));
		}
	}

	// If the result is a failure, check whether the result should be remapped to NoPatchRequired.
	TOptional<FPreviousSuccessInfo>& PreviousCachedSuccess = bConsoleCheck ? PreviousPlatformSuccess : PreviousOnlineServiceSuccess;
	if (Result == EPatchCheckResult::PatchCheckFailure)
	{
		if (CVarPatchCheckFailOpenOnError.GetValueOnGameThread())
		{
			auto GetLastSuccessString = [](const TOptional<FPreviousSuccessInfo>& CachedSuccess) -> FString
			{
				return CachedSuccess.IsSet() ? FString::Printf(TEXT("%f seconds ago"), FPlatformTime::Seconds() - CachedSuccess->ResultTime) : TEXT("Never");
			};
			UE_LOGF(LogPatchCheck, Log, "[%s] Remapping failure to NoPatchRequired. [Type=%ls], [LastSuccess=%ls]",
				__FUNCTION__, bConsoleCheck ? TEXT("PlatformOSS") : TEXT("DefaultOSS"), *GetLastSuccessString(PreviousCachedSuccess));
			Result = EPatchCheckResult::NoPatchRequired;
		}
	}
	else if (Result == EPatchCheckResult::NoPatchRequired)
	{
		// Store the most recent success.
		FPreviousSuccessInfo& SuccessInfo = PreviousCachedSuccess.IsSet() ? *PreviousCachedSuccess : PreviousCachedSuccess.Emplace();
		SuccessInfo.ResultTime = FPlatformTime::Seconds();
	}

	if (bCheckOSSForUpdate && bConsoleCheck && Result == EPatchCheckResult::NoPatchRequired)
	{
		// We perform both checks in this case
		StartOSSPatchCheck();
		return;
	}

	PatchCheckComplete(Result);
}

void FPatchCheck::PatchCheckComplete(EPatchCheckResult PatchResult)
{
	UE_LOGF(LogPatchCheck, Log, "[%s] [PatchResult=%ls]", __FUNCTION__, LexToString(PatchResult));
	if (TSharedPtr<IPatchCheckStatsCollector> StrongStats = Stats.Pin())
	{
		StrongStats->OnPatchCheckComplete(PatchResult);
	}

	LastPatchCheckResult = PatchResult;
	OnComplete.Broadcast(PatchResult);
	bIsCheckInProgress = false;
}
