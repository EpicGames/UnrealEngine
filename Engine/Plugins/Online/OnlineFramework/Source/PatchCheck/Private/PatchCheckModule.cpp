// Copyright Epic Games, Inc. All Rights Reserved.

#include "PatchCheckModule.h"

class FPatchCheckModule : public TPatchCheckModule<FPatchCheck>
{
};

IMPLEMENT_MODULE(FPatchCheckModule, PatchCheck);
