// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Interfaces/OnlineIdentityInterface.h"

#define UE_API PATCHCHECK_API

template<class PatchCheckClass> class TPatchCheckModule;

PATCHCHECK_API DECLARE_LOG_CATEGORY_EXTERN(LogPatchCheck, Log, All);

/**
 * Possible outcomes at the end of just the patch check
 */
enum class EPatchCheckResult : uint8
{
	/** No patch required */
	NoPatchRequired,
	/** Patch required to continue */
	PatchRequired,
	/** Logged in user required for a patch check */
	NoLoggedInUser,
	/** Patch check failed */
	PatchCheckFailure,
	Count,
};
PATCHCHECK_API const TCHAR* LexToString(EPatchCheckResult Value);

class IPatchCheckStatsCollector
{
public:
	virtual ~IPatchCheckStatsCollector() = default;

	// Overall process/
	virtual void OnPatchCheckStarted() = 0;
	virtual void OnPatchCheckComplete(EPatchCheckResult Result) = 0;

	// Individual stages.
	virtual void OnPatchCheckStep_DetectEnvironmentStarted() = 0;
	virtual void OnPatchCheckStep_DetectEnvironmentComplete(bool bSuccess, const FString& Error) = 0;
	virtual void OnPatchCheckStep_CheckPlatformPatchStarted() = 0;
	virtual void OnPatchCheckStep_CheckPlatformPatchComplete(bool bSuccess, const FString& Error) = 0;
	virtual void OnPatchCheckStep_CheckOnlineServicePatchStarted() = 0;
	virtual void OnPatchCheckStep_CheckOnlineServicePatchComplete(bool bSuccess, const FString& Error) = 0;
};

DECLARE_MULTICAST_DELEGATE_OneParam(FOnPatchCheckComplete, EPatchCheckResult /*Result*/);

// For backwards compatibility only!
DECLARE_DELEGATE_RetVal(bool, FEnvironmentWantsPatchCheck);

class FPatchCheck
{
public:
	static UE_API FPatchCheck& Get();

	UE_API virtual ~FPatchCheck();

protected:
	UE_API FPatchCheck();

private:
	FPatchCheck(const FPatchCheck& Other) = delete;

	FPatchCheck& operator=(const FPatchCheck& Other) = delete;

public:
	UE_API void StartPatchCheck();

	UE_API void AddEnvironmentWantsPatchCheckBackCompatDelegate(FName Tag, FEnvironmentWantsPatchCheck Delegate);
	UE_API void RemoveEnvironmentWantsPatchCheckBackCompatDelegate(FName Tag);

	FOnPatchCheckComplete& GetOnComplete() { return OnComplete; }
	TOptional<EPatchCheckResult> GetLastPatchCheckResult() const { return LastPatchCheckResult; }

	UE_API void RegisterStatsCollector(const TSharedPtr<IPatchCheckStatsCollector>& InStats);

protected:
	UE_API void RefreshConfig();
	UE_API virtual void StartPlatformOSSPatchCheck();
	UE_API virtual void StartOSSPatchCheck();
	UE_API void HandleOSSPatchCheck();
	UE_API virtual bool EnvironmentWantsPatchCheck() const;
	UE_API virtual bool EditorWantsPatchCheck() const;
	UE_API bool SkipPatchCheck() const;
	UE_API void OnCheckForPatchComplete(const FUniqueNetId& UniqueId, EUserPrivileges::Type Privilege, uint32 PrivilegeResult, bool bConsoleCheck);
	UE_API virtual void PatchCheckComplete(EPatchCheckResult PatchResult);

#if PATCH_CHECK_PLATFORM_ENVIRONMENT_DETECTION
	/**
	 * Platform specific implementation of platform environment detection
	 * @return true if the detection began.  false if the detection did not begin and we should continue the checks.
	 */
	UE_API bool DetectPlatformEnvironment();

	/**
	 * Platform specific callback for logging in on console which is needed for the platform environment detection
	 */
	UE_API virtual void DetectPlatformEnvironment_OnLoginConsoleComplete(int32 LocalUserNum, bool bWasSuccessful, const FUniqueNetId& UserId, const FString& Error);
#endif
	/**
	 * Callback when detecting platform environment completes
	 */
	UE_API void OnDetectPlatformEnvironmentComplete(const FOnlineError& Result);

protected:

	struct FPreviousSuccessInfo
	{
		double ResultTime = 0;
	};

	TWeakPtr<IPatchCheckStatsCollector> Stats;
	TOptional<FPreviousSuccessInfo> PreviousPlatformSuccess;
	TOptional<FPreviousSuccessInfo> PreviousOnlineServiceSuccess;
	TOptional<EPatchCheckResult> LastPatchCheckResult;

	FOnPatchCheckComplete OnComplete;

	/** For backwards compatibility with UUpdateManager */
	TMap<FName, FEnvironmentWantsPatchCheck> BackCompatEnvironmentWantsPatchCheckDelegates;

	/** Track whether we can start a new check */
	bool bIsCheckInProgress = false;

	/** Check the platform OSS for an update */	
	bool bCheckPlatformOSSForUpdate = true;
	/** Check the default OSS for an update */
	bool bCheckOSSForUpdate = true;

#if PATCH_CHECK_PLATFORM_ENVIRONMENT_DETECTION
	FDelegateHandle OnLoginConsoleCompleteHandle;
#endif

	/** true if we've already detected the backend environment */
	bool bPlatformEnvironmentDetected = !PATCH_CHECK_PLATFORM_ENVIRONMENT_DETECTION; // Default to true if we do not need to detect
	bool bPlatformEnvironmentDetectionEnabled = true;

	friend class TPatchCheckModule<FPatchCheck>;
};

#undef UE_API
