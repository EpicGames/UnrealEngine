// Copyright Epic Games, Inc. All Rights Reserved.

// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleManager.h"
#include "Misc/ConfigCacheIni.h"
#include "PatchCheck.h"

class IPatchCheckModule : public IModuleInterface
{
public:
	virtual void StartupModule() override
	{
		// Only instantiate if this is the version the game has been configured to use
		FString ModuleName;
		GConfig->GetString(TEXT("PatchCheck"), TEXT("ModuleName"), ModuleName, GEngineIni);

		if (FModuleManager::Get().GetModule(*ModuleName) == this)
		{
			MakePatchCheck();
		}
	}

	virtual void PreUnloadCallback() override
	{
		PatchCheck = nullptr;
	}

	virtual FPatchCheck* MakePatchCheck() = 0;

	FPatchCheck* GetPatchCheck()
	{
		return PatchCheck.Get();
	}

protected:
	TUniquePtr<FPatchCheck> PatchCheck;
};

template<class PatchCheckClass>
class TPatchCheckModule : public IPatchCheckModule
{
public:
	virtual FPatchCheck* MakePatchCheck() override
	{
		if (PatchCheck == nullptr)
		{
			PatchCheck = TUniquePtr<PatchCheckClass>(new PatchCheckClass());
		}
		return PatchCheck.Get();
	}
};
