// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class PlayTimeLimit : ModuleRules
{
	public PlayTimeLimit(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreOnline",
				"CoreUObject",
			}
		);

		PrivateDependencyModuleNames.AddRange(
		new string[]
			{
				"OnlineSubsystem",
			}
		);
	}
}
