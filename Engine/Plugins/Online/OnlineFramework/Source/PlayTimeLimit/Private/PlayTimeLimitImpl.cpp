// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlayTimeLimitImpl.h"

#include "Features/IModularFeatures.h"

#include "PlayTimeLimitModule.h"
#include "PlayTimeLimitUserMock.h"

#include "Online/CoreOnline.h"

// FPlayTimeLimitImpl
FPlayTimeLimitImpl::FPlayTimeLimitImpl()
{
}

FPlayTimeLimitImpl::~FPlayTimeLimitImpl()
{
}

FPlayTimeLimitImpl& FPlayTimeLimitImpl::Get()
{
	static FPlayTimeLimitImpl Singleton;	
	return Singleton;
}

void FPlayTimeLimitImpl::Initialize()
{
	IModularFeatures::Get().RegisterModularFeature(GetModularFeatureName(), this);

	// @todo make this data driven
	ConfigRates.Emplace(0, 60, 1.0f); // Notify every hour, 100% rewards at 0 hours
	ConfigRates.Emplace(3 * 60, 30, 0.5f); // Notify every 30 minutes, 50% rewards at 3 hours
	ConfigRates.Emplace(5 * 60, 15, 0.0f); // Notify every 15 minutes, 0% rewards at 5 hours
	// For simplicity of usage, sort by start time.
	// @todo Uncomment when we make this data driven, with hard coded values we can ensure the order
	ConfigRates.Sort([](const FOnlinePlayLimitConfigEntry& A, const FOnlinePlayLimitConfigEntry& B) { return A.TimeStartMinutes < B.TimeStartMinutes; });

	if (ensure(!TickHandle.IsValid()))
	{
		// Register delegate for ticker callback
		FTickerDelegate TickDelegate = FTickerDelegate::CreateRaw(this, &FPlayTimeLimitImpl::Tick);
		TickHandle = FTSTicker::GetCoreTicker().AddTicker(TickDelegate, 0.0f);
	}
}

void FPlayTimeLimitImpl::Shutdown()
{
	IModularFeatures::Get().UnregisterModularFeature(GetModularFeatureName(), this);

	if (TickHandle.IsValid())
	{
		FTSTicker::RemoveTicker(TickHandle);
		TickHandle.Reset();
	}

	Users.Empty();
}

bool FPlayTimeLimitImpl::Tick(float DeltaTime)
{
    QUICK_SCOPE_CYCLE_COUNTER(STAT_FPlayTimeLimitImpl_Tick);
	const bool bRetick = true;
	if (Users.Num() == 0)
	{
		return bRetick;
	}

	// Perform logic periodically
	static constexpr double TickFrequencySeconds = 1.0;
	const double Now = FPlatformTime::Seconds();
	if ((LastTickLogicTime == 0) || ((Now - LastTickLogicTime) > TickFrequencySeconds))
	{
		for (const FPlayTimeLimitUserPtr& User : Users)
		{
			User->Tick();
			if (User->HasTimeLimit())
			{
				const float LastKnownRewardRate = User->GetLastKnownRewardRate();
				const float RewardRate = User->GetRewardRate();
				const TOptional<double> NextNotificationTime = User->GetNextNotificationTime();

				const bool bRewardRateChanged = !FMath::IsNearlyEqual(LastKnownRewardRate, RewardRate);
				const bool bShouldNotifyFromPeriodicReminder = NextNotificationTime.IsSet() && NextNotificationTime.GetValue() < Now;
				const bool bShouldNotify = bRewardRateChanged || bShouldNotifyFromPeriodicReminder;

				if (bRewardRateChanged)
				{
					User->SetLastKnownRewardRate(RewardRate);
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
					// Do we want this to log in shipping builds?
					UE_LOGF(LogPlayTimeLimit, Log, "FPlayTimeLimitImpl: User [%ls] RewardRate changed from %0.2f to %0.2f", *User->GetUserId()->ToDebugString(), LastKnownRewardRate, RewardRate);
#endif // !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
				}

				if (bShouldNotify)
				{
					const int32 PlayTimeMinutes = User->GetPlayTimeMinutes();
					GetWarnUserPlayTimeDelegate().Broadcast(*User->GetUserId(), PlayTimeMinutes, RewardRate, *User->OverrideDialogTitle, *User->OverrideDialogText, *User->OverrideButtonText);
					UpdateNextNotificationTime(*User, PlayTimeMinutes);
					User->ClearDialogOverrideText();
				}
			}
		}
		LastTickLogicTime = Now;
	}
	return bRetick;
}

void FPlayTimeLimitImpl::RegisterUser(const FUniqueNetId& UserId)
{
	if (!Users.ContainsByPredicate([&UserId](const FPlayTimeLimitUserPtr& User) { return *User->GetUserId() == UserId; }))
	{
		if (OnRequestCreateUser.IsBound())
		{
			FPlayTimeLimitUserRawPtr NewUser = OnRequestCreateUser.Execute(UserId);
			if (NewUser)
			{
				FPlayTimeLimitUserPtr User = MakeShareable(NewUser);
				Users.Emplace(User);

				User->Init();
				const int32 PlayTimeMinutes = User->GetPlayTimeMinutes();
				UpdateNextNotificationTime(*User, PlayTimeMinutes);
			}
			else
			{
				UE_LOGF(LogPlayTimeLimit, Warning, "FPlayTimeLimitImpl: OnRequestCreateUser Delegate returned a null User.");
			}
		}
		else
		{
			UE_LOGF(LogPlayTimeLimit, Warning, "FPlayTimeLimitImpl: No OnRequestCreateUser delegate bound.");
		}
	}
	else
	{
		UE_LOGF(LogPlayTimeLimit, Log, "FPlayTimeLimitImpl: User [%ls] already registered", *UserId.ToDebugString());
	}
}

void FPlayTimeLimitImpl::UnregisterUser(const FUniqueNetId& UserId)
{
	const int32 Index = Users.IndexOfByPredicate([&UserId](const FPlayTimeLimitUserPtr& User) { return *User->GetUserId() == UserId; });
	if (Index != INDEX_NONE)
	{
		Users.RemoveAtSwap(Index);
	}
	else
	{
		UE_LOGF(LogPlayTimeLimit, Log, "FPlayTimeLimitImpl: User [%ls] not registered", *UserId.ToDebugString());
	}
}

void FPlayTimeLimitImpl::MockUser(const FUniqueNetId& UserId, const bool bHasTimeLimit, const double CurrentPlayTimeMinutes)
{
#if ALLOW_PLAY_LIMIT_MOCK
	const int32 ExistingIndex = Users.IndexOfByPredicate([&UserId](const FPlayTimeLimitUserPtr& User) { return *User->GetUserId() == UserId; });
	if (ExistingIndex != INDEX_NONE)
	{
		Users.RemoveAtSwap(ExistingIndex);
		const FPlayTimeLimitUserPtr& User = Users[Users.Emplace(new FPlayTimeLimitUserMock(UserId.AsShared(), bHasTimeLimit, CurrentPlayTimeMinutes))];
		// Hacky solution to try to line up the next notification time based on the new play time minutes
		// Pretend the user logged in at 0 minutes, so that notifications happen at exactly 60 minutes, 120 minutes, etc
		// The behavior of the real system is 60 minutes (etc) from login time, because WeGame does not tell us the exact number of minutes the player has played
		const FOnlinePlayLimitConfigEntry* ConfigRate = nullptr;
		if (User->HasTimeLimit())
		{
			ConfigRate = GetConfigEntry(static_cast<int32>(CurrentPlayTimeMinutes));
		}

		const float RewardRate = (ConfigRate != nullptr) ? ConfigRate->RewardRate : 1.0f;
		User->SetLastKnownRewardRate(RewardRate);

		int32 SecondsToNextNotification = 0;
		if (ConfigRate && ConfigRate->NotificationRateMinutes != 0)
		{
			const double NumMinutesInBracketAlready = CurrentPlayTimeMinutes - ConfigRate->TimeStartMinutes;
			const int32 NumNotificationsInBracketAlready = static_cast<int32>(NumMinutesInBracketAlready) / ConfigRate->NotificationRateMinutes;
			const double NowSeconds = FPlatformTime::Seconds();
			const double BracketStartTime = NowSeconds - (NumMinutesInBracketAlready * 60);
			const double NextNotificationTime = BracketStartTime + ((NumNotificationsInBracketAlready + 1) * ConfigRate->NotificationRateMinutes * 60);
			User->SetNextNotificationTime(NextNotificationTime);

			SecondsToNextNotification = static_cast<int32>(NextNotificationTime - NowSeconds);
		}
		else
		{
			User->SetNextNotificationTime(TOptional<double>());
		}
		UE_LOGF(LogPlayTimeLimit, Log, "MockUser: UserId=%ls, bHasTimeLimit=%ls, CurrentPlayTimeMinutes=%d, SecondsToNextNotification=%d", *UserId.ToDebugString(), bHasTimeLimit ? TEXT("true") : TEXT("false"), static_cast<int32>(CurrentPlayTimeMinutes), SecondsToNextNotification);
	}
#endif
}

void FPlayTimeLimitImpl::NotifyNow()
{
	// Well... on next Tick
	LastTickLogicTime = 0.0;
	double Now = FPlatformTime::Seconds();
	for (const FPlayTimeLimitUserPtr& User : Users)
	{
		User->SetNextNotificationTime(Now);
	}
}

void FPlayTimeLimitImpl::DumpState()
{
	UE_LOGF(LogPlayTimeLimit, Display, "FPlayTimeLimitImpl::DumpState: Begin");
	if (Users.Num() != 0)
	{
		const double Now = FPlatformTime::Seconds();
		for (const FPlayTimeLimitUserPtr& User : Users)
		{
			FString NextNotificationTimeString;
			const TOptional<double> NextNotificationTime = User->GetNextNotificationTime();
			if (NextNotificationTime.IsSet())
			{
				double SecondsToNextNotification = NextNotificationTime.GetValue() - Now;
				NextNotificationTimeString = FPlatformTime::PrettyTime(SecondsToNextNotification);
			}
			else
			{
				NextNotificationTimeString = TEXT("n/a");
			}
			UE_LOGF(LogPlayTimeLimit, Display, "  User [%ls]", *User->GetUserId()->ToDebugString());
			UE_LOGF(LogPlayTimeLimit, Display, "    HasTimeLimit: [%ls]", User->HasTimeLimit() ? TEXT("true") : TEXT("false"));
			UE_LOGF(LogPlayTimeLimit, Display, "    NextNotificationTime: [%ls]", *NextNotificationTimeString);
			UE_LOGF(LogPlayTimeLimit, Display, "    LastKnownRewardRate: %0.2f", User->GetLastKnownRewardRate());
			UE_LOGF(LogPlayTimeLimit, Display, "    RewardRate: %0.2f", User->GetRewardRate());
			UE_LOGF(LogPlayTimeLimit, Display, "    PlayTimeMinutes: %d", User->GetPlayTimeMinutes());
		}
	}
	else
	{
		UE_LOGF(LogPlayTimeLimit, Display, "No users");
	}
	UE_LOGF(LogPlayTimeLimit, Display, "FPlayTimeLimitImpl::DumpState: End");
}

bool FPlayTimeLimitImpl::HasTimeLimit(const FUniqueNetId& UserId)
{
	bool bHasTimeLimit = false;
	const FPlayTimeLimitUserPtr* const User = Users.FindByPredicate([&UserId](const FPlayTimeLimitUserPtr& ExistingUser) { return *ExistingUser->GetUserId() == UserId; });
	if (User != nullptr)
	{
		bHasTimeLimit = (*User)->HasTimeLimit();
	}
	else
	{
		UE_LOGF(LogPlayTimeLimit, Warning, "HasTimeLimit: UserId [%ls] is not registered", *UserId.ToDebugString());
	}
	return bHasTimeLimit;
}

int32 FPlayTimeLimitImpl::GetPlayTimeMinutes(const FUniqueNetId& UserId)
{
	int32 PlayTimeMinutes = 0;
	const FPlayTimeLimitUserPtr* const User = Users.FindByPredicate([&UserId](const FPlayTimeLimitUserPtr& ExistingUser) { return *ExistingUser->GetUserId() == UserId; });
	if (User != nullptr)
	{
		PlayTimeMinutes = (*User)->GetPlayTimeMinutes();
	}
	else
	{
		UE_LOGF(LogPlayTimeLimit, Warning, "GetPlayTimeMinutes: UserId [%ls] is not registered", *UserId.ToDebugString());
	}
	return PlayTimeMinutes;
}

float FPlayTimeLimitImpl::GetRewardRate(const FUniqueNetId& UserId)
{
	float RewardRate = 1.0f;
	const FPlayTimeLimitUserPtr* const User = Users.FindByPredicate([&UserId](const FPlayTimeLimitUserPtr& ExistingUser) { return *ExistingUser->GetUserId() == UserId; });
	if (User != nullptr)
	{
		RewardRate = (*User)->GetLastKnownRewardRate();
	}
	else
	{
		UE_LOGF(LogPlayTimeLimit, Warning, "GetRewardRate: UserId [%ls] is not registered", *UserId.ToDebugString());
	}
	if (RewardRate > 1.0f || RewardRate < 0.0f)
	{
		// Warn once if we find something suspicious
		static bool bWarned = false;
		if (!bWarned)
		{
			const int32 PlayTimeMinutes = GetPlayTimeMinutes(UserId);
			UE_LOGF(LogPlayTimeLimit, Warning, "GetRewardRate: Received RewardRate=%0.2f (Expected range: [0.0, 1.0]). PlayTimeMinutes=%d. Clamping to the expected range.", RewardRate, PlayTimeMinutes);
			bWarned = true;
		}
		RewardRate = FMath::Clamp(RewardRate, 0.0f, 1.0f);
	}
	return RewardRate;
}

FWarnUserPlayTime& FPlayTimeLimitImpl::GetWarnUserPlayTimeDelegate()
{
	return WarnUserPlayTimeDelegate;
}

void FPlayTimeLimitImpl::GameExitByRequest()
{
	OnGameExitRequestedDelegate.Broadcast();
}

const FOnlinePlayLimitConfigEntry* FPlayTimeLimitImpl::GetConfigEntry(const int32 PlayTimeMinutes) const
{
	const FOnlinePlayLimitConfigEntry* Result = nullptr;
	// Find the first entry that has a time start greater than the number of minutes played
	// Note that the list is already sorted by TimeStartMinutes
	for (const FOnlinePlayLimitConfigEntry& ConfigRate : ConfigRates)
	{
		if (PlayTimeMinutes >= ConfigRate.TimeStartMinutes)
		{
			// If we played this long, we are least this limited.  No break, next limit might also apply to us
			Result = &ConfigRate;
		}
		else
		{
			break;
		}
	}
	return Result;
}

void FPlayTimeLimitImpl::UpdateNextNotificationTime(FPlayTimeLimitUser& User, const int32 PlayTimeMinutes) const
{
	const FOnlinePlayLimitConfigEntry* ConfigRate = nullptr;
	if (User.HasTimeLimit())
	{
		ConfigRate = GetConfigEntry(PlayTimeMinutes);
	}
	if (ConfigRate && ConfigRate->NotificationRateMinutes != 0)
	{
		User.SetNextNotificationTime(FPlatformTime::Seconds() + (ConfigRate->NotificationRateMinutes * 60));
	}
	else
	{
		User.SetNextNotificationTime(TOptional<double>());
	}
}
