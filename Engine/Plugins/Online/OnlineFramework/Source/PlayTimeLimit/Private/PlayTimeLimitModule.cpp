// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlayTimeLimitModule.h"

#include "PlayTimeLimitImpl.h"

#include "OnlineSubsystem.h"
#include "Interfaces/OnlineIdentityInterface.h"
#include "Misc/ConfigCacheIni.h"

IMPLEMENT_MODULE(FPlayTimeLimitModule, PlayTimeLimit);

DEFINE_LOG_CATEGORY(LogPlayTimeLimit);

void FPlayTimeLimitModule::StartupModule()
{
	GConfig->GetBool(TEXT("PlayTimeLimit"), TEXT("bEnabled"), bPlayTimeLimitEnabled, GEngineIni);
	if (bPlayTimeLimitEnabled)
	{
		FPlayTimeLimitImpl::Get().Initialize();
	}
}

void FPlayTimeLimitModule::ShutdownModule()
{
	if (bPlayTimeLimitEnabled)
	{
		FPlayTimeLimitImpl::Get().Shutdown();
	}
}

bool FPlayTimeLimitModule::Exec_Runtime(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
{
	bool bWasHandled = false;

	// Ignore any execs that don't start with PlayTimeLimit
	if (FParse::Command(&Cmd, TEXT("PlayTimeLimit")))
	{
		if (bPlayTimeLimitEnabled)
		{
			bWasHandled = true; // Set to false in the catch-all else block
			FPlayTimeLimitImpl& PlayTimeLimit = FPlayTimeLimitImpl::Get();
			if (FParse::Command(&Cmd, TEXT("DUMP")))
			{
				PlayTimeLimit.DumpState();
			}
			else if (FParse::Command(&Cmd, TEXT("NOTIFYNOW")))
			{
				PlayTimeLimit.NotifyNow();
			}
			else if (FParse::Command(&Cmd, TEXT("SETUSERLIMITS")))
			{
				// Usage: SETUSERLIMITS <(optional) sub=(Online Subsystem Name)> <usernum> <has limit = [true, false]> <current play time limits>
				// Ex:  PLAYTIMELIMIT SETUSERLIMITS SUB=NULL 0 TRUE 60
				// Ex:  PLAYTIMELIMIT SETUSERLIMITS 0 FALSE

				FString SubNameString;
				FParse::Value(Cmd, TEXT("Sub="), SubNameString);
				// Allow for either Sub=<platform> or Subsystem=<platform>
				if (SubNameString.Len() > 0)
				{
					Cmd += FCString::Strlen(TEXT("Sub=")) + SubNameString.Len();
				}
				else
				{ 
					FParse::Value(Cmd, TEXT("Subsystem="), SubNameString);
					if (SubNameString.Len() > 0)
					{
						Cmd += FCString::Strlen(TEXT("Subsystem=")) + SubNameString.Len();
					}
				}

				FName SubName(NAME_None);
				if (!SubNameString.IsEmpty())
				{
					SubName = FName(*SubNameString);
				}

				const int32 LocalUserNum = FCString::Atoi(*FParse::Token(Cmd, false));
				const bool bHasLimit = FCString::ToBool(*FParse::Token(Cmd, false));
				const double CurrentPlayTimeMinutes = FCString::Atof(*FParse::Token(Cmd, false));

				IOnlineSubsystem* const OnlineSubsystem = IOnlineSubsystem::Get(SubName);
				if (OnlineSubsystem)
				{
					const IOnlineIdentityPtr IdentityInt = OnlineSubsystem->GetIdentityInterface();
					if (IdentityInt.IsValid())
					{
						const FUniqueNetIdPtr UserId = IdentityInt->GetUniquePlayerId(LocalUserNum);
						if (UserId.IsValid())
						{
							PlayTimeLimit.MockUser(*UserId, bHasLimit, CurrentPlayTimeMinutes);
						}
						else
						{
							UE_LOGF(LogPlayTimeLimit, Warning, "SETUSERLIMITS: Could not get player id from user num=%d, ensure you are logged in first", LocalUserNum);
						}
					}
					else
					{
						UE_LOGF(LogPlayTimeLimit, Warning, "SETUSERLIMITS: Missing Identity interface");
					}
				}
				else
				{
					UE_LOGF(LogPlayTimeLimit, Warning, "SETUSERLIMITS: Missing OnlineSubsystem");
				}
			}
			else
			{
				bWasHandled = false;
			}
		}
		else
		{
			UE_LOGF(LogPlayTimeLimit, Warning, "PlayTimeLimit is not enabled by config file");
		}
	}
	return bWasHandled;
}
