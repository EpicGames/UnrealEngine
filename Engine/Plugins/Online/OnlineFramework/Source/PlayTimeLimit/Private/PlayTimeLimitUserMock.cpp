// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlayTimeLimitUserMock.h"
#include "HAL/PlatformTime.h"
#include "PlayTimeLimitImpl.h"

#if ALLOW_PLAY_LIMIT_MOCK

FPlayTimeLimitUserMock::FPlayTimeLimitUserMock(const FUniqueNetIdRef& InUserId, const bool bInHasTimeLimit, const double InPlayTimeMinutes)
	: FPlayTimeLimitUser(InUserId)
	, bHasTimeLimit(bInHasTimeLimit)
	, PlayTimeMinutesStart(InPlayTimeMinutes)
	, TimeOverrideSet(FPlatformTime::Seconds())
{
}

bool FPlayTimeLimitUserMock::HasTimeLimit() const
{
	return bHasTimeLimit;
}

int32 FPlayTimeLimitUserMock::GetPlayTimeMinutes() const
{
	int32 PlayTimeMinutes = 0;
	// Only track number of minutes played for users that have a limit
	if (bHasTimeLimit)
	{
		const double Now(FPlatformTime::Seconds());
		PlayTimeMinutes = static_cast<int32>(PlayTimeMinutesStart + ((Now - TimeOverrideSet) / 60.0));
	}
	return PlayTimeMinutes;
}

float FPlayTimeLimitUserMock::GetRewardRate() const
{
	float RewardRate = 1.0f;
	if (bHasTimeLimit)
	{
		const int32 PlayTimeMinutes = GetPlayTimeMinutes();
		const FOnlinePlayLimitConfigEntry* const ConfigEntry = FPlayTimeLimitImpl::Get().GetConfigEntry(PlayTimeMinutes);
		if (ConfigEntry != nullptr)
		{
			RewardRate = ConfigEntry->RewardRate;
		}
	}
	return RewardRate;
}

#endif // ALLOW_PLAY_LIMIT_MOCK
