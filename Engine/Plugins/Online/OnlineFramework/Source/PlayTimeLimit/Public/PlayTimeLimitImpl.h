// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Interfaces/OnlinePlayTimeLimit.h"
#include "PlayTimeLimitUser.h"
#include "Containers/Ticker.h"

#define UE_API PLAYTIMELIMIT_API

/**
 * Configuration
 */
struct FOnlinePlayLimitConfigEntry
{
	/** Constructor */
	FOnlinePlayLimitConfigEntry(int32 InTimeStartMinutes, int32 InNotificationRateMinutes, float InRewardRate)
		: TimeStartMinutes(InTimeStartMinutes)
		, NotificationRateMinutes(InNotificationRateMinutes)
		, RewardRate(InRewardRate)
	{}

	/** Number of minutes the user must play before this is effective */
	int32 TimeStartMinutes;
	/** Number of minutes between notifications to the user about their play time */
	int32 NotificationRateMinutes;
	/** Reward rate at this limit */
	float RewardRate;
};

/**
 * Implementation of IOnlinePlayTimeLimit
 */
class FPlayTimeLimitImpl
	: public IOnlinePlayTimeLimit
{
public:
	// FPlayTimeLimitImpl

	/** Default constructor */
	UE_API FPlayTimeLimitImpl();
	/** Destructor */
	UE_API virtual ~FPlayTimeLimitImpl();

	/**
	 * Get the singleton
	 * @return Singleton instance
	 */
	static UE_API FPlayTimeLimitImpl& Get();
	
	/**
	 * Initialize
	 */
	UE_API void Initialize();

	/**
	 * Shutdown
	 */
	UE_API void Shutdown();

	/**
	 * Tick - update users and execute warn time delegates
	 */
	UE_API bool Tick(float Delta);

	DECLARE_DELEGATE_RetVal_OneParam(FPlayTimeLimitUserRawPtr, OnRequestCreateUserDelegate, const FUniqueNetId&);

	/**
	*  Delegate called when a game exit is requested
	*/
	DECLARE_MULTICAST_DELEGATE(FOnGameExitRequested);
	typedef FOnGameExitRequested::FDelegate FOnGameExitRequestedDelegate;

	/**
	 * Register a user to monitor their play time
	 * @see UnregisterUser
	 * @param NewUser the user to register
	 */
	UE_API void RegisterUser(const FUniqueNetId& NewUser);

	/**
	 * Unregister a user
	 * @see RegisterUser
	 * @param UserId the user id
	 */
	UE_API void UnregisterUser(const FUniqueNetId& UserId);

	/**
	 * Override a user's play time
	 * For testing the system without needing to potentially wait hours - waiting to accumulate time and waiting for the time to reset
	 */
	UE_API void MockUser(const FUniqueNetId& UserId, const bool bHasTimeLimit, const double CurrentPlayTimeMinutes);

	/**
	 * Cheat function to trigger the notification to players of their play time immediately
	 */
	UE_API void NotifyNow();
	
	// Begin IOnlinePlayTimeLimit
	UE_API virtual bool HasTimeLimit(const FUniqueNetId& UserId) override;
	UE_API virtual int32 GetPlayTimeMinutes(const FUniqueNetId& UserId) override;
	UE_API virtual float GetRewardRate(const FUniqueNetId& UserId) override;
	UE_API virtual FWarnUserPlayTime& GetWarnUserPlayTimeDelegate() override;
	UE_API void GameExitByRequest();
	// End IOnlinePlayTimeLimit

	/**
	 * Get the config entry that corresponds to the number of minutes played
	 * @param PlayTimeMinutes the number of minutes played to get the entry for
	 * @return the entry corresponding to the number of minutes played
	 */
	UE_API const FOnlinePlayLimitConfigEntry* GetConfigEntry(const int32 PlayTimeMinutes) const;

	/**
	 * Dump state to log
	 */
	UE_API void DumpState();

	OnRequestCreateUserDelegate OnRequestCreateUser;

	/** Delegate used to request a game exit */
	FOnGameExitRequested OnGameExitRequestedDelegate;

protected:
	/**
	 * Update the next notification time for a user based on their current play time
	 * @param User the user to update
	 */
	UE_API void UpdateNextNotificationTime(FPlayTimeLimitUser& User, const int32 PlayTimeMinutes) const;

	/** Delegate used to display a warning to the user about their play time */
	FWarnUserPlayTime WarnUserPlayTimeDelegate;

	/** List of users we are monitoring */
	TArray<FPlayTimeLimitUserPtr> Users;

	/** Last time we performed tick logic */
	double LastTickLogicTime = 0.0;

	/** Configuration to control notification rate at different levels of play time */
	TArray<FOnlinePlayLimitConfigEntry> ConfigRates;

	/** Delegate for callbacks to Tick */
	FTSTicker::FDelegateHandle TickHandle;

private:
	// Not copyable
	FPlayTimeLimitImpl(const FPlayTimeLimitImpl& Other) = delete;
	FPlayTimeLimitImpl& operator=(FPlayTimeLimitImpl& Other) = delete;
};

#undef UE_API
