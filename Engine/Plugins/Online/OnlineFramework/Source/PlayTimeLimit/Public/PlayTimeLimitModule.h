// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleManager.h"

PLAYTIMELIMIT_API DECLARE_LOG_CATEGORY_EXTERN(LogPlayTimeLimit, Log, All);

/**
 * Module for reducing rewards based on play time
 */
class FPlayTimeLimitModule : 
	public IModuleInterface, public FSelfRegisteringExec
{

public:
	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static inline FPlayTimeLimitModule& Get()
	{
		return FModuleManager::LoadModuleChecked<FPlayTimeLimitModule>("PlayTimeLimit");
	}

	/**
	 * Checks to see if this module is loaded and ready.  It is only valid to call Get() if IsAvailable() returns true.
	 *
	 * @return True if the module is loaded and ready to use
	 */
	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded("PlayTimeLimit");
	}

protected:
	// FSelfRegisteringExec
	virtual bool Exec_Runtime(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar) override;

private:

	// IModuleInterface

	/**
	 * Called when party module is loaded
	 * Initialize platform specific parts of voice handling
	 */
	virtual void StartupModule() override;
	
	/**
	 * Called when party module is unloaded
	 * Shutdown platform specific parts of voice handling
	 */
	virtual void ShutdownModule() override;

	/** Is this feature enabled? */
	bool bPlayTimeLimitEnabled = false;
};
