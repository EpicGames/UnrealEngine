// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/CoreOnlineFwd.h"

/**
 * Information about a user we are observing the play time and instituting limits for
 */
class FPlayTimeLimitUser : 
	public TSharedFromThis<FPlayTimeLimitUser, ESPMode::ThreadSafe>
{
public:
	/** Constructor */
	explicit FPlayTimeLimitUser(const FUniqueNetIdRef& InUserId)
		: UserId(InUserId)
		, NextNotificationTime(0)
		, LastKnownRewardRate(0.0f)
	{}

	virtual void Init()
	{
		LastKnownRewardRate = GetRewardRate();
	}

	/** Destructor */
	virtual ~FPlayTimeLimitUser() = default;

	/**
	 * Get the user's unique id
	 * @return the user's unique id
	 */
	FUniqueNetIdRef GetUserId() const { return UserId; }

	/**
	 * Tick
	 */
	virtual void Tick() {}
	
	/**
	 * Check if the user has a play time limit
	 * @return if this user has a play time limit
	 */
	virtual bool HasTimeLimit() const = 0;

	/**
	 * Get the number of minutes this user has played
	 * @return number of minutes this user has played
	 */
	virtual int32 GetPlayTimeMinutes() const = 0;

	/**
	 * Get the current reward rate
	 * @return the current reward rate
	 */
	virtual float GetRewardRate() const = 0;

	/**
	 * Get the next time we are scheduled to send the user a notification
	 * This is for the periodic notification; if the reward rate changes we will immediately display a notification
	 * @return the time (compared to FPlatformTime::Seconds()) for the next notification, or empty if this user should not receive periodic notifications
	 */
	TOptional<double> GetNextNotificationTime() const { return NextNotificationTime; }

	/**
	 * Set the next time to send the user a notification
	 * @see GetNextNotificationTime
	 * @param InNextNotificationTime the next notification time, or empty if this user should not receive periodic notifications
	 */
	virtual void SetNextNotificationTime(const TOptional<double>& InNextNotificationTime) { NextNotificationTime = InNextNotificationTime; }

	/**
	 * Get the last known reward rate
	 * @return the last known reward rate
	 */
	float GetLastKnownRewardRate() const { return LastKnownRewardRate; }

	/**
	 * Set the last known reward rate
	 * @param InLastKnownRewardRate the last known reward rate
	 */
	void SetLastKnownRewardRate(float InLastKnownRewardRate) { LastKnownRewardRate = InLastKnownRewardRate; }

	/**
	* Clear override dialog text
	*/
	virtual void ClearDialogOverrideText() {
		OverrideDialogTitle.Empty(); 
		OverrideDialogText.Empty();
		OverrideButtonText.Empty();
	}

	FString OverrideDialogTitle;
	FString OverrideDialogText;
	FString OverrideButtonText;

protected:
	/** The user id */
	FUniqueNetIdRef UserId;
	/** Time for the next notification (or empty for no notification) */
	TOptional<double> NextNotificationTime;
	/** Last known reward rate so we can alert on changes */
	float LastKnownRewardRate;
};

typedef FPlayTimeLimitUser* FPlayTimeLimitUserRawPtr;
typedef TSharedPtr<FPlayTimeLimitUser, ESPMode::ThreadSafe> FPlayTimeLimitUserPtr;
