// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PlayTimeLimitUser.h"

#if !defined ALLOW_PLAY_LIMIT_MOCK
#define ALLOW_PLAY_LIMIT_MOCK (!(UE_BUILD_SHIPPING))
#endif

#if ALLOW_PLAY_LIMIT_MOCK
/**
 * Mock implementation of FPlayTimeLimitUser
 */
class FPlayTimeLimitUserMock
	: public FPlayTimeLimitUser
{
public:
	FPlayTimeLimitUserMock(const FUniqueNetIdRef& InUserId, const bool bInHasTimeLimit, const double InPlayTimeMinutes);

	//~ Begin FPlayTimeLimitUser Interface
	virtual bool HasTimeLimit() const override;
	virtual int32 GetPlayTimeMinutes() const override;
	virtual float GetRewardRate() const override;
	//~ End FPlayTimeLimitUser Interface
	
protected:
	/** Do the time limits apply to the user? */
	const bool bHasTimeLimit;
	/** Override time played. Effective time played is PlayTimeMinutesStart + (Now - TimeOverrideSet) */
	const double PlayTimeMinutesStart;
	/** The time this override was set. Effective time played is PlayTimeMinutesStart + (Now - TimeOverrideSet) */
	const double TimeOverrideSet;
};
#endif // ALLOW_PLAY_LIMIT_MOCK
