// Copyright Epic Games, Inc. All Rights Reserved.

#include "QosBeaconClient.h"
#include "Interfaces/OnlineSessionInterface.h"
#include "OnlineSessionSettings.h"
#include "QosBeaconHost.h"
#include "OnlineSubsystemUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(QosBeaconClient)

AQosBeaconClient::AQosBeaconClient(const FObjectInitializer& ObjectInitializer) :
	Super(ObjectInitializer),
	ConnectionStartTime(0.0),
	QosStartTime(0.0),
	bPendingQosRequest(false)
{
}

void AQosBeaconClient::OnConnected()
{
	// Start a timer and wait for a response
	UE_LOGF(LogBeacon, Verbose, "Qos beacon connection established, sending request.");
	ServerQosRequest(DestSessionId);
	QosStartTime = FPlatformTime::Seconds();
	bPendingQosRequest = true;
}

void AQosBeaconClient::SendQosRequest(const FOnlineSessionSearchResult& DesiredHost)
{
	bool bSuccess = false;
	IOnlineSubsystem* OnlineSub = Online::GetSubsystem(GetWorld());
	if (OnlineSub)
	{
		IOnlineSessionPtr SessionInt = OnlineSub->GetSessionInterface();
		if (SessionInt.IsValid())
		{
			FString ConnectInfo;
			if (SessionInt->GetResolvedConnectString(DesiredHost, NAME_BeaconPort, ConnectInfo))
			{
				FURL ConnectURL(NULL, *ConnectInfo, TRAVEL_Absolute);
				if (InitClient(ConnectURL) && DesiredHost.Session.SessionInfo.IsValid())
				{
					ConnectionStartTime = FPlatformTime::Seconds();

					DestSessionId = DesiredHost.Session.SessionInfo->GetSessionId().ToString();
					bPendingQosRequest = false;
					bSuccess = true;
				}
				else
				{
					UE_LOGF(LogBeacon, Warning, "SendQosRequest: Failure to init client beacon with %ls.", *ConnectURL.ToString());
				}
			}
		}
	}

	if (!bSuccess)
	{
		AOnlineBeacon::OnFailure(EBeaconFailureReason::InitFailure, TEXTVIEW("beacon_qosclient_init_failure"));
	}
}

bool AQosBeaconClient::ServerQosRequest_Validate(const FString& InSessionId)
{
	return !InSessionId.IsEmpty();
}

void AQosBeaconClient::ServerQosRequest_Implementation(const FString& InSessionId)
{
	AQosBeaconHost* BeaconHost = Cast<AQosBeaconHost>(GetBeaconOwner());
	if (BeaconHost)
	{
		BeaconHost->ProcessQosRequest(this, InSessionId);
	}
}

void AQosBeaconClient::ClientQosResponse_Implementation(EQosResponseType Response)
{
	double EndTime = FPlatformTime::Seconds();
	UE_LOGF(LogBeacon, Display, "ClientQosResponse: total time: %f RPC time: %f.", EndTime - ConnectionStartTime, EndTime - QosStartTime);

	int32 ResponseTime = FMath::TruncToInt((EndTime - QosStartTime) * 1000.0);
	QosRequestComplete.ExecuteIfBound(EQosResponseType::Success, ResponseTime);
}


