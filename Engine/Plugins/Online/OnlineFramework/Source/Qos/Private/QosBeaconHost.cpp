// Copyright Epic Games, Inc. All Rights Reserved.

#include "QosBeaconHost.h"
#include "Engine/NetConnection.h"
#include "Interfaces/OnlineSessionInterface.h"
#include "OnlineSessionSettings.h"
#include "QosBeaconClient.h"
#include "OnlineSubsystemUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(QosBeaconHost)

AQosBeaconHost::AQosBeaconHost(const FObjectInitializer& ObjectInitializer) :
	Super(ObjectInitializer),
	SessionName(NAME_None),
	NumQosRequests(0)
{
	ClientBeaconActorClass = AQosBeaconClient::StaticClass();
	BeaconTypeName = ClientBeaconActorClass->GetName();
}

bool AQosBeaconHost::Init(FName InSessionName)
{
	SessionName = InSessionName;
	NumQosRequests = 0;
	return true;
}

bool AQosBeaconHost::DoesSessionMatch(const FString& SessionId) const
{
	UWorld* World = GetWorld();

	IOnlineSessionPtr SessionInt = Online::GetSessionInterface(World);
	FNamedOnlineSession* Session = SessionInt.IsValid() ? SessionInt->GetNamedSession(SessionName) : NULL;
	if (Session && Session->SessionInfo.IsValid() && !SessionId.IsEmpty() && Session->SessionInfo->GetSessionId().ToString() == SessionId)
	{
		return true;
	}

	return false;
}

void AQosBeaconHost::ProcessQosRequest(AQosBeaconClient* Client, const FString& SessionId)
{
	UE_LOGF(LogBeacon, Verbose, "ProcessQosRequest %ls SessionId %ls from (%ls)",
		Client ? *Client->GetName() : TEXT("NULL"),
		*SessionId,
		Client ? *Client->GetNetConnection()->LowLevelDescribe() : TEXT("NULL"));

	NumQosRequests++;

	if (Client)
	{
		if (DoesSessionMatch(SessionId))
		{
			Client->ClientQosResponse(EQosResponseType::Success);
		}
		else
		{
			Client->ClientQosResponse(EQosResponseType::Failure);
		}
	}
}

void AQosBeaconHost::DumpState() const
{
	UE_LOGF(LogBeacon, Display, "Qos Beacon: %ls", *GetBeaconType());
	UE_LOGF(LogBeacon, Display, "Session that beacon is for: %ls", *SessionName.ToString());
	UE_LOGF(LogBeacon, Display, "Number of Qos requests: %d", NumQosRequests);
}

