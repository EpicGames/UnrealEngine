// Copyright Epic Games, Inc. All Rights Reserved.

#include "QosEvaluator.h"
#include "TimerManager.h"
#include "QosModule.h"
#include "QosStats.h"
#include "Engine/World.h"
#include "Icmp.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(QosEvaluator)

UQosEvaluator::UQosEvaluator(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, ParentWorld(nullptr)
	, StartTimestamp(0.0)
	, bInProgress(false)
	, bCancelOperation(false)
	, AnalyticsProvider(nullptr)
	, QosStats(nullptr)
{
}

void UQosEvaluator::SetWorld(UWorld* InWorld)
{
	ParentWorld = InWorld;
}

void UQosEvaluator::SetAnalyticsProvider(TSharedPtr<IAnalyticsProvider> InAnalyticsProvider)
{
	AnalyticsProvider = InAnalyticsProvider;
}

void UQosEvaluator::Cancel()
{
	bCancelOperation = true;
}

void UQosEvaluator::FindDatacenters(const FQosParams& InParams, const TArray<FQosRegionInfo>& InRegions, const TArray<FQosDatacenterInfo>& InDatacenters, const FOnQosSearchComplete& InCompletionDelegate)
{
	if (!IsActive())
	{
		bInProgress = true;
		StartTimestamp = FPlatformTime::Seconds();

		StartAnalytics();

		Datacenters.Empty(InDatacenters.Num());
		for (const FQosRegionInfo& Region : InRegions)
		{
			if (Region.IsPingable())
			{
				int32 NumDatacenters = 0;
				for (const FQosDatacenterInfo& Datacenter : InDatacenters)
				{
					if (Datacenter.RegionId == Region.RegionId)
					{
						if (Datacenter.IsPingable())
						{
							Datacenters.Emplace(Datacenter, Region.IsUsable());
							NumDatacenters++;
						}
						else
						{
							UE_LOGF(LogQos, Verbose, "Skipping datacenter [%ls]", *Datacenter.Id);
						}
					}
				}

				if (NumDatacenters == 0)
				{
					UE_LOGF(LogQos, Warning, "Region [%ls] has no usable datacenters", *Region.RegionId);
				}
			}
			else
			{
				UE_LOGF(LogQos, Verbose, "Skipping region [%ls]", *Region.RegionId);
			}
		}

		// Ping list of known servers defined by config
		PingRegionServers(InParams, InCompletionDelegate);
	}
	else
	{
		UE_LOGF(LogQos, Log, "Qos evaluation already in progress, ignoring");
		// Just trigger delegate now (Finalize resets state vars)
		GetWorldTimerManager().SetTimerForNextTick(FTimerDelegate::CreateLambda([InCompletionDelegate]() {
			InCompletionDelegate.ExecuteIfBound(EQosCompletionResult::Failure, TArray<FDatacenterQosInstance>(), nullptr, nullptr);
		}));
	}
}

void UQosEvaluator::CalculatePingAverages(int32 TimeToDiscount)
{
	UE_LOGF(LogQos, Verbose, "ping average detail per datacenter:");
	for (FDatacenterQosInstance& Datacenter : Datacenters)
	{
		int32 TotalPingInMs = 0;
		int32 NumResults = 0;
		int32 NumUnreachable = 0;
		for (int32 PingIdx = 0; PingIdx < Datacenter.PingResults.Num(); PingIdx++)
		{
			if (Datacenter.PingResults[PingIdx] != UNREACHABLE_PING)
			{
				TotalPingInMs += Datacenter.PingResults[PingIdx];
				NumResults++;
			}
			else
			{
				NumUnreachable++;
			}
		}

		int32 RawAvgPing = UNREACHABLE_PING;
		Datacenter.AvgPingMs = RawAvgPing;
		if (NumResults > 0)
		{
			RawAvgPing = TotalPingInMs / NumResults;
			Datacenter.AvgPingMs = FMath::Max<int32>(RawAvgPing - TimeToDiscount, 1);
		}

		UE_LOGF(LogQos, Verbose, "\t%ls (%ls): %d/%d queries succeeded, average ping: %dms (adj: %dms)", *Datacenter.Definition.Id, *Datacenter.Definition.RegionId, NumResults, NumResults + NumUnreachable, Datacenter.AvgPingMs, RawAvgPing);

		if (QosStats.IsValid())
		{
			QosStats->RecordRegionInfo(Datacenter, NumResults);
		}
	}
}

bool UQosEvaluator::AreAllRegionsComplete()
{
	for (FDatacenterQosInstance& Region : Datacenters)
	{
		if (Region.Definition.bEnabled && Region.Result == EQosDatacenterResult::Invalid)
		{
			return false;
		}
	}

	return true;
}

void UQosEvaluator::OnEchoManyCompleted(FIcmpEchoManyCompleteResult FinalResult, int32 NumTestsPerRegion, const FOnQosSearchComplete& InQosSearchCompleteDelegate)
{
	UE_LOGF(LogQos, Log, "UQosEvaluator OnEchoManyCompleted; result status=%d", int(FinalResult.Status));

	// Copy our aggregated ping results to the appropriate datacenter result sets

	for (const FIcmpEchoManyResult& EchoManyResult : FinalResult.AllResults)
	{
		const FIcmpEchoResult& Result = EchoManyResult.EchoResult;
		const FIcmpTarget& Target = EchoManyResult.Target;

		// Find datacenter that matches the original request's target address for this result.
		FDatacenterQosInstance* const FoundDatacenter = FindDatacenterByAddress(
			Datacenters, Target.Address, Target.Port);

		if (FoundDatacenter)
		{
			FDatacenterQosInstance& Region = *FoundDatacenter;

			UE_LOGF(LogQos, VeryVerbose, "Ping Complete %ls %ls: %d",
				*Region.Definition.ToDebugString(), *Result.ResolvedAddress, static_cast<int32>(Result.Time * 1000.0f));

			int32 PingInMs = UNREACHABLE_PING;
			const bool bSuccess = (Result.Status == EIcmpResponseStatus::Success);
			if (bSuccess)
			{
				PingInMs = static_cast<int32>(Result.Time * 1000.0f);
				++Region.NumResponses;
			}
			Region.PingResults.Add(PingInMs);

			if (QosStats.IsValid())
			{
				const FString& DatacenterId = Region.Definition.Id;

				UE_LOGF(LogQos, VeryVerbose, "Record Qos attempt; ping=%d ms", PingInMs);
				QosStats->RecordQosAttempt(DatacenterId, Result.ResolvedAddress, PingInMs, bSuccess);
			}

			if (Region.PingResults.Num() == NumTestsPerRegion)
			{
				Region.LastCheckTimestamp = FDateTime::UtcNow();
				Region.Result = (Region.NumResponses == NumTestsPerRegion) ? EQosDatacenterResult::Success : EQosDatacenterResult::Incomplete;
			}

			UE_LOGF(LogQos, VeryVerbose, "Got %d/%d ping results (%d reachable) for datacenter (%ls, %ls); status=%d",
						 Region.PingResults.Num(), NumTestsPerRegion, Region.NumResponses, *Region.Definition.Id, *Region.Definition.RegionId, int(Region.Result));
		}
		else
		{
			UE_LOGF(LogQos, Verbose, "could not find matching datacenter for result (%ls) from %ls:%d!", LexToString(Result.Status), *Target.Address, Target.Port);
		}
	}

	EQosCompletionResult CompletionResult = EQosCompletionResult::Invalid;

	switch (FinalResult.Status)
	{
	case EIcmpEchoManyStatus::Success:
	{
		UE_LOGF(LogQos, Verbose, "Qos complete in %0.2f s  (all regions: %ls)",
			FPlatformTime::Seconds() - StartTimestamp, AreAllRegionsComplete() ? TEXT("yes") : TEXT("no"));

		CompletionResult = EQosCompletionResult::Success;
		CalculatePingAverages();
		FString SelectedSubRegion;
		FString SelectedRegion;
		InQosSearchCompleteDelegate.ExecuteIfBound(CompletionResult, Datacenters, &SelectedRegion, &SelectedSubRegion);

		if (QosStats.IsValid())
		{
			QosStats->SetChosenRegion(MoveTemp(SelectedRegion));
			QosStats->SetChosenSubRegion(MoveTemp(SelectedSubRegion));
		}
		EndAnalytics(CompletionResult);
	}
	break;

	case EIcmpEchoManyStatus::Failure:
	{
		UE_LOGF(LogQos, Verbose, "Qos failed after in %0.2f s", FPlatformTime::Seconds() - StartTimestamp);

		CompletionResult = EQosCompletionResult::Failure;
		EndAnalytics(CompletionResult);
		InQosSearchCompleteDelegate.ExecuteIfBound(CompletionResult, Datacenters, nullptr, nullptr);
	}
	break;

	case EIcmpEchoManyStatus::Canceled:
	{
		UE_LOGF(LogQos, Verbose, "Qos canceled after %0.2f s", FPlatformTime::Seconds() - StartTimestamp);

		CompletionResult = EQosCompletionResult::Canceled;
		EndAnalytics(CompletionResult);
		InQosSearchCompleteDelegate.ExecuteIfBound(CompletionResult, Datacenters, nullptr, nullptr);
	}
	break;

	default:
		break;
	};

	UE_LOGF(LogQos, Verbose, "UQosEvaluator OnEchoManyCompleted; Qos result=%d", int(CompletionResult));
}

void UQosEvaluator::StartAnalytics()
{
	if (AnalyticsProvider.IsValid())
	{
		ensure(!QosStats.IsValid());
		QosStats = MakeShareable(new FQosDatacenterStats());
		QosStats->StartQosPass();
	}
}

void UQosEvaluator::EndAnalytics(EQosCompletionResult CompletionResult)
{
	if (QosStats.IsValid())
	{
		if (CompletionResult != EQosCompletionResult::Canceled)
		{
			EDatacenterResultType ResultType = EDatacenterResultType::Failure;
			if (CompletionResult != EQosCompletionResult::Failure)
			{
				ResultType = EDatacenterResultType::Normal;
			}
			QosStats->EndQosPass(ResultType);
			QosStats->Upload(AnalyticsProvider);
		}

		QosStats = nullptr;
	}
}

UWorld* UQosEvaluator::GetWorld() const
{
	UWorld* World = ParentWorld.Get();
	check(World);
	return World;
}

FTimerManager& UQosEvaluator::GetWorldTimerManager() const
{
	return GetWorld()->GetTimerManager();
}

void UQosEvaluator::ResetDatacenterPingResults()
{
	for (FDatacenterQosInstance& Datacenter : Datacenters)
	{
		if (Datacenter.Definition.IsPingable())
		{
			Datacenter.Result = EQosDatacenterResult::Invalid;
		}
	}
}

TArray<FIcmpTarget>& UQosEvaluator::PopulatePingRequestList(TArray<FIcmpTarget>& OutTargets,
	const TArray<FDatacenterQosInstance>& Datacenters, int32 NumTestsPerRegion)
{
	for (const FDatacenterQosInstance& Datacenter : Datacenters)
	{
		if (Datacenter.Definition.IsPingable())
		{
			PopulatePingRequestList(OutTargets, Datacenter.Definition, NumTestsPerRegion);
		}
		else
		{
			UE_LOGF(LogQos, Verbose, "Datacenter disabled %ls", *Datacenter.Definition.ToDebugString());
		}
	}

	return OutTargets;
}

TArray<FIcmpTarget>& UQosEvaluator::PopulatePingRequestList(TArray<FIcmpTarget>& OutTargets,
	const FQosDatacenterInfo& DatacenterDefinition, int32 NumTestsPerRegion)
{
	const TArray<FQosPingServerInfo>& Servers = DatacenterDefinition.Servers;
	const int NumServers = Servers.Num();

	const int ServerStartIdx = FMath::RandHelper(NumServers);
	const int NumPings = NumTestsPerRegion;

	for (int PingIdx = 0; PingIdx < NumPings; ++PingIdx)
	{
		const int ServerIdx = (ServerStartIdx + PingIdx) % NumServers;
		const FQosPingServerInfo& Server = Servers[ServerIdx];

		FIcmpTarget Target(Server.Address, Server.Port);
		OutTargets.Add(Target);
	}

	return OutTargets;
}

FDatacenterQosInstance *const UQosEvaluator::FindDatacenterByAddress(TArray<FDatacenterQosInstance>& Datacenters,
	const FString& ServerAddress, int32 ServerPort)
{
	// Ugly O(n^2) search to find matching datacenter.  Maybe reverse-lookup table/set would be better here.
	for (FDatacenterQosInstance& Datacenter : Datacenters)
	{
		for (const FQosPingServerInfo& Server : Datacenter.Definition.Servers)
		{
			if ((Server.Address == ServerAddress) && (Server.Port == ServerPort))
			{
				return &Datacenter;
			}
		}
	}

	return nullptr;
}

bool UQosEvaluator::PingRegionServers(const FQosParams& InParams, const FOnQosSearchComplete& InQosSearchCompleteDelegate)
{
	const int32 NumTestsPerRegion = InParams.NumTestsPerRegion;

	TArray<FIcmpTarget> Targets;
	PopulatePingRequestList(Targets, Datacenters, NumTestsPerRegion);

	if (0 == Targets.Num())
	{
		// Nothing to do if no servers provided in the list of datacenters.
		InQosSearchCompleteDelegate.ExecuteIfBound(EQosCompletionResult::Failure, Datacenters, nullptr, nullptr);
		return false;
	}

	// Clear the ping results (i.e. set results as invalid) for all datacenters that can be pinged.
	ResetDatacenterPingResults();

	auto PingCompletionCallback = FIcmpEchoManyCompleteDelegate::CreateWeakLambda(this,
		[this, NumTestsPerRegion, InQosSearchCompleteDelegate](FIcmpEchoManyCompleteResult FinalResult)
	{
		// Process total data set from async task
		OnEchoManyCompleted(FinalResult, NumTestsPerRegion, InQosSearchCompleteDelegate);
	});

	FUDPPing::UDPEchoMany(Targets, InParams.Timeout, PingCompletionCallback);

	return true;
}

