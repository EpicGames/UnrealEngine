// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "QosRegionManager.h"
#include "QosEvaluator.generated.h"

#define UE_API QOS_API

class FQosDatacenterStats;
class FTimerManager;
class IAnalyticsProvider;
struct FIcmpEchoResult;
struct FIcmpTarget;
struct FIcmpEchoManyCompleteResult;
enum class EQosResponseType : uint8;

/**
 * Input parameters to start a qos ping check
 */
struct FQosParams
{
	/** Number of ping requests per region */
	int32 NumTestsPerRegion;
	/** Amount of time to wait for each request */
	float Timeout;
};

/*
 * Delegate triggered when an evaluation of ping for all servers in a search query have completed
 *
 * @param Result the ping operation result
 */
DECLARE_DELEGATE_OneParam(FOnQosPingEvalComplete, EQosCompletionResult /** Result */);

/** 
 * Delegate triggered when all QoS search results have been investigated 
 *
 * @param Result the QoS operation result
 * @param DatacenterInstances The per-datacenter ping information
 * @param OutSelectedRegion If present target will be set to the ReionManagers recommended region (may differ from actual best)
 * @param OutSelectedSubRegion If present target will be set to the ReionManagers recommended sub region (may differ from actual best)
 */
DECLARE_DELEGATE_FourParams(FOnQosSearchComplete, EQosCompletionResult /** Result */, const TArray<FDatacenterQosInstance>& /** DatacenterInstances */, FString*  /** OutSelectedRegion */, FString*  /** OutSelectedSubRegion */);


/**
 * Evaluates QoS metrics to determine the best datacenter under current conditions
 * Additionally capable of generically pinging an array of servers that have a QosBeaconHost active
 */
UCLASS(MinimalAPI, config = Engine)
class UQosEvaluator : public UObject
{
	GENERATED_UCLASS_BODY()

public:

	/**
	 * QoS services
	 */

	/**
	 * Find all the advertised datacenters and begin the process of evaluating ping results
	 * Will return the default datacenter in the event of failure or no advertised datacenters
	 *
	 * @param InParams parameters defining the 
	 * @param InRegions array of regions to query
	 * @param InDatacenters array of datacenters to query
	 * @param InCompletionDelegate delegate to fire when a datacenter choice has been made
	 */
	UE_API void FindDatacenters(const FQosParams& InParams, const TArray<FQosRegionInfo>& InRegions, const TArray<FQosDatacenterInfo>& InDatacenters, const FOnQosSearchComplete& InCompletionDelegate);

	/**
	 * Is a QoS operation active
	 *
	 * @return true if QoS is active, false otherwise
	 */
	bool IsActive() const { return bInProgress; }

	/**
	 * Cancel the current QoS operation at the earliest opportunity
	 */
	UE_API void Cancel();

	UE_API void SetWorld(UWorld* InWorld);

	bool IsCanceled() const { return bCancelOperation; }

protected:

	/**
	 * Use the udp ping code to ping known servers
	 *
	 * @param InParams parameters defining the request
	 * @param InQosSearchCompleteDelegate delegate to fire when all regions have completed their tests
	 */
	UE_API bool PingRegionServers(const FQosParams& InParams, const FOnQosSearchComplete& InQosSearchCompleteDelegate);

private:

	UE_API void ResetDatacenterPingResults();

	static UE_API TArray<FIcmpTarget>& PopulatePingRequestList(TArray<FIcmpTarget>& OutTargets,
		const TArray<FDatacenterQosInstance>& Datacenters, int32 NumTestsPerRegion);

	static UE_API TArray<FIcmpTarget>& PopulatePingRequestList(TArray<FIcmpTarget>& OutTargets,
		const FQosDatacenterInfo& DatacenterDefinition, int32 NumTestsPerRegion);

	static UE_API FDatacenterQosInstance *const FindDatacenterByAddress(TArray<FDatacenterQosInstance>& Datacenters,
		const FString& ServerAddress, int32 ServerPort);

	UE_API void OnEchoManyCompleted(FIcmpEchoManyCompleteResult FinalResult, int32 NumTestsPerRegion, const FOnQosSearchComplete& InQosSearchCompleteDelegate);

private:

	/** Reference to external UWorld */
	TWeakObjectPtr<UWorld> ParentWorld;

	FOnQosPingEvalComplete OnQosPingEvalComplete;

	/** Start time of total test */
	double StartTimestamp;
	/** A QoS operation is in progress */
	UPROPERTY()
	bool bInProgress;
	/** Should cancel occur at the next available opportunity */
	UPROPERTY()
	bool bCancelOperation;

	/** Array of datacenters currently being evaluated */
	UPROPERTY(Transient)
	TArray<FDatacenterQosInstance> Datacenters;

	/**
	 * @return true if all ping requests have completed (new method)
	 */
	UE_API bool AreAllRegionsComplete();

	/**
	 * Take all found ping results and process them before consumption at higher levels
	 *
	 * @param TimeToDiscount amount of time to subtract from calculation to compensate for external factors (frame rate, etc)
	 */
	UE_API void CalculatePingAverages(int32 TimeToDiscount = 0);

public:

	/**
	 * Analytics
	 */

	UE_API void SetAnalyticsProvider(TSharedPtr<IAnalyticsProvider> InAnalyticsProvider);

private:

	UE_API void StartAnalytics();
	UE_API void EndAnalytics(EQosCompletionResult CompletionResult);

	/** Reference to the provider to submit data to */
	TSharedPtr<IAnalyticsProvider> AnalyticsProvider;
	/** Stats related to these operations */
	TSharedPtr<FQosDatacenterStats> QosStats;

private:

	/**
	 * Helpers
	 */

	/** Quick access to the current world */
	UE_API UWorld* GetWorld() const;

	/** Quick access to the world timer manager */
	UE_API FTimerManager& GetWorldTimerManager() const;
};

inline const TCHAR* ToString(EQosDatacenterResult Result)
{
	switch (Result)
	{
		case EQosDatacenterResult::Invalid:
		{
			return TEXT("Invalid");
		}
		case EQosDatacenterResult::Success:
		{
			return TEXT("Success");
		}
		case EQosDatacenterResult::Incomplete:
		{
			return TEXT("Incomplete");
		}
	}

	return TEXT("");
}

inline const TCHAR* ToString(EQosCompletionResult Result)
{
	switch (Result)
	{
		case EQosCompletionResult::Invalid:
		{
			return TEXT("Invalid");
		}
		case EQosCompletionResult::Success:
		{
			return TEXT("Success");
		}
		case EQosCompletionResult::Failure:
		{
			return TEXT("Failure");
		}
		case EQosCompletionResult::Canceled:
		{
			return TEXT("Canceled");
		}
	}

	return TEXT("");
}

#undef UE_API
