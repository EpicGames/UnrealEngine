// Copyright Epic Games, Inc. All Rights Reserved.

#include "QosInterface.h"
#include "QosModule.h"
#include "QosRegionManager.h"

//static 
TSharedRef<FQosInterface> FQosInterface::Get()
{
	return FQosModule::Get().GetQosInterface();
}

FQosInterface::FQosInterface()
	: RegionManager(nullptr)
{
}

bool FQosInterface::Init()
{
	RegionManager = NewObject<UQosRegionManager>();
	return RegionManager != nullptr;
}

void FQosInterface::AddReferencedObjects(FReferenceCollector& Collector)
{
	if (RegionManager)
	{
		Collector.AddReferencedObject(RegionManager);
	}
}

// static
FString FQosInterface::GetDatacenterId()
{
	return UQosRegionManager::GetDatacenterId();
}

FString FQosInterface::GetAdvertisedSubregionId()
{
	return UQosRegionManager::GetAdvertisedSubregionId();
}

void FQosInterface::BeginQosEvaluation(UWorld* World, const TSharedPtr<IAnalyticsProvider>& AnalyticsProvider, const FSimpleDelegate& OnComplete)
{
	check(RegionManager);
	RegionManager->BeginQosEvaluation(World, AnalyticsProvider, OnComplete);
}

FQosInterface::FOnQosEvalCompleteDelegate& FQosInterface::OnQosEvalComplete()
{
	check(RegionManager);
	return RegionManager->OnQosEvalComplete();
}

bool FQosInterface::IsQosEvaluationInProgress() const
{
	check(RegionManager);
	return RegionManager->IsQosEvaluationInProgress();
}

FString FQosInterface::GetRegionId() const
{
	check(RegionManager);
	return RegionManager->GetRegionId();
}

FString FQosInterface::GetBestRegion() const
{
	check(RegionManager);
	return RegionManager->GetBestRegion();
}

bool FQosInterface::AllRegionsFound() const
{
	check(RegionManager);
	return RegionManager->AllRegionsFound();
}

const TArray<FRegionQosInstance>& FQosInterface::GetRegionOptions() const
{
	check(RegionManager);
	return RegionManager->GetRegionOptions();
}

void FQosInterface::GetSubregionPreferences(const FString& RegionId, TArray<FString>& OutSubregions) const
{
	check(RegionManager);
	return RegionManager->GetSubregionPreferences(RegionId, OutSubregions);
}

TMap<FString, int64> FQosInterface::GetAllSubregionPingsMap() const
{
	check(RegionManager);
	return RegionManager->GetAllSubregionPingsMap();
}

void FQosInterface::ForceSelectRegion(const FString& InRegionId)
{
	check(RegionManager);
	RegionManager->ForceSelectRegion(InRegionId);
}

bool FQosInterface::IsUsableRegion(const FString& InRegionId) const
{
	check(RegionManager);
	return RegionManager->IsUsableRegion(InRegionId);
}

bool FQosInterface::SetSelectedRegion(const FString& InRegionId)
{
	check(RegionManager);
	return RegionManager->SetSelectedRegion(InRegionId);
}

void FQosInterface::ClearSelectedRegion()
{
	check(RegionManager);
	RegionManager->ClearSelectedRegion();
}

void FQosInterface::DumpRegionStats()
{
	check(RegionManager);
	return RegionManager->DumpRegionStats();
}

void FQosInterface::RegisterQoSSettingsChangedDelegate(const FSimpleDelegate& OnQoSSettingsChanged)
{
	check(RegionManager);
	RegionManager->RegisterQoSSettingsChangedDelegate(OnQoSSettingsChanged);
}

FQosInterface::FOnQosRegionIdChanged& FQosInterface::OnQosRegionIdChanged()
{
	check(RegionManager);
	return RegionManager->OnQosRegionIdChanged();
}
