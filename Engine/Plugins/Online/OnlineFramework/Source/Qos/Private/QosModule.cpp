// Copyright Epic Games, Inc. All Rights Reserved.

#include "QosModule.h"
#include "QosInterface.h"


IMPLEMENT_MODULE(FQosModule, Qos);

DEFINE_LOG_CATEGORY(LogQos);

void FQosModule::StartupModule()
{
}

void FQosModule::ShutdownModule()
{
	ensure(!QosInterface.IsValid() || QosInterface.IsUnique());
	QosInterface = nullptr;
}

TSharedRef<FQosInterface> FQosModule::GetQosInterface()
{
	if (!QosInterface.IsValid())
	{
		QosInterface = MakeShareable(new FQosInterface());
		QosInterface->Init();
	}
	return QosInterface.ToSharedRef();
}

bool FQosModule::Exec_Runtime(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
{
	bool bWasHandled = false;

	// Ignore any execs that don't start with Qos
	if (FParse::Command(&Cmd, TEXT("Qos")))
	{
		if (FParse::Command(&Cmd, TEXT("Ping")))
		{
			GetQosInterface()->BeginQosEvaluation(InWorld, nullptr, FSimpleDelegate::CreateLambda([this]()
			{
				UE_LOGF(LogQos, Log, "ExecQosPingComplete!");
				GetQosInterface()->DumpRegionStats();
			}));
			bWasHandled = true;
		}
		else if (FParse::Command(&Cmd, TEXT("DumpRegions")))
		{
			GetQosInterface()->DumpRegionStats();
			bWasHandled = true;
		}
	}

	return bWasHandled;
}


