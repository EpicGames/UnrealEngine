// Copyright Epic Games, Inc. All Rights Reserved.

#include "QosRegionManager.h"

#include "Algo/Transform.h"
#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/TrackedActivity.h"
#include "QosInterface.h"
#include "TimerManager.h"
#include "Engine/World.h"
#include "QosEvaluator.h"
#include "QosModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(QosRegionManager)

#define LAST_REGION_EVALUATION 3

namespace QosCvars
{
	static bool bIncludeAllSubregionPreferences = true;
	static FAutoConsoleVariableRef CVarbIncludeAllSubregionPreferences(
		TEXT("QOS.IncludeAllSubregionPreferences"),
		bIncludeAllSubregionPreferences,
		TEXT("Whether requesting subregions includes all subregions regardless of parent region.\n")
		TEXT("true Enables. false disables."),
		ECVF_Default);
}

bool FDatacenterQosInstance::IsLessWhenBiasedTowardsNonSubspace(
	const FDatacenterQosInstance& A,
	const FDatacenterQosInstance& B,
	const FQosSubspaceComparisonParams& ComparisonParams,
	const TCHAR* const SubspaceDelimiter)
{
	if (A.Definition.IsSubspace(SubspaceDelimiter) == B.Definition.IsSubspace(SubspaceDelimiter))
	{
		// Comparing like-for-like, so use normal ordering by average ping.
		return A.AvgPingMs < B.AvgPingMs;
	}

	// Comparing subspace vs non-subspace, or vice-versa.
	const bool bAIsSubspace = A.Definition.IsSubspace(SubspaceDelimiter);
	const FDatacenterQosInstance& Subspace = bAIsSubspace ? A : B;
	const FDatacenterQosInstance& NonSubspace = bAIsSubspace ? B : A;
	const bool bIsNonSubspaceRecommended = IsNonSubspaceRecommended(NonSubspace, Subspace, ComparisonParams);

	const bool bResult = bIsNonSubspaceRecommended != bAIsSubspace;
	return bResult;
}

bool FDatacenterQosInstance::IsNonSubspaceRecommended(
	const FDatacenterQosInstance& NonSubspace,
	const FDatacenterQosInstance& Subspace,
	const FQosSubspaceComparisonParams& ComparisonParams)
{
	const int32 NonSubspacePingMs = NonSubspace.AvgPingMs;
	const int32 SubspacePingMs = Subspace.AvgPingMs;

	if (NonSubspacePingMs < SubspacePingMs)
	{
		// Edge case where non-subspace is faster, recommend it immediately.
		return true;
	}

	const int32 PingMsDelta = NonSubspacePingMs - SubspacePingMs;

	const float ScaledMaxToleranceMs = ComparisonParams.CalcScaledMaxToleranceMs(NonSubspacePingMs);

	// Test if non-subspace qualifies over subspace, via rules-based comparison that biases towards non-subspace.
	bool bDisqualify = (ComparisonParams.MaxNonSubspacePingMs > 0) && (NonSubspacePingMs > ComparisonParams.MaxNonSubspacePingMs);
	bDisqualify = bDisqualify || ((ComparisonParams.MinSubspacePingMs > 0) && (SubspacePingMs < ComparisonParams.MinSubspacePingMs));
	bDisqualify = bDisqualify || ((ComparisonParams.ConstantMaxToleranceMs > 0) && (PingMsDelta > ComparisonParams.ConstantMaxToleranceMs));
	bDisqualify = bDisqualify || ((ComparisonParams.ScaledMaxTolerancePct > 0.0f) && (static_cast<float>(PingMsDelta) > ScaledMaxToleranceMs));

	return !bDisqualify;
}

const FDatacenterQosInstance& FDatacenterQosInstance::CompareBiasedTowardsNonSubspace(
	const FDatacenterQosInstance& NonSubspace, const FDatacenterQosInstance& Subspace,
	const FQosSubspaceComparisonParams& ComparisonParams)
{
	const bool bPreferNonSubspace = IsNonSubspaceRecommended(NonSubspace, Subspace, ComparisonParams);
	return bPreferNonSubspace ? NonSubspace : Subspace;
}

const FDatacenterQosInstance* const FDatacenterQosInstance::CompareBiasedTowardsNonSubspace(
	const FDatacenterQosInstance* const NonSubspace, const FDatacenterQosInstance* const Subspace,
	const FQosSubspaceComparisonParams& ComparisonParams)
{
	check(NonSubspace);
	check(Subspace);
	const bool bPreferNonSubspace = IsNonSubspaceRecommended(*NonSubspace, *Subspace, ComparisonParams);
	return bPreferNonSubspace ? NonSubspace : Subspace;
}

EQosDatacenterResult FRegionQosInstance::GetRegionResult() const
{
	EQosDatacenterResult Result = EQosDatacenterResult::Success;
	for (const FDatacenterQosInstance& Datacenter : DatacenterOptions)
	{
		if (Datacenter.Result == EQosDatacenterResult::Invalid)
		{
			Result = EQosDatacenterResult::Invalid;
			break;
		}
		if (Datacenter.Result == EQosDatacenterResult::Incomplete)
		{
			Result = EQosDatacenterResult::Incomplete;
			break;
		}
	}

	return Result;
}

int32 FRegionQosInstance::GetBestAvgPing() const
{
	int32 BestPing = UNREACHABLE_PING;
	if (DatacenterOptions.Num() > 0)
	{
		// Presorted for best result first
		BestPing = DatacenterOptions[0].AvgPingMs;
	}

	return BestPing;
}

FString FRegionQosInstance::GetBestSubregion() const
{
	FString BestDatacenterId;
	if (DatacenterOptions.Num() > 0)
	{
		// Presorted for best/recommended result first
		BestDatacenterId = DatacenterOptions[0].Definition.Id;
	}

	return BestDatacenterId;
}

void FRegionQosInstance::GetSubregionPreferences(TArray<FString>& OutSubregions) const
{
#if !UE_BUILD_SHIPPING
	FString OverrideSubregion;
	if (FParse::Value(FCommandLine::Get(), TEXT("McpSubregion="), OverrideSubregion))
	{
		// Subregion specified on command line
		OutSubregions.Add(OverrideSubregion);
		UE_LOGF(LogQos, Verbose, "[FRegionQosInstance::GetSubregionPreferences] overriding subregion to %ls", *OverrideSubregion);
		return;
	}
#endif

	// Presorted for best/recommended result first
	for (const FDatacenterQosInstance& Option : DatacenterOptions)
	{
		OutSubregions.Add(Option.Definition.Id);
	}
}

void FRegionQosInstance::SortDatacenterOptionsByAvgPingAsc()
{
	UE_LOGF(LogQos, VeryVerbose, "[FRegionQosInstance::SortDatacenterOptionsByAvgPingAsc] sorting datacenter results for region %ls", *GetRegionId());

	// Sort ping best to worst (ascending ping ms)
	DatacenterOptions.Sort(
		[](const FDatacenterQosInstance& A, const FDatacenterQosInstance& B) {
			return A.AvgPingMs < B.AvgPingMs;
		}
	);
}

void FRegionQosInstance::SortDatacenterSubspacesByRecommended(const FQosSubspaceComparisonParams& ComparisonParams, const TCHAR* const SubspaceDelimiter)
{
	UE_LOGF(LogQos, VeryVerbose, "[FRegionQosInstance::SortDatacenterSubspacesByRecommended] sorting datacenter results for region %ls", *GetRegionId());

	// Sort by subspace recommendation rules
	DatacenterOptions.Sort(
		[&ComparisonParams, &SubspaceDelimiter](const FDatacenterQosInstance& A, const FDatacenterQosInstance& B) {
			return FDatacenterQosInstance::IsLessWhenBiasedTowardsNonSubspace(A, B, ComparisonParams, SubspaceDelimiter);
		}
	);
}

void FRegionQosInstance::LogDatacenterResults() const
{
	UE_LOGF(LogQos, VeryVerbose, "Region %ls:", *GetRegionId());

	for (const FDatacenterQosInstance& Datacenter : DatacenterOptions)
	{
		UE_LOGF(LogQos, VeryVerbose, "  %ls  avg ping: %dms, num responses: %d", *Datacenter.Definition.Id, Datacenter.AvgPingMs, Datacenter.NumResponses);
	}
}


const TCHAR* UQosRegionManager::SubspaceDelimiterDefault = TEXT("_");

UQosRegionManager::UQosRegionManager(const FObjectInitializer& ObjectInitializer)
	: NumTestsPerRegion(3)
	, PingTimeout(5.0f)
	, bEnableSubspaceBiasOrder(false)
	, SubspaceDelimiter(SubspaceDelimiterDefault)
	, LastCheckTimestamp(0)
	, Evaluator(nullptr)
	, QosEvalResult(EQosCompletionResult::Invalid)
{
	check(GConfig);
	GConfig->GetString(TEXT("Qos"), TEXT("ForceRegionId"), ForceRegionId, GEngineIni);

	// get a forced region id from the command line as an override
	bRegionForcedViaCommandline = FParse::Value(FCommandLine::Get(), TEXT("McpRegion="), ForceRegionId);
	if (!ForceRegionId.IsEmpty())
	{
		ForceRegionId.ToUpperInline();
	}

	// Will add an info entry to the console
	static FTrackedActivity Ta(TEXT("McpRegion"), *ForceRegionId, FTrackedActivity::ELight::None, FTrackedActivity::EType::Info);
}

void UQosRegionManager::PostReloadConfig(FProperty* PropertyThatWasLoaded)
{
	UE_LOGF(LogQos, Log, "[UQosRegionManager::PostReloadConfig] begin");
	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		for (int32 RegionIdx = RegionOptions.Num() - 1; RegionIdx >= 0; RegionIdx--)
		{
			FRegionQosInstance& RegionOption = RegionOptions[RegionIdx];

			bool bFound = false;
			for (FQosRegionInfo& RegionDef : RegionDefinitions)
			{
				if (RegionDef.RegionId == RegionOption.Definition.RegionId)
				{
					bFound = true;
				}
			}

			UE_LOGF(LogQos, Log, "[UQosRegionManager::PostReloadConfig] RegionId '%ls' found in RegionDefinitions? %d", *RegionOption.Definition.RegionId, bFound);
			if (!bFound)
			{
				// Old value needs to be removed, preserve order
				RegionOptions.RemoveAt(RegionIdx);
			}
		}

		for (int32 RegionIdx = 0; RegionIdx < RegionDefinitions.Num(); RegionIdx++)
		{
			FQosRegionInfo& RegionDef = RegionDefinitions[RegionIdx];

			bool bFound = false;
			for (FRegionQosInstance& RegionOption : RegionOptions)
			{
				if (RegionDef.RegionId == RegionOption.Definition.RegionId)
				{
					// Overwrite the metadata
					RegionOption.Definition = RegionDef;
					bFound = true;
				}
			}

			UE_LOGF(LogQos, Log, "[UQosRegionManager::PostReloadConfig] RegionId '%ls' found in RegionOptions? %d", *RegionDef.RegionId, bFound);

			if (!bFound)
			{
				// Add new value not in old list
				FRegionQosInstance NewRegion(RegionDef);
				RegionOptions.Insert(NewRegion, RegionIdx);
			}
		}

		UE_LOGF(LogQos, Log, "[UQosRegionManager::PostReloadConfig] firing OnQoSSettingsChanged delegate");
		OnQoSSettingsChangedDelegate.ExecuteIfBound();

		// Validate the current region selection (skipped if a selection has never been attempted)
		if (QosEvalResult != EQosCompletionResult::Invalid)
		{
			TrySetDefaultRegion();
		}

		RefreshUsableSubregions();
		SanityCheckDefinitions();
	}
}

int32 UQosRegionManager::GetMaxPingMs() const
{
	int32 MaxPing = -1;
	if (GConfig->GetInt(TEXT("Qos"), TEXT("MaximumPingMs"), MaxPing, GEngineIni) && MaxPing > 0)
	{
		return MaxPing;
	}
	return -1;
}

bool UQosRegionManager::IsSubspaceBiasOrderEnabled() const
{
	// Command-line override, if specified.
	bool bEnabledCommandLine = false;
	if (FParse::Bool(FCommandLine::Get(), TEXT("qossubspacebias="), bEnabledCommandLine))
	{
		UE_LOGF(LogQos, VeryVerbose, "[UQosRegionManager::IsSubspaceBiasOrderEnabled] overridden by command-line to: %ls", *LexToString(bEnabledCommandLine));
		return bEnabledCommandLine;
	}

	UE_LOGF(LogQos, VeryVerbose, "[UQosRegionManager::IsSubspaceBiasOrderEnabled] set in ini or defaulted to: %ls", *LexToString(bEnableSubspaceBiasOrder));

	// Use config value (or internal code default)
	return bEnableSubspaceBiasOrder;
}

bool UQosRegionManager::IsSubspaceBiasOrderEnabled(const FQosRegionInfo& RegionDefinition) const
{
	return IsSubspaceBiasOrderEnabled() && RegionDefinition.bAllowSubspaceBias;
}

void UQosRegionManager::RefreshUsableSubregions()
{
	TArray<const FDatacenterQosInstance*> UsableDatacentersByPing;
	UsableDatacentersByPing.Reserve(DatacenterDefinitions.Num());
	for (const FRegionQosInstance& RegionInstance : RegionOptions)
	{
		if (RegionInstance.IsUsable())
		{
			for (const FDatacenterQosInstance& DatacenterInstance : RegionInstance.DatacenterOptions)
			{
				UsableDatacentersByPing.Add(&DatacenterInstance);
			}
		}
	}

	UsableDatacentersByPing.Sort(
		[&SubspaceBiasParams = SubspaceBiasParams, SubspaceDelimiter = GetSubspaceDelimiter()](const FDatacenterQosInstance& A, const FDatacenterQosInstance& B) {
			return FDatacenterQosInstance::IsLessWhenBiasedTowardsNonSubspace(A, B, SubspaceBiasParams, SubspaceDelimiter);
		});

	UsableSubregions.Empty(UsableDatacentersByPing.Num());
	Algo::Transform(UsableDatacentersByPing, UsableSubregions,
		[](const FDatacenterQosInstance* Datacenter)
		{
			return Datacenter->Definition.Id;
		});

	UE_LOGF(LogQos, Log, "[UQosRegionManager::RefreshUsableSubregions] Refreshed subreion list. [%ls]", *FString::Join(UsableSubregions, TEXT(",")));
}

// static
FString UQosRegionManager::GetDatacenterId()
{
	struct FDcidInfo
	{
		FDcidInfo()
		{
			FString OverrideDCID;
			if (FParse::Value(FCommandLine::Get(), TEXT("DCID="), OverrideDCID))
			{
				// DCID specified on command line
				DCIDString = OverrideDCID.ToUpper();
			}
			else
			{
				FString DefaultDCID;
				check(GConfig);
				if (GConfig->GetString(TEXT("Qos"), TEXT("DCID"), DefaultDCID, GEngineIni))
				{
					// DCID specified in ini file
					DCIDString = DefaultDCID.ToUpper();
				}
			}
		}

		FString DCIDString;
	};

	// TODO (EvanK): making this not static (and thus removing the only-once optimization) due to forked servers
	//               not being able to change their DCID after the fork point. will add it back in after we can
	//               deprecate the static versions of the getters.
	FDcidInfo DCID;
	return DCID.DCIDString;
}

FString UQosRegionManager::GetAdvertisedSubregionId()
{
	struct FSubregion
	{
		FSubregion()
		{
			FString OverrideSubregion;
			if (FParse::Value(FCommandLine::Get(), TEXT("McpSubregion="), OverrideSubregion))
			{
				// Subregion specified on command line
				SubregionString = OverrideSubregion.ToUpper();
			}
			else
			{
				FString DefaultSubregion;
				check(GConfig);
				if (GConfig->GetString(TEXT("Qos"), TEXT("McpSubregion"), DefaultSubregion, GEngineIni))
				{
					// DCID specified in ini file
					SubregionString = DefaultSubregion.ToUpper();
				}
			}
		}

		FString SubregionString;
	};

	// TODO (EvanK): making this not static (and thus removing the only-once optimization) due to forked servers
	//               not being able to change their subregion after the fork point. will add it back in after we can
	//               deprecate the static versions of the getters.
	FSubregion Subregion;
	return Subregion.SubregionString;
}

void UQosRegionManager::BeginQosEvaluation(UWorld* World, const TSharedPtr<IAnalyticsProvider>& AnalyticsProvider, const FSimpleDelegate& OnComplete)
{
	check(World);

	UE_LOGF(LogQos, Log, "[UQosRegionManager::BeginQosEvaluation] starting!");
	// There are valid cached results, use them
	if ((RegionOptions.Num() > 0) &&
		(QosEvalResult == EQosCompletionResult::Success) &&
		(FDateTime::UtcNow() - LastCheckTimestamp).GetTotalSeconds() <= LAST_REGION_EVALUATION)
	{
		UE_LOGF(LogQos, Log, "[UQosRegionManager::BeginQosEvaluation] cached results are valid (from %.2f sec ago), using those", (FDateTime::UtcNow() - LastCheckTimestamp).GetTotalSeconds());
		World->GetTimerManager().SetTimerForNextTick(FTimerDelegate::CreateLambda([OnComplete]()
		{
			OnComplete.ExecuteIfBound();
		}));
		return;
	}

	// add to the completion delegate
	OnQosEvalCompleteDelegate.Add(OnComplete);

	// if we're already evaluating, simply return
	if (Evaluator == nullptr)
	{
		UE_LOGF(LogQos, Log, "[UQosRegionManager::BeginQosEvaluation] no eval in progress, creating new evaluator");
		// create a new evaluator and start the process of running
		Evaluator = NewObject<UQosEvaluator>();
		Evaluator->AddToRoot();
		Evaluator->SetWorld(World);
		Evaluator->SetAnalyticsProvider(AnalyticsProvider);

		FQosParams Params;
		Params.NumTestsPerRegion = NumTestsPerRegion;
		Params.Timeout = PingTimeout;
		Evaluator->FindDatacenters(Params, RegionDefinitions, DatacenterDefinitions, FOnQosSearchComplete::CreateUObject(this, &UQosRegionManager::OnQosEvaluationComplete));
	}
}

bool UQosRegionManager::IsQosEvaluationInProgress() const
{
	return Evaluator != nullptr;
}

void UQosRegionManager::OnQosEvaluationComplete(EQosCompletionResult Result, const TArray<FDatacenterQosInstance>& DatacenterInstances, FString* OutSelectedRegion, FString* OutSelectedSubRegion)
{
	// toss the evaluator
	if (Evaluator != nullptr)
	{
		Evaluator->RemoveFromRoot();
		Evaluator->MarkAsGarbage();
		Evaluator = nullptr;
	}
	QosEvalResult = Result;
	RegionOptions.Empty(RegionDefinitions.Num());

	TMultiMap<FString, FDatacenterQosInstance> DatacenterMap;
	for (const FDatacenterQosInstance& Datacenter : DatacenterInstances)
	{
		DatacenterMap.Add(Datacenter.Definition.RegionId, Datacenter);
	}

	const bool bSubspaceBiasEnabled = IsSubspaceBiasOrderEnabled();
	const TCHAR* const Delim = GetSubspaceDelimiter();

	UE_LOGF(LogQos, Verbose, "[UQosRegionManager::OnQosEvaluationComplete] bSubspaceBiasEnabled=%ls", *LexToString(bSubspaceBiasEnabled));

#if DEBUG_SUBCOMPARE_BY_SUBSPACE
	// Test example regions (indepedent of real obtained QoS data)
	{
		TestCompareDatacentersBySubspace();
		TestSortDatacenterSubspacesByRecommended();

		FRegionQosInstance TestRegion = TestCreateExampleRegionResult();
		TestRegion.SortDatacenterSubspacesByRecommended(TestRegion.Definition.SubspaceBiasParams, TEXT("_"));
	}
#endif // DEBUG_SUBCOMPARE_BY_SUBSPACE

	for (const FQosRegionInfo& RegionInfo : RegionDefinitions)
	{
		if (RegionInfo.IsPingable())
		{
			if (DatacenterMap.Num(RegionInfo.RegionId))
			{
				// Build region options from datacenter details
				FRegionQosInstance* NewRegion = new (RegionOptions) FRegionQosInstance(RegionInfo);
				DatacenterMap.MultiFind(RegionInfo.RegionId, NewRegion->DatacenterOptions);

				const bool bSubspaceBias = bSubspaceBiasEnabled && RegionInfo.bAllowSubspaceBias;

				UE_LOGF(LogQos, VeryVerbose, "[UQosRegionManager::OnQosEvaluationComplete] pre-sort results for region=%ls, bias=%ls",
					*RegionInfo.RegionId, *LexToString(bSubspaceBias));
				NewRegion->LogDatacenterResults();

				if (bSubspaceBias)
				{
					// Use ordering that applies rules-based comparison for subspace vs non-subspace,
					// and avg ping when comparing like-for-like.
					const FQosSubspaceComparisonParams ComparisonParams = RegionInfo.SubspaceBiasParams;
					NewRegion->SortDatacenterSubspacesByRecommended(ComparisonParams, Delim);
				}
				else
				{
					// Use regular sorting of subregions, by ascendering order of avg ping.
					NewRegion->SortDatacenterOptionsByAvgPingAsc();
				}

				UE_LOGF(LogQos, VeryVerbose, "[UQosRegionManager::OnQosEvaluationComplete] post-sort results for region=%ls, bias=%ls",
					*RegionInfo.RegionId, *LexToString(bSubspaceBias));
				NewRegion->LogDatacenterResults();
			}
			else
			{
				UE_LOGF(LogQos, Warning, "No datacenters for region %ls", *RegionInfo.RegionId);
			}
		}
	}

	RefreshUsableSubregions();
	LastCheckTimestamp = FDateTime::UtcNow();

	if (!SelectedRegionId.IsEmpty() && SelectedRegionId == NO_REGION)
	{
		// Put the dev region back into the list and select it
		ForceSelectRegion(SelectedRegionId);
	}

	// treat lack of any regions as a failure
	if (RegionOptions.Num() <= 0)
	{
		QosEvalResult = EQosCompletionResult::Failure;
	}

	if (QosEvalResult == EQosCompletionResult::Success ||
		QosEvalResult == EQosCompletionResult::Failure)
	{
		if (RegionOptions.Num() > 0)
		{
			// Try to set something regardless of Qos result
			TrySetDefaultRegion();
		}
	}
	FString BestRegion = GetBestRegion();
	TArray<FString> BestRegionSubregions;
	GetSubregionPreferences(BestRegion, BestRegionSubregions);
	UE_LOGF(LogQos, Log, "[UQosRegionManager::OnQosEvaluationComplete] ping eval has completed.  Best region is '%ls', recommended subregion is '%ls'",
		*BestRegion, BestRegionSubregions.Num() ? *BestRegionSubregions[0] : TEXT("NONE"));

	if (OutSelectedRegion)
	{
		*OutSelectedRegion = BestRegion;
	}
	
	if (OutSelectedSubRegion)
	{
		*OutSelectedSubRegion = BestRegionSubregions.Num() ? *BestRegionSubregions[0] : FString();
	}

#if DEBUG_SUBCOMPARE_BY_SUBSPACE
	DumpRegionStats();
#endif // DEBUG_SUBCOMPARE_BY_SUBSPACE

	OnQosEvalCompleteDelegate.Broadcast();
	OnQosEvalCompleteDelegate.Clear();
}

FString UQosRegionManager::GetRegionId() const
{
	if (!ForceRegionId.IsEmpty())
	{
		//UE_LOGF(LogQos, VeryVerbose, "[UQosRegionManager::GetRegionId] Force region: \"%ls\"", *ForceRegionId);

		// we may have updated INI to bypass this process
		return ForceRegionId;
	}

	if (QosEvalResult == EQosCompletionResult::Invalid)
	{
		UE_LOGF(LogQos, VeryVerbose, "[UQosRegionManager::GetRegionId] No QoS result: \"%ls\"", NO_REGION);

		// if we haven't run the evaluator just use the region from settings
		// development dedicated server will come here, live services should use -mcpregion
		return NO_REGION;
	}

	if (SelectedRegionId.IsEmpty())
	{
		// Always set some kind of region, empty implies "wildcard" to the matchmaking code
		UE_LOGF(LogQos, Verbose, "No region currently set.");
		return NO_REGION;
	}

	UE_LOGF(LogQos, VeryVerbose, "[UQosRegionManager::GetRegionId] Selected region: \"%ls\"", *SelectedRegionId);
	return SelectedRegionId;
}

const TCHAR* UQosRegionManager::GetSubspaceDelimiter() const
{
	return SubspaceDelimiter.IsEmpty() ? SubspaceDelimiterDefault : *SubspaceDelimiter;
}

const FRegionQosInstance* UQosRegionManager::FindQosRegionById(const TArray<FRegionQosInstance>& Regions, const FString& RegionId)
{
	const FRegionQosInstance* Found = Regions.FindByPredicate(
		[&RegionId](const FRegionQosInstance& Elem) {
			return Elem.Definition.RegionId == RegionId;
		}
	);

	return Found;
}

const FRegionQosInstance* UQosRegionManager::FindBestQosRegion(const TArray<FRegionQosInstance>& Regions)
{
	int32 LowestRegionPing = UNREACHABLE_PING;
	const FRegionQosInstance* BestRegion = nullptr;

	// "Best" average ping is the avg ping of the first datacenter in each region's list of
	// datacenter options.  Assumes that the datacenter options list is pre-sorted as needed.
	for (const FRegionQosInstance& Region : Regions)
	{
		if (Region.IsAutoAssignable() && Region.GetBestAvgPing() < LowestRegionPing)
		{
			LowestRegionPing = Region.GetBestAvgPing();
			BestRegion = &Region;
		}
	}

	return BestRegion;
}

FString UQosRegionManager::GetBestRegion() const
{
	if (!ForceRegionId.IsEmpty())
	{
		UE_LOGF(LogQos, Verbose, "[UQosRegionManager::GetBestRegion] Force region: %ls", *ForceRegionId);

		return ForceRegionId;
	}

	const TArray<FRegionQosInstance>& LocalRegionOptions = GetRegionOptions();
	const FRegionQosInstance* BestRegion = FindBestQosRegion(LocalRegionOptions);
	const FString BestRegionId = BestRegion ? BestRegion->Definition.RegionId : FString();

	UE_LOGF(LogQos, Verbose, "[UQosRegionManager::GetBestRegion] Best region: \"%ls\"  (Current selected: \"%ls\")",
		*BestRegionId, *SelectedRegionId);

	return BestRegionId;
}

void UQosRegionManager::GetSubregionPreferences(const FString& RegionId, TArray<FString>& OutSubregions) const
{
	if (QosCvars::bIncludeAllSubregionPreferences && !bRegionForcedViaCommandline)
	{
		OutSubregions = UsableSubregions;
		return;
	}

	const TArray<FRegionQosInstance>& LocalRegionOptions = GetRegionOptions();
	for (const FRegionQosInstance& Region : LocalRegionOptions)
	{
		if (Region.Definition.RegionId == RegionId)
		{
			Region.GetSubregionPreferences(OutSubregions);
			break;
		}
	}
}

TMap<FString, int64> UQosRegionManager::GetAllSubregionPingsMap() const
{
	TMap<FString, int64> Pings;
	for (const FRegionQosInstance& Region : RegionOptions)
	{
		for (const FDatacenterQosInstance& Option : Region.DatacenterOptions)
		{
			Pings.Add(Option.Definition.Id, Option.AvgPingMs);
		}
	}
	return Pings;
}

const TArray<FRegionQosInstance>& UQosRegionManager::GetRegionOptions() const
{
	if (ForceRegionId.IsEmpty())
	{
		return RegionOptions;
	}

	static TArray<FRegionQosInstance> ForcedRegionOptions;
	ForcedRegionOptions.Empty(1);
	for (const FRegionQosInstance& RegionOption : RegionOptions)
	{
		if (RegionOption.Definition.RegionId == ForceRegionId)
		{
			ForcedRegionOptions.Add(RegionOption);
		}
	}
#if !UE_BUILD_SHIPPING
	if (ForcedRegionOptions.Num() == 0)
	{
		FRegionQosInstance FakeRegionInfo;
		FakeRegionInfo.Definition.DisplayName =	NSLOCTEXT("MMRegion", "DevRegion", "Development");
		FakeRegionInfo.Definition.RegionId = ForceRegionId;
		FakeRegionInfo.Definition.bEnabled = true;
		FakeRegionInfo.Definition.bVisible = true;
		FakeRegionInfo.Definition.bAutoAssignable = false;
		FDatacenterQosInstance FakeDatacenterInfo;
		FakeDatacenterInfo.Result = EQosDatacenterResult::Success;
		FakeDatacenterInfo.AvgPingMs = 0;
		FakeRegionInfo.DatacenterOptions.Add(MoveTemp(FakeDatacenterInfo));
		ForcedRegionOptions.Add(MoveTemp(FakeRegionInfo));
	}
#endif
	return ForcedRegionOptions;
}

void UQosRegionManager::ForceSelectRegion(const FString& InRegionId)
{
	if (!bRegionForcedViaCommandline)
	{
		QosEvalResult = EQosCompletionResult::Success;
		ForceRegionId = InRegionId.ToUpper();

		UE_LOGF(LogQos, Verbose, "[UQosRegionManager::ForceSelectRegion] Force region: \"%ls\"", *ForceRegionId);

		// make sure we can select this region
		if (!SetSelectedRegion(ForceRegionId, true))
		{
			UE_LOGF(LogQos, Log, "Failed to force set region id %ls", *ForceRegionId);
			ForceRegionId.Empty();
		}
	}
	else
	{
		UE_LOGF(LogQos, Log, "Forcing region %ls skipped because commandline override used %ls", *InRegionId, *ForceRegionId);
	}
}

void UQosRegionManager::TrySetDefaultRegion()
{
	if (!IsRunningDedicatedServer())
	{
		const FString& RegionId = GetRegionId();
		UE_LOGF(LogQos, Verbose, "[UQosRegionManager::TrySetDefaultRegion] setting default from GetRegionId() (%ls)", *RegionId);

		// Try to set a default region if one hasn't already been selected
		if (!SetSelectedRegion(RegionId))
		{
			FString BestRegionId = GetBestRegion();
			UE_LOGF(LogQos, Verbose, "[UQosRegionManager::TrySetDefaultRegion] setting default from best (%ls)", *BestRegionId);

			if (!SetSelectedRegion(BestRegionId))
			{
				UE_LOGF(LogQos, Warning, "Unable to set a good region!  Wanted to set '%ls'; fall back to '%ls' failed; current selected: '%ls'",
					*GetRegionId(), *BestRegionId, *SelectedRegionId);
				DumpRegionStats();
			}
		}
	}
}

bool UQosRegionManager::IsUsableRegion(const FString& InRegionId) const
{
	const TArray<FRegionQosInstance>& LocalRegionOptions = GetRegionOptions();
	for (const FRegionQosInstance& RegionInfo : LocalRegionOptions)
	{
		if (RegionInfo.Definition.RegionId == InRegionId)
		{
			return RegionInfo.IsUsable();
		}
	}

	UE_LOGF(LogQos, Log, "IsUsableRegion: failed to find region id %ls", *InRegionId);
	return false;
}

bool UQosRegionManager::SetSelectedRegion(const FString& InRegionId, bool bForce)
{
	// make sure we've enumerated
	if (bForce || QosEvalResult == EQosCompletionResult::Success)
	{
		// make sure it's in the option list
		FString RegionId = InRegionId.ToUpper();

		const TArray<FRegionQosInstance>& LocalRegionOptions = GetRegionOptions();
		for (const FRegionQosInstance& RegionInfo : LocalRegionOptions)
		{
			if (RegionInfo.Definition.RegionId == RegionId)
			{
				if (RegionInfo.IsUsable())
				{
					FString OldRegionId(SelectedRegionId);
					SelectedRegionId = MoveTemp(RegionId);

					UE_LOGF(LogQos, Verbose, "[UQosRegionManager::SetSelectedRegion] Old: \"%ls\"  New: \"%ls\"  (force? %ls)",
						*OldRegionId, *SelectedRegionId, *LexToString(bForce));

					OnQosRegionIdChanged().Broadcast(OldRegionId, SelectedRegionId);
					return true;
				}
				else
				{
					return false;
				}
			}
		}
	}

	// can't select a region not in the options list (NONE is special, it means pick best)
	if (!InRegionId.IsEmpty() && (InRegionId != NO_REGION))
	{
		UE_LOGF(LogQos, Log, "SetSelectedRegion: failed to find region id %ls", *InRegionId);
	}
	return false;
}

void UQosRegionManager::ClearSelectedRegion()
{
	UE_LOGF(LogQos, Verbose, "[UQosRegionManager::ClearSelectedRegion] Selected region: \"%ls\"  Forced region: \"%ls\"  bRegionForcedViaCommandline=%ls",
		*SelectedRegionId, *ForceRegionId, *LexToString(bRegionForcedViaCommandline));

	// Do not default to NO_REGION
	SelectedRegionId.Empty();
	if (!bRegionForcedViaCommandline)
	{
		ForceRegionId.Empty();
	}
}

bool UQosRegionManager::AllRegionsFound() const
{
	int32 NumDatacenters = 0;
	for (const FQosDatacenterInfo& Datacenter : DatacenterDefinitions)
	{
		if (Datacenter.IsPingable())
		{
			++NumDatacenters;
		}
	}

	int32 NumDatacentersWithGoodResponses = 0;
	for (const FRegionQosInstance& Region : RegionOptions)
	{
		for (const FDatacenterQosInstance& Datacenter : Region.DatacenterOptions)
		{
			const bool bGoodPercentage = (((float)Datacenter.NumResponses / (float)NumTestsPerRegion) >= 0.5f);
			NumDatacentersWithGoodResponses += bGoodPercentage ? 1 : 0;
		}
	}

	return (NumDatacenters > 0) && (NumDatacentersWithGoodResponses > 0) && (NumDatacenters == NumDatacentersWithGoodResponses);
}

void UQosRegionManager::SanityCheckDefinitions() const
{
	// Check data syntax
	for (const FQosRegionInfo& Region : RegionDefinitions)
	{
		UE_CLOGF(!Region.IsValid(), LogQos, Warning, "Invalid QOS region entry!");
	}

	// Check data syntax
	for (const FQosDatacenterInfo& Datacenter : DatacenterDefinitions)
	{
		UE_CLOGF(!Datacenter.IsValid(), LogQos, Warning, "Invalid QOS datacenter entry!");
	}

	// Every datacenter maps to a parent region
	for (const FQosDatacenterInfo& Datacenter : DatacenterDefinitions)
	{
		bool bFoundParentRegion = false;
		for (const FQosRegionInfo& Region : RegionDefinitions)
		{
			if (Datacenter.RegionId == Region.RegionId)
			{
				bFoundParentRegion = true;
				break;
			}
		}

		if (!bFoundParentRegion)
		{
			UE_LOGF(LogQos, Warning, "Datacenter %ls has undefined parent region %ls", *Datacenter.Id, *Datacenter.RegionId);
		}
	}

	// Regions with no available datacenters
	for (const FQosRegionInfo& Region : RegionDefinitions)
	{
		int32 NumDatacenters = 0;
		int32 NumPingableDatacenters = 0;
		for (const FQosDatacenterInfo& Datacenter : DatacenterDefinitions)
		{
			if (Datacenter.RegionId == Region.RegionId)
			{
				NumDatacenters++;
				if (Datacenter.IsPingable())
				{
					NumPingableDatacenters++;
				}
			}
		}

		if (NumDatacenters == 0)
		{
			UE_LOGF(LogQos, Warning, "Region %ls has no datacenters", *Region.RegionId);
		}

		if (NumDatacenters > 0 && NumPingableDatacenters == 0)
		{
			UE_LOGF(LogQos, Warning, "Region %ls has %d datacenters, all disabled", *Region.RegionId, NumDatacenters);
		}
	}

	// Every auto assignable region has at least one auto assignable datacenter
	int32 NumAutoAssignableRegions = 0;
	for (const FQosRegionInfo& Region : RegionDefinitions)
	{
		if (Region.IsAutoAssignable())
		{
			int32 NumPingableDatacenters = 0;
			for (const FQosDatacenterInfo& Datacenter : DatacenterDefinitions)
			{
				if (Datacenter.RegionId == Region.RegionId)
				{
					if (Datacenter.IsPingable())
					{
						NumPingableDatacenters++;
					}
				}
			}

			if (NumPingableDatacenters)
			{
				NumAutoAssignableRegions++;
			}

			UE_LOGF(LogQos, Display, "AutoRegion %ls: %d datacenters available", *Region.RegionId, NumPingableDatacenters);
		}
	}

	// At least one region is auto assignable
	if (NumAutoAssignableRegions == 0)
	{
		UE_LOGF(LogQos, Warning, "No auto assignable regions available!");
	}
}

void UQosRegionManager::DumpRegionStats() const
{
	UE_LOGF(LogQos, Display, "Region Info:");
	UE_LOGF(LogQos, Display, "Current: %ls ", *SelectedRegionId);
	if (!ForceRegionId.IsEmpty())
	{
		UE_LOGF(LogQos, Display, "Forced: %ls ", *ForceRegionId);
	}

	TMultiMap<FString, const FQosDatacenterInfo*> DatacentersByRegion;
	for (const FQosDatacenterInfo& DatacenterDef : DatacenterDefinitions)
	{
		DatacentersByRegion.Emplace(DatacenterDef.RegionId, &DatacenterDef);
	}

	TMap<FString, const FRegionQosInstance* const> RegionInstanceByRegion;
	for (const FRegionQosInstance& Region : RegionOptions)
	{
		RegionInstanceByRegion.Emplace(Region.Definition.RegionId, &Region);
	}

	// Look at real region options here
	UE_LOGF(LogQos, Display, "Definitions:");
	for (const FQosRegionInfo& RegionDef : RegionDefinitions)
	{
		const FRegionQosInstance* const* RegionInst = RegionInstanceByRegion.Find(RegionDef.RegionId);

		TArray<const FQosDatacenterInfo*> OutValues;
		DatacentersByRegion.MultiFind(RegionDef.RegionId, OutValues);

		UE_LOGF(LogQos, Display, "\tRegion: %ls [%ls] (%d datacenters)", *RegionDef.DisplayName.ToString(), *RegionDef.RegionId, OutValues.Num());
		UE_LOGF(LogQos, Display, "\t Enabled: %d Visible: %d Beta: %d", RegionDef.bEnabled, RegionDef.bVisible, RegionDef.bAutoAssignable);

		TSet<FString> FoundSubregions;
		if (RegionInst)
		{
			for (const FDatacenterQosInstance& Datacenter : (*RegionInst)->DatacenterOptions)
			{
				for (const FQosDatacenterInfo* DatacenterDef : OutValues)
				{
					if (DatacenterDef->Id == Datacenter.Definition.Id)
					{
						FoundSubregions.Add(DatacenterDef->Id);
						float ResponsePercent = (static_cast<float>(Datacenter.NumResponses) / static_cast<float>(NumTestsPerRegion)) * 100.0f;
						UE_LOGF(LogQos, Display, "\t  Datacenter: %ls%ls %dms (%0.2f%%) %ls", 
							*DatacenterDef->Id, !DatacenterDef->bEnabled ? TEXT(" Disabled") : TEXT(""),
							Datacenter.AvgPingMs, ResponsePercent, ToString(Datacenter.Result)
						);
						break;
					}
				}
			}
		}

		for (const FQosDatacenterInfo* DatacenterDef : OutValues)
		{
			UE_CLOGF(!FoundSubregions.Contains(DatacenterDef->Id), LogQos, Display, "\t  Datacenter: %ls%ls", *DatacenterDef->Id, !DatacenterDef->bEnabled ? TEXT(" Disabled") : TEXT(""));
		}

		if (!RegionInst)
		{
			UE_LOGF(LogQos, Display, "No instances for region");
		}
	}

	UE_LOGF(LogQos, Display, "Results: %ls", ToString(QosEvalResult));

	SanityCheckDefinitions();
}

void UQosRegionManager::RegisterQoSSettingsChangedDelegate(const FSimpleDelegate& OnQoSSettingsChanged)
{
	UE_LOGF(LogQos, Log, "[UQosRegionManager::RegisterQoSSettingsChangedDelegate] delegate was replaced");
	// add to the completion delegate
	OnQoSSettingsChangedDelegate = OnQoSSettingsChanged;
}

#if DEBUG_SUBCOMPARE_BY_SUBSPACE

bool UQosRegionManager::TestCompareDatacentersBySubspace()
{
	const FQosSubspaceComparisonParams Invariants(300, 8, 25, 50.0f);

	// Parent is a non-subspace datacenter.
	FQosDatacenterInfo ParentDatacenter;
	ParentDatacenter.Id = TEXT("DE");
	ParentDatacenter.RegionId = TEXT("EU");
	ParentDatacenter.Servers.Add(FQosPingServerInfo{ "100.1.1.1", 2345 });
	ParentDatacenter.Servers.Add(FQosPingServerInfo{ "100.1.1.2", 2345 });
	ParentDatacenter.Servers.Add(FQosPingServerInfo{ "100.1.1.3", 2345 });

	// Child is a subspcae datacenter.
	FQosDatacenterInfo ChildDatacenter;
	ChildDatacenter.Id = TEXT("DE_S");
	ChildDatacenter.RegionId = TEXT("EU");
	ChildDatacenter.Servers.Add(FQosPingServerInfo{ "100.1.2.11", 2345 });
	ChildDatacenter.Servers.Add(FQosPingServerInfo{ "100.1.2.12", 2345 });
	ChildDatacenter.Servers.Add(FQosPingServerInfo{ "100.1.2.13", 2345 });

	// Test 1
	// Parent is faster, somehow (potential edge case)
	// Expected result: Pick parent. (override succeeds)
	{
		FDatacenterQosInstance ChildSubregion;
		ChildSubregion.Definition = ChildDatacenter;
		ChildSubregion.AvgPingMs = 68;

		FDatacenterQosInstance ParentSubregion;
		ParentSubregion.Definition = ParentDatacenter;
		ParentSubregion.AvgPingMs = 24;

		const FDatacenterQosInstance& RecommendedSubregion = FDatacenterQosInstance::CompareBiasedTowardsNonSubspace(
			ParentSubregion, ChildSubregion, Invariants);
		check(&RecommendedSubregion == &ParentSubregion);
	}

	// Test 2
	// Parent is too slow.
	// Expected result: Pick child. (override fails)
	{
		FDatacenterQosInstance ChildSubregion;
		ChildSubregion.Definition = ChildDatacenter;
		ChildSubregion.AvgPingMs = 48;

		FDatacenterQosInstance ParentSubregion;
		ParentSubregion.Definition = ParentDatacenter;
		ParentSubregion.AvgPingMs = 360;

		const FDatacenterQosInstance& RecommendedSubregion = FDatacenterQosInstance::CompareBiasedTowardsNonSubspace(
			ParentSubregion, ChildSubregion, Invariants);
		check(&RecommendedSubregion == &ChildSubregion);
	}

	// Test 3
	// Parent is fast, but child is faster than minimum ping
	// Expected result: Pick child. (override fails)
	{
		FDatacenterQosInstance ChildSubregion;
		ChildSubregion.Definition = ChildDatacenter;
		ChildSubregion.AvgPingMs = 6;

		FDatacenterQosInstance ParentSubregion;
		ParentSubregion.Definition = ParentDatacenter;
		ParentSubregion.AvgPingMs = 34;

		const FDatacenterQosInstance& RecommendedSubregion = FDatacenterQosInstance::CompareBiasedTowardsNonSubspace(
			ParentSubregion, ChildSubregion, Invariants);
		check(&RecommendedSubregion == &ChildSubregion);
	}

	// Test 4
	// Parent is fast enough for consideration, child is slower than minimum ping.
	// Difference in ping is GREATER THAN allowed constant tolerance in milliseconds.
	// Expected result: Pick child. (override fails)
	{
		FDatacenterQosInstance ChildSubregion;
		ChildSubregion.Definition = ChildDatacenter;
		ChildSubregion.AvgPingMs = 48;

		FDatacenterQosInstance ParentSubregion;
		ParentSubregion.Definition = ParentDatacenter;
		ParentSubregion.AvgPingMs = 78; // (78 - 48 = 30) > 25

		const FDatacenterQosInstance& RecommendedSubregion = FDatacenterQosInstance::CompareBiasedTowardsNonSubspace(
			ParentSubregion, ChildSubregion, Invariants);
		check(&RecommendedSubregion == &ChildSubregion);
	}

	// If the difference in ping is LESS THAN or EQUAL TO allowed constant tolerance
	// in milliseconds, then we test against proportional tolerance (below).

	// Test 5.1
	// Parent is fast enough for consideration, child is slower than minimum ping.
	// Difference in ping is within constant tolerance (i.e. would pass test 4).
	// Different in ping is GREATER THAN allowed proportional tolerance, as a %age of parent's ping.
	// Expected result: Pick child. (override fails)
	{
		FDatacenterQosInstance ChildSubregion;
		ChildSubregion.Definition = ChildDatacenter;
		ChildSubregion.AvgPingMs = 10;

		FDatacenterQosInstance ParentSubregion;
		ParentSubregion.Definition = ParentDatacenter;
		ParentSubregion.AvgPingMs = 24; // (24 - 10 = 14) > (0.5 * 24 = 12)

		const FDatacenterQosInstance& RecommendedSubregion = FDatacenterQosInstance::CompareBiasedTowardsNonSubspace(
			ParentSubregion, ChildSubregion, Invariants);
		check(&RecommendedSubregion == &ChildSubregion);
	}

	// Test 5.2
	// Parent is fast enough for consideration, child is slower than minimum ping.
	// Difference in ping is within constant tolerance (i.e. would pass test 4).
	// Different in ping is LESS THAN than allowed proportional tolerance, as a %age of parent's ping.
	// Expected result: Pick parent. (override succeeds)
	{
		FDatacenterQosInstance ChildSubregion;
		ChildSubregion.Definition = ChildDatacenter;
		ChildSubregion.AvgPingMs = 10;

		FDatacenterQosInstance ParentSubregion;
		ParentSubregion.Definition = ParentDatacenter;
		ParentSubregion.AvgPingMs = 16; // (16 - 10 = 6) < (0.5 * 16 = 8)

		const FDatacenterQosInstance& RecommendedSubregion = FDatacenterQosInstance::CompareBiasedTowardsNonSubspace(
			ParentSubregion, ChildSubregion, Invariants);
		check(&RecommendedSubregion == &ParentSubregion);
	}

	// Test 5.3
	// Parent is fast enough for consideration, child is slower than minimum ping.
	// Difference in ping is within constant tolerance (i.e. would pass test 4).
	// Different in ping is EQUAL TO allowed proportional tolerance, as a %age of parent's ping.
	// Expected result: Pick parent. (override succeeds)
	{
		FDatacenterQosInstance ChildSubregion;
		ChildSubregion.Definition = ChildDatacenter;
		ChildSubregion.AvgPingMs = 12;

		FDatacenterQosInstance ParentSubregion;
		ParentSubregion.Definition = ParentDatacenter;
		ParentSubregion.AvgPingMs = 24; // (24 - 12 = 12) == (0.5 * 24 = 12)

		const FDatacenterQosInstance& RecommendedSubregion = FDatacenterQosInstance::CompareBiasedTowardsNonSubspace(
			ParentSubregion, ChildSubregion, Invariants);
		check(&RecommendedSubregion == &ParentSubregion);
	}

	return true;
}

bool UQosRegionManager::TestSortDatacenterSubspacesByRecommended()
{
	const FQosSubspaceComparisonParams Invariants(300, 8, 25, 50.0f);

	// Example region

	FQosRegionInfo ExampleRegion;
	ExampleRegion.DisplayName = NSLOCTEXT("MMRegion", "Europe", "Europe");
	ExampleRegion.RegionId = TEXT("EU");

	FQosDatacenterInfo ParentDatacenter_DE;
	ParentDatacenter_DE.Id = TEXT("DE");
	ParentDatacenter_DE.RegionId = TEXT("EU");
	ParentDatacenter_DE.Servers.Add(FQosPingServerInfo{ "100.1.1.1", 2345 });
	ParentDatacenter_DE.Servers.Add(FQosPingServerInfo{ "100.1.1.2", 2345 });
	ParentDatacenter_DE.Servers.Add(FQosPingServerInfo{ "100.1.1.3", 2345 });

	FQosDatacenterInfo ChildDatacenter_DE_S1;
	ChildDatacenter_DE_S1.Id = TEXT("DE_S1");
	ChildDatacenter_DE_S1.RegionId = TEXT("EU");
	ChildDatacenter_DE_S1.Servers.Add(FQosPingServerInfo{ "100.2.1.11", 2345 });
	ChildDatacenter_DE_S1.Servers.Add(FQosPingServerInfo{ "100.2.1.12", 2345 });
	ChildDatacenter_DE_S1.Servers.Add(FQosPingServerInfo{ "100.2.1.13", 2345 });

	FQosDatacenterInfo ChildDatacenter_DE_S2;
	ChildDatacenter_DE_S2.Id = TEXT("DE_S2");
	ChildDatacenter_DE_S2.RegionId = TEXT("EU");
	ChildDatacenter_DE_S2.Servers.Add(FQosPingServerInfo{ "100.3.1.11", 2345 });
	ChildDatacenter_DE_S2.Servers.Add(FQosPingServerInfo{ "100.3.1.12", 2345 });
	ChildDatacenter_DE_S2.Servers.Add(FQosPingServerInfo{ "100.3.1.13", 2345 });

	FQosDatacenterInfo ParentDatacenter_FR;
	ParentDatacenter_FR.Id = TEXT("FR");
	ParentDatacenter_FR.RegionId = TEXT("EU");
	ParentDatacenter_FR.Servers.Add(FQosPingServerInfo{ "105.1.1.1", 2345 });
	ParentDatacenter_FR.Servers.Add(FQosPingServerInfo{ "105.1.1.2", 2345 });
	ParentDatacenter_FR.Servers.Add(FQosPingServerInfo{ "105.1.1.3", 2345 });

	FQosDatacenterInfo ParentDatacenter_GB;
	ParentDatacenter_GB.Id = TEXT("GB");
	ParentDatacenter_GB.RegionId = TEXT("EU");
	ParentDatacenter_GB.Servers.Add(FQosPingServerInfo{ "120.1.1.1", 2345 });
	ParentDatacenter_GB.Servers.Add(FQosPingServerInfo{ "120.1.1.2", 2345 });
	ParentDatacenter_GB.Servers.Add(FQosPingServerInfo{ "120.1.1.3", 2345 });

	FQosDatacenterInfo ChildDatacenter_GB_S;
	ChildDatacenter_GB_S.Id = TEXT("GB_S");
	ChildDatacenter_GB_S.RegionId = TEXT("EU");
	ChildDatacenter_GB_S.Servers.Add(FQosPingServerInfo{ "120.2.1.11", 2345 });
	ChildDatacenter_GB_S.Servers.Add(FQosPingServerInfo{ "120.2.1.12", 2345 });
	ChildDatacenter_GB_S.Servers.Add(FQosPingServerInfo{ "120.2.1.13", 2345 });

	const TCHAR* const SubspaceDelimiter = TEXT("_");

	// Test cases.

	FRegionQosInstance TestRegionQosResult;
	TestRegionQosResult.Definition = ExampleRegion;

	{
		FDatacenterQosInstance SubregionResult_DE;
		SubregionResult_DE.Definition = ParentDatacenter_DE;
		SubregionResult_DE.AvgPingMs = 56;

		FDatacenterQosInstance SubregionResult_DE_S1;
		SubregionResult_DE_S1.Definition = ChildDatacenter_DE_S1;
		SubregionResult_DE_S1.AvgPingMs = 48;

		FDatacenterQosInstance SubregionResult_DE_S2;
		SubregionResult_DE_S2.Definition = ChildDatacenter_DE_S2;
		SubregionResult_DE_S2.AvgPingMs = 36;

		FDatacenterQosInstance SubregionResult_FR;
		SubregionResult_FR.Definition = ParentDatacenter_FR;
		SubregionResult_FR.AvgPingMs = 74;

		FDatacenterQosInstance SubregionResult_GB;
		SubregionResult_GB.Definition = ParentDatacenter_GB;
		SubregionResult_GB.AvgPingMs = 92;

		FDatacenterQosInstance SubregionResult_GB_S;
		SubregionResult_GB_S.Definition = ChildDatacenter_GB_S;
		SubregionResult_GB_S.AvgPingMs = 84;

		TestRegionQosResult.DatacenterOptions.Empty();
		TestRegionQosResult.DatacenterOptions.Add(SubregionResult_DE);
		TestRegionQosResult.DatacenterOptions.Add(SubregionResult_DE_S1);
		TestRegionQosResult.DatacenterOptions.Add(SubregionResult_DE_S2);
		TestRegionQosResult.DatacenterOptions.Add(SubregionResult_FR);
		TestRegionQosResult.DatacenterOptions.Add(SubregionResult_GB);
		TestRegionQosResult.DatacenterOptions.Add(SubregionResult_GB_S);

		TestRegionQosResult.SortDatacenterSubspacesByRecommended(Invariants, SubspaceDelimiter);

		check(TestRegionQosResult.DatacenterOptions[0].Definition.Id == SubregionResult_DE.Definition.Id); // avg ping = 56
		check(TestRegionQosResult.DatacenterOptions[1].Definition.Id == SubregionResult_DE_S2.Definition.Id); // avg ping = 36
		check(TestRegionQosResult.DatacenterOptions[2].Definition.Id == SubregionResult_DE_S1.Definition.Id); // avg ping = 48
		check(TestRegionQosResult.DatacenterOptions[3].Definition.Id == SubregionResult_FR.Definition.Id); // avg ping = 74
		check(TestRegionQosResult.DatacenterOptions[4].Definition.Id == SubregionResult_GB.Definition.Id); // avg ping = 92
		check(TestRegionQosResult.DatacenterOptions[5].Definition.Id == SubregionResult_GB_S.Definition.Id); // avg ping = 84
	}

	return true;
}

FRegionQosInstance UQosRegionManager::TestCreateExampleRegionResult()
{
	// Definitions.

	static const FString RegionId = TEXT("NAE");

	FQosRegionInfo ExampleRegion;
	ExampleRegion.DisplayName = NSLOCTEXT("MMRegion", "NA-East", "NA-East");
	ExampleRegion.RegionId = RegionId;
	ExampleRegion.RegionId = RegionId;
	ExampleRegion.bEnabled = true;
	ExampleRegion.bVisible = true;
	ExampleRegion.bAutoAssignable = true;

	ExampleRegion.bAllowSubspaceBias = true;
	ExampleRegion.SubspaceBiasParams = FQosSubspaceComparisonParams(300, 8, 25, 50.0f);

	FQosDatacenterInfo ParentDatacenter_VA;
	ParentDatacenter_VA.Id = TEXT("VA");
	ParentDatacenter_VA.RegionId = RegionId;
	ParentDatacenter_VA.bEnabled = true;
	ParentDatacenter_VA.Servers.Add(FQosPingServerInfo{ "334.193.154.39", 22222 });
	ParentDatacenter_VA.Servers.Add(FQosPingServerInfo{ "352.203.3.55", 22222 });
	ParentDatacenter_VA.Servers.Add(FQosPingServerInfo{ "354.82.195.216", 22222 });
	ParentDatacenter_VA.Servers.Add(FQosPingServerInfo{ "334.194.116.183", 22222 });

	FQosDatacenterInfo ChildDatacenter_VA_S1;
	ChildDatacenter_VA_S1.Id = TEXT("VA_S1");
	ChildDatacenter_VA_S1.RegionId = RegionId;
	ChildDatacenter_VA_S1.bEnabled = true;
	ChildDatacenter_VA_S1.Servers.Add(FQosPingServerInfo{ "352.203.34.155", 22222 });
	ChildDatacenter_VA_S1.Servers.Add(FQosPingServerInfo{ "352.203.34.156", 22222 });
	ChildDatacenter_VA_S1.Servers.Add(FQosPingServerInfo{ "352.203.34.157", 22222 });

	FQosDatacenterInfo ChildDatacenter_VA_S2;
	ChildDatacenter_VA_S2.Id = TEXT("VA_S2");
	ChildDatacenter_VA_S2.RegionId = RegionId;
	ChildDatacenter_VA_S2.bEnabled = true;
	ChildDatacenter_VA_S2.Servers.Add(FQosPingServerInfo{ "354.82.199.216", 22222 });
	ChildDatacenter_VA_S2.Servers.Add(FQosPingServerInfo{ "354.82.199.217", 22222 });
	ChildDatacenter_VA_S2.Servers.Add(FQosPingServerInfo{ "354.82.199.218", 22222 });

	FQosDatacenterInfo ChildDatacenter_VA_S3;
	ChildDatacenter_VA_S3.Id = TEXT("VA_S3");
	ChildDatacenter_VA_S3.RegionId = RegionId;
	ChildDatacenter_VA_S3.bEnabled = true;
	ChildDatacenter_VA_S3.Servers.Add(FQosPingServerInfo{ "334.194.111.183", 22222 });
	ChildDatacenter_VA_S3.Servers.Add(FQosPingServerInfo{ "334.194.111.184", 22222 });
	ChildDatacenter_VA_S3.Servers.Add(FQosPingServerInfo{ "334.194.111.185", 22222 });

	FQosDatacenterInfo ParentDatacenter_OH;
	ParentDatacenter_OH.Id = TEXT("OH");
	ParentDatacenter_OH.RegionId = RegionId;
	ParentDatacenter_OH.bEnabled = true;
	ParentDatacenter_OH.Servers.Add(FQosPingServerInfo{ "313.59.18.131", 22222 });
	ParentDatacenter_OH.Servers.Add(FQosPingServerInfo{ "318.216.47.148", 22222 });
	ParentDatacenter_OH.Servers.Add(FQosPingServerInfo{ "318.221.198.242", 22222 });
	ParentDatacenter_OH.Servers.Add(FQosPingServerInfo{ "352.15.144.157", 22222 });

	FQosDatacenterInfo ChildDatacenter_OH_S;
	ChildDatacenter_OH_S.Id = TEXT("OH_S");
	ChildDatacenter_OH_S.RegionId = RegionId;
	ChildDatacenter_OH_S.bEnabled = true;
	ChildDatacenter_OH_S.Servers.Add(FQosPingServerInfo{ "318.216.49.148", 22222 });
	ChildDatacenter_OH_S.Servers.Add(FQosPingServerInfo{ "318.216.49.149", 22222 });
	ChildDatacenter_OH_S.Servers.Add(FQosPingServerInfo{ "318.216.49.150", 22222 });

	// Results

	FRegionQosInstance OutRegionResult;
	OutRegionResult.Definition = ExampleRegion;
	OutRegionResult.DatacenterOptions.Empty();

	FDatacenterQosInstance SubregionResult_VA;
	SubregionResult_VA.Definition = ParentDatacenter_VA;
	SubregionResult_VA.AvgPingMs = 96;
	OutRegionResult.DatacenterOptions.Add(SubregionResult_VA);

	FDatacenterQosInstance SubregionResult_VA_S1;
	SubregionResult_VA_S1.Definition = ChildDatacenter_VA_S1;
	SubregionResult_VA_S1.AvgPingMs = 91;
	OutRegionResult.DatacenterOptions.Add(SubregionResult_VA_S1);

	FDatacenterQosInstance SubregionResult_VA_S2;
	SubregionResult_VA_S2.Definition = ChildDatacenter_VA_S2;
	SubregionResult_VA_S2.AvgPingMs = 88;
	OutRegionResult.DatacenterOptions.Add(SubregionResult_VA_S2);

	FDatacenterQosInstance SubregionResult_VA_S3;
	SubregionResult_VA_S3.Definition = ChildDatacenter_VA_S3;
	SubregionResult_VA_S3.AvgPingMs = 90;
	OutRegionResult.DatacenterOptions.Add(SubregionResult_VA_S3);

	FDatacenterQosInstance SubregionResult_OH;
	SubregionResult_OH.Definition = ParentDatacenter_OH;
	SubregionResult_OH.AvgPingMs = 117;
	OutRegionResult.DatacenterOptions.Add(SubregionResult_OH);

	FDatacenterQosInstance SubregionResult_OH_S;
	SubregionResult_OH_S.Definition = ChildDatacenter_OH_S;
	SubregionResult_OH_S.AvgPingMs = 97;
	OutRegionResult.DatacenterOptions.Add(SubregionResult_OH_S);

	return OutRegionResult;
}

#endif // DEBUG_SUBCOMPARE_BY_SUBSPACE
