// Copyright Epic Games, Inc. All Rights Reserved.

#include "QosStats.h"
#include "Interfaces/IAnalyticsProvider.h"

#define QOS_STATS_VERSION 2
#define DEBUG_QOS_STATS 0

// Events
const FString FQosDatacenterStats::QosStats_DatacenterEvent = TEXT("QosStats_DatacenterEvent");

// Common attribution
const FString FQosDatacenterStats::QosStats_SessionId = TEXT("SessionId");
const FString FQosDatacenterStats::QosStats_Version = TEXT("Version");

// Header stats
const FString FQosDatacenterStats::QosStats_Timestamp = TEXT("Timestamp");
const FString FQosDatacenterStats::QosStats_TotalTime = TEXT("TotalTime");

// Qos stats
const FString FQosDatacenterStats::QosStats_DeterminationType = TEXT("DeterminationType");
const FString FQosDatacenterStats::QosStats_NumRegions = TEXT("NumRegions");
const FString FQosDatacenterStats::QosStats_RegionDetails = TEXT("RegionDetailsv2");
const FString FQosDatacenterStats::QosStats_NumResults = TEXT("NumResults");
const FString FQosDatacenterStats::QosStats_NumSuccessCount = TEXT("NumSuccessCount");
const FString FQosDatacenterStats::QosStats_NetworkType = TEXT("NetworkType");
const FString FQosDatacenterStats::QosStats_BestRegionId = TEXT("BestRegionId");
const FString FQosDatacenterStats::QosStats_BestRegionPing = TEXT("BestRegionPing");
const FString FQosDatacenterStats::QosStats_BestEndpointId = TEXT("BestEndpointId");
const FString FQosDatacenterStats::QosStats_BestEndpointPing = TEXT("BestEndpointPing");
const FString FQosDatacenterStats::QosStats_ChosenRegionId = TEXT("ChosenbRegionId");
const FString FQosDatacenterStats::QosStats_ChosenSubRegionId = TEXT("ChosenSubRegionId");
const FString FQosDatacenterStats::QosStats_ChosenSubRegionPing = TEXT("ChosenSubRegionPing");

/**
 * Debug output for the contents of a recorded stats event
 *
 * @param StatsEvent event type recorded
 * @param Attributes attribution of the event
 */
inline void PrintEventAndAttributes(const FString& StatsEvent, const TArray<FAnalyticsEventAttribute>& Attributes)
{
#if DEBUG_QOS_STATS
	UE_LOGF(LogQos, Display, "Event: %ls", *StatsEvent);
	for (int32 AttrIdx = 0; AttrIdx<Attributes.Num(); AttrIdx++)
	{
		const FAnalyticsEventAttribute& Attr = Attributes[AttrIdx];
		UE_LOGF(LogQos, Display, "\t%ls : %ls", *Attr.GetName(), *Attr.GetValue());
	}
#endif
}

FQosDatacenterStats::FQosDatacenterStats() :
	StatsVersion(QOS_STATS_VERSION),
	bAnalyticsInProgress(false)
{
}

void FQosDatacenterStats::StartTimer(FQosStats_Timer& Timer)
{
	Timer.MSecs = FPlatformTime::Seconds();
	Timer.bInProgress = true;
}

void FQosDatacenterStats::StopTimer(FQosStats_Timer& Timer)
{
	if (Timer.bInProgress)
	{
		Timer.MSecs = (FPlatformTime::Seconds() - Timer.MSecs) * 1000;
		Timer.bInProgress = false;
	}
}

void FQosDatacenterStats::StartQosPass()
{
	if (!bAnalyticsInProgress)
	{
		FDateTime UTCTime = FDateTime::UtcNow();
		QosData.Timestamp = UTCTime.ToString();

		StartTimer(QosData.SearchTime);
		bAnalyticsInProgress = true;
	}
}

void FQosDatacenterStats::RecordRegionInfo(const FDatacenterQosInstance& RegionInfo, int32 NumResults)
{
	if (bAnalyticsInProgress)
	{
		FQosStats_RegionInfo& NewRegion = *new (QosData.Regions) FQosStats_RegionInfo();
		NewRegion.RegionId = RegionInfo.Definition.Id;
		NewRegion.ParentRegionId = RegionInfo.Definition.RegionId;
		NewRegion.AvgPing = RegionInfo.AvgPingMs;
		NewRegion.bUsable = RegionInfo.bUsable;
		NewRegion.NumResults = NumResults;
	}
}

void FQosDatacenterStats::RecordQosAttempt(const FString& Region, const FString& OwnerId, int32 PingInMs, bool bSuccess)
{
	if (bAnalyticsInProgress)
	{
		QosData.NumTotalSearches++;
		QosData.NumSuccessAttempts += bSuccess ? 1 : 0;

		FQosStats_QosSearchResult& NewSearchResult = *new (QosData.SearchResults) FQosStats_QosSearchResult();
		NewSearchResult.OwnerId = OwnerId;
		NewSearchResult.PingInMs = PingInMs;
		NewSearchResult.DatacenterId = Region;
		NewSearchResult.bIsValid = true;
	}
}

void FQosDatacenterStats::SetChosenSubRegion(FString&& SubRegion)
{
	if (bAnalyticsInProgress)
	{
		QosData.RulesChosenSubRegion = MoveTemp(SubRegion);
	}
}

void FQosDatacenterStats::SetChosenRegion(FString && Region)
{
	if (bAnalyticsInProgress)
	{
		QosData.RulesChosenRegion = MoveTemp(Region);
	}
}

void FQosDatacenterStats::EndQosPass(EDatacenterResultType Result)
{
	if (bAnalyticsInProgress)
	{
		Finalize();
		QosData.DeterminationType = Result;
	}
}

void FQosDatacenterStats::Finalize()
{
	StopTimer(QosData.SearchTime);
	bAnalyticsInProgress = false;
}

void FQosDatacenterStats::Upload(TSharedPtr<IAnalyticsProvider>& AnalyticsProvider)
{
	if (bAnalyticsInProgress)
	{
		Finalize();
	}

	// GUID representing the entire datacenter determination attempt
	FGuid QosStatsGuid;
	FPlatformMisc::CreateGuid(QosStatsGuid);

	ParseQosResults(AnalyticsProvider, QosStatsGuid);
}

/**
 * @EventName QosStats_DatacenterEvent
 * @Trigger Attempt to determine a user datacenter from available QoS information
 * @Type Client
 * @EventParam SessionId string Guid of this attempt
 * @EventParam Version integer Qos analytics version
 * @EventParam Timestamp string Timestamp when this whole attempt started
 * @EventParam TotalTime float Total time this complete attempt took, includes delay between all ping queries (ms)
 * @EventParam DeterminationType how the region data was determined @see EDatacenterResultType
 * @EventParam NumRegions integer Total number of regions considered or known at the time
 * @EventParam NumResults integer Total number of results found for consideration
 * @EventParam NumSuccessCount integer Total number of successful ping evaluations
 * @EventParam NetworkType string type of network the client is connected to. (Unknown, None, AirplaneMode, Cell, Wifi, Ethernet) are possible values. Will be Unknown on PC and Switch.
 * @EventParam BestRegionId string RegionId with best ping (that is usable, UNREACHABLE if none pass QoS)
 * @EventParam BestRegionPing integer ping in the best RegionId (that is usable, 0 if BestRegionId is UNREACHABLE)
 * @EventParam BestEndpointId string RegionId with best ping (regardless of usability, UNREACHABLE if none pass QoS)
 * @EventParam BestEndpointPing integer ping in the best RegionId (regardless of usability, 0 if BestEndpointId is UNREACHABLE)
 * @EventParam ChosenRegionId string RegionId chosen by RegionManager including bias, may differ form best (that is usable, UNREACHABLE if none pass QoS)
 * @EventParam ChosenRegionPing integer ping in the chosen RegionId (that is usable, 0 if BestRegionId is UNREACHABLE)
 * @EventParam RegionDetails json representation of ping details
 * @Comments Analytics data for a complete qos datacenter determination attempt
 * 
 * @Owner Josh.Markiewicz
 */
void FQosDatacenterStats::ParseQosResults(TSharedPtr<IAnalyticsProvider>& AnalyticsProvider, FGuid& SessionId)
{
	TArray<FAnalyticsEventAttribute> QoSAttributes = MakeAnalyticsEventAttributeArray(
		QosStats_SessionId, SessionId.ToString(),
		QosStats_Version, StatsVersion,
		QosStats_Timestamp, QosData.Timestamp,
		QosStats_TotalTime, QosData.SearchTime.MSecs,
		QosStats_DeterminationType, ToString(QosData.DeterminationType),
		QosStats_NumRegions, QosData.Regions.Num(),
		QosStats_NumResults, QosData.NumTotalSearches,
		QosStats_NumSuccessCount, QosData.NumSuccessAttempts,
		QosStats_NetworkType, FPlatformMisc::GetNetworkConnectionType()
	);

	{
		FString BestRegionId(TEXT("Unknown"));
		int32 BestPing = INT_MAX;
		FString BestEndpointId(TEXT("Unknown"));
		int32 BestEndpointPing = INT_MAX;

		FString ChosenSubRegionId(TEXT("Unknown"));
		int32 ChosenSubRegionPing = INT_MAX;

		for (const FQosStats_RegionInfo& Region : QosData.Regions)
		{
			if (Region.AvgPing < BestPing)
			{
				if (Region.bUsable)
				{
					BestRegionId = Region.RegionId;
					BestPing = Region.AvgPing;
				}
				
				BestEndpointId = Region.RegionId;
				BestEndpointPing = Region.AvgPing;
			}

			if(Region.RegionId == QosData.RulesChosenSubRegion)
			{
				ChosenSubRegionId = Region.RegionId;
				ChosenSubRegionPing = Region.AvgPing;
			}
		}

		BestPing = FMath::Clamp(BestPing, 0, UNREACHABLE_PING);
		ChosenSubRegionPing = FMath::Clamp(ChosenSubRegionPing, 0, UNREACHABLE_PING);
		if (BestPing < UNREACHABLE_PING)
		{
			QoSAttributes.Add(FAnalyticsEventAttribute(QosStats_BestRegionId, BestRegionId));
			QoSAttributes.Add(FAnalyticsEventAttribute(QosStats_BestRegionPing, BestPing));
		}
		else
		{
			QoSAttributes.Add(FAnalyticsEventAttribute(QosStats_BestRegionId, TEXT("UNREACHABLE")));
			QoSAttributes.Add(FAnalyticsEventAttribute(QosStats_BestRegionPing, 0));
		}

		if (ChosenSubRegionPing < UNREACHABLE_PING)
		{
			QoSAttributes.Add(FAnalyticsEventAttribute(QosStats_ChosenSubRegionId, ChosenSubRegionId));
			QoSAttributes.Add(FAnalyticsEventAttribute(QosStats_ChosenSubRegionPing, ChosenSubRegionPing));
		}
		else
		{
			QoSAttributes.Add(FAnalyticsEventAttribute(QosStats_ChosenSubRegionId, TEXT("UNREACHABLE")));
			QoSAttributes.Add(FAnalyticsEventAttribute(QosStats_ChosenSubRegionPing, 0));
		}

		if(!QosData.RulesChosenRegion.IsEmpty())
		{
			QoSAttributes.Add(FAnalyticsEventAttribute(QosStats_ChosenRegionId, QosData.RulesChosenRegion));
		}

		BestEndpointPing = FMath::Clamp(BestEndpointPing, 0, UNREACHABLE_PING);
		if (BestEndpointPing < UNREACHABLE_PING)
		{
			QoSAttributes.Add(FAnalyticsEventAttribute(QosStats_BestEndpointId, BestEndpointId));
			QoSAttributes.Add(FAnalyticsEventAttribute(QosStats_BestEndpointPing, BestEndpointPing));
		}
		else
		{
			QoSAttributes.Add(FAnalyticsEventAttribute(QosStats_BestEndpointId, TEXT("UNREACHABLE")));
			QoSAttributes.Add(FAnalyticsEventAttribute(QosStats_BestEndpointPing, 0));
		}
	}

	{
		FString StatsJson;
		TSharedRef<TJsonWriter<TCHAR, TCondensedJsonPrintPolicy< TCHAR > > > JsonWriter = TJsonWriterFactory<TCHAR, TCondensedJsonPrintPolicy< TCHAR > >::Create(&StatsJson);
		FJsonSerializerWriter<TCHAR, TCondensedJsonPrintPolicy< TCHAR > > Serializer(JsonWriter);
		Serializer.StartArray();
		for (FQosStats_RegionInfo& Region : QosData.Regions)
		{
			Region.Serialize(Serializer, false);
		}
		Serializer.EndArray();
		JsonWriter->Close();

		QoSAttributes.Add(FAnalyticsEventAttribute(QosStats_RegionDetails, FJsonFragment(MoveTemp(StatsJson))));
	}

	PrintEventAndAttributes(QosStats_DatacenterEvent, QoSAttributes);
	if (AnalyticsProvider.IsValid())
	{
		AnalyticsProvider->RecordEvent(QosStats_DatacenterEvent, QoSAttributes);
	}
}
