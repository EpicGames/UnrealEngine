// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Serialization/JsonSerializerMacros.h"

#include "QosRegionManager.h"

class IAnalyticsProvider;
struct FDatacenterQosInstance;

/** Types of result determination types */

enum class EDatacenterResultType :uint8
{
	/** Normal flow result */
	Normal,
	/** Using previously cached value */
	Cached,
	/** Using forced */
	Forced,
	/** Using forced default */
	Default,
	/** Failure to complete */
	Failure
};

/** @return the stringified version of the enum passed in */
inline const TCHAR* ToString(EDatacenterResultType ResultType)
{
	switch (ResultType)
	{
		case EDatacenterResultType::Normal:
		{
			return TEXT("Normal");
		}
		case EDatacenterResultType::Cached:
		{
			return TEXT("Cached");
		}
		case EDatacenterResultType::Forced:
		{
			return TEXT("Forced");
		}
		case EDatacenterResultType::Default:
		{
			return TEXT("Default");
		}
		case EDatacenterResultType::Failure:
		{
			return TEXT("Failure");
		}
	}

	return TEXT("");
}

/**
 * Datacenter determination stats
 */
class FQosDatacenterStats 
{
private:

	struct FQosStats_Timer
	{
		/** Time in ms captured */
		double MSecs;
		/** Is this timer running */
		bool bInProgress;

		FQosStats_Timer() :
			MSecs(0.0),
			bInProgress(false)
		{}
	};

	class FQosStats_RegionInfo : public FJsonSerializable
	{
	public:
		/** Region designation */
		FString RegionId;
		/** Parent region */
		FString ParentRegionId;
		/** Number of Qos servers pinged */
		int32 NumResults;
		/** Average ping across all results */
		int32 AvgPing;
		/** Is the region usable by the player */
		bool bUsable;
		FQosStats_RegionInfo()
			: RegionId(TEXT("Unknown"))
			, ParentRegionId(TEXT("Unknown"))
			, NumResults(0)
			, AvgPing(UNREACHABLE_PING)
			, bUsable(false)
		{}

		BEGIN_JSON_SERIALIZER
			JSON_SERIALIZE("RegionId", RegionId);
			JSON_SERIALIZE("ParentRegionId", ParentRegionId);
			JSON_SERIALIZE("bUsable", bUsable);
			JSON_SERIALIZE("AvgPing", AvgPing);
			JSON_SERIALIZE("NumResults", NumResults);
		END_JSON_SERIALIZER
	};

	/** Stats representation of a single Qos search result */
	struct FQosStats_QosSearchResult
	{
		/** Owner of the session */
		FString OwnerId;
		/** Datacenter Id */
		FString DatacenterId;
		/** Ping time */
		int32 PingInMs;
		/** Is this result valid */
		bool bIsValid;

		FQosStats_QosSearchResult() :
			PingInMs(0),
			bIsValid(false)
		{}
	};

	struct FQosStats_CompletePass
	{
		/** Time of the search */
		FString Timestamp;
		/** Way the datacenter was chosen */
		EDatacenterResultType DeterminationType;
		/** Time in ms it took to find the search results (exclusive) */
		FQosStats_Timer SearchTime;
		/** Array of region information */
		TArray<FQosStats_RegionInfo> Regions;
		/** name for the region chosen based on rules */
		FString  RulesChosenRegion;
		/** name for the sub region chosen based on rules */
		FString  RulesChosenSubRegion;
		/** Number of search results tested */
		int32 NumTotalSearches;
		/** Number of successful search results */
		int32 NumSuccessAttempts;
		/** Array of search result details found this pass */
		TArray<FQosStats_QosSearchResult> SearchResults;
		
		FQosStats_CompletePass() :
			DeterminationType(EDatacenterResultType::Failure),
			NumTotalSearches(0),
			NumSuccessAttempts(0)
		{}
		~FQosStats_CompletePass() = default;
	};

	// Events
	static const FString QosStats_DatacenterEvent;

	// Common attribution
	static const FString QosStats_SessionId;
	static const FString QosStats_Version;

	// Header stats
	static const FString QosStats_Timestamp;
	static const FString QosStats_TotalTime;

	// Qos stats
	static const FString QosStats_DeterminationType;
	static const FString QosStats_NumRegions;
	static const FString QosStats_RegionDetails;
	static const FString QosStats_NumResults;
	static const FString QosStats_NumSuccessCount;
	static const FString QosStats_NetworkType;
	static const FString QosStats_BestRegionId;
	static const FString QosStats_BestRegionPing;
	static const FString QosStats_BestEndpointId;
	static const FString QosStats_BestEndpointPing;
	static const FString QosStats_ChosenRegionId;
	static const FString QosStats_ChosenSubRegionId;
	static const FString QosStats_ChosenSubRegionPing;

	/** Version of the stats for separation */
	int32 StatsVersion;

	/** Container of an entire search process */
	FQosStats_CompletePass QosData;

	/** Analytics in progress */
	bool bAnalyticsInProgress;

	/**
	 * Start the timer for a given stats container 
	 *
	 * @param Timer Timer to start
	 */
	void StartTimer(FQosStats_Timer& Timer);

	/**
	 * Stops the timer for a given stats container 
	 *
	 * @param Timer Timer to stop
	 */
	void StopTimer(FQosStats_Timer& Timer);

	/**
	 * Finalize all the data, stopping timers, etc
	 */
	void Finalize();

	/**
	 * Parse an entire search, adding its data to the recorded event
	 *
	 * @param AnalyticsProvider provider recording the event
	 * @param SessionId unique id for the qos event
	 */
	void ParseQosResults(TSharedPtr<IAnalyticsProvider>& AnalyticsProvider, FGuid& SessionId);

public:

	FQosDatacenterStats();
	virtual ~FQosDatacenterStats() = default;

	/**
	 * Start a Qos search pass
	 */
	void StartQosPass();

	/**
	 * Record a new region
	 * 
	 * @param RegionInfo info obtained about a region
	 * @param NumResults num results considered
	 */
	void RecordRegionInfo(const FDatacenterQosInstance& RegionInfo, int32 NumResults);

	/**
	 * Record a single ping attempt
	 * 
	 * @param Region region of the server
	 * @param OwnerId the owner of the server (ip address)
	 * @param PingInMs ping to the server
	 * @param bSuccess was the attempt successful
	 */
	void RecordQosAttempt(const FString& Region, const FString& OwnerId, int32 PingInMs, bool bSuccess);

	/**
	 * End recording of a Qos determination
	 *
	 * @param Result results of the qos pass
	 */
	void EndQosPass(EDatacenterResultType Result);

	/**
	 * Set the chosen region from Qos Evaluation (may differ from the fastest/best based on rules)
	 *
	 * @param Region string region code of chosen region
	 */
	void SetChosenRegion(FString&& Region);

	/**
	 * Set the chosen SubRegion from Qos Evaluation (may differ from the fastest/best based on rules)
	 *
	 * @param Region string SubRegion code of chosen region
	 */
	void SetChosenSubRegion(FString && SubRegion);

	/**
	 * Record previously saved stats to an analytics provider
	 *
	 * @param AnalyticsProvider provider to record stats to
	 */
	void Upload(TSharedPtr<IAnalyticsProvider>& AnalyticsProvider);

};
