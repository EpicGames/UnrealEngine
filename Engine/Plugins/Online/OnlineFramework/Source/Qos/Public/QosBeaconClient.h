// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineBeaconClient.h"
#include "QosBeaconClient.generated.h"

#define UE_API QOS_API

class FOnlineSessionSearchResult;

/**
 * Types of responses that can come back from the beacon
 */
UENUM()
enum class EQosResponseType : uint8
{
	/** Failed to connect to Qos endpoint */
	NoResponse,
	/** Response received from the Qos host  */
	Success,
	/** Some kind of failure */
	Failure
};

/**
 * Delegate triggered when a response from the Qos beacon has been received
 *
 * @param QosResponse response from the server
 * @param ResponseTime time to respond in ms
 */
DECLARE_DELEGATE_TwoParams(FOnQosRequestComplete, EQosResponseType /* QosResponse */, int32 /* ResponseTime */);

/**
 * A beacon client used for quality timings to a specified session
 */
UCLASS(MinimalAPI, transient, config=Engine)
class AQosBeaconClient : public AOnlineBeaconClient
{
	GENERATED_UCLASS_BODY()
	
	// Begin AOnlineBeaconClient Interface
	UE_API virtual void OnConnected() override;
	// End AOnlineBeaconClient Interface

	/**
	 * Initiate a Qos request with a given server
	 */
	UE_API virtual void SendQosRequest(const FOnlineSessionSearchResult& DesiredHost);

	/**
	 * Delegate triggered when a response from the Qos beacon has been received
	 *
	 * @return QosResponse response from the server
	 */
	FOnQosRequestComplete& OnQosRequestComplete() { return QosRequestComplete; }

protected:

	/** Time connection was established */
	double ConnectionStartTime;
	/** Time the Qos started */
	double QosStartTime;
	/** Session Id of the destination host */
	FString DestSessionId;
	/** Is there a Qos request in flight */
	bool bPendingQosRequest;

	/** Delegate for Qos request responses */
	FOnQosRequestComplete QosRequestComplete;

	/**
	 * Contact the server with a Qos request and begin timing
	 *
	 * @param InSessionId reference session id to make sure the session is the correct one
	 */
	UFUNCTION(server, reliable, WithValidation)
	UE_API virtual void ServerQosRequest(const FString& InSessionId);

	/**
	 * Response from the host session after making a Qos request
	 */
	UFUNCTION(client, reliable)
	UE_API virtual void ClientQosResponse(EQosResponseType Response);

	friend class AQosBeaconHost;
};

#undef UE_API
