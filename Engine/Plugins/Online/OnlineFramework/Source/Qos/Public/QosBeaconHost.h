// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineBeaconHostObject.h"
#include "QosBeaconHost.generated.h"

#define UE_API QOS_API

class AQosBeaconClient;

/**
 * A beacon host listening for Qos requests from a potential client
 */
UCLASS(MinimalAPI, transient, config=Engine)
class AQosBeaconHost : public AOnlineBeaconHostObject
{
	GENERATED_UCLASS_BODY()

	// Begin AOnlineBeaconHostObject Interface 
	// End AOnlineBeaconHostObject Interface 

	UE_API bool Init(FName InSessionName);

	/**
	 * Handle a single Qos request received from an incoming client
	 *
	 * @param Client client beacon making the request
	 * @param SessionId id of the session that is being checked
	 */
	UE_API void ProcessQosRequest(AQosBeaconClient* Client, const FString& SessionId);
	UE_API bool DoesSessionMatch(const FString& SessionId) const;

	/**
	 * Output current state of beacon to log
	 */
	UE_API void DumpState() const;

protected:

	/** Name of session this beacon is associated with */
	FName SessionName;

	/** Running total of Qos requests received since the beacon was created */
	int32 NumQosRequests;
};

#undef UE_API
