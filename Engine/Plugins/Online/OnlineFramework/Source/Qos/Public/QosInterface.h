// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/GCObject.h"

#define UE_API QOS_API

class UQosRegionManager;
struct FRegionQosInstance;

class IAnalyticsProvider;

#define NO_REGION TEXT("NONE")

/**
 * Main Qos interface for actions related to server quality of service
 */
class FQosInterface : public TSharedFromThis<FQosInterface>, public FGCObject
{
public:

	/**
	 * Get the interface singleton
	 */
	static UE_API TSharedRef<FQosInterface> Get();

	/**
	 * Re-initialize our FQosRegionManager instance
	 */
	UE_API bool Init();

	/**
	 * Start running the async QoS evaluation
	 */
	UE_API void BeginQosEvaluation(UWorld* World, const TSharedPtr<IAnalyticsProvider>& AnalyticsProvider, const FSimpleDelegate& OnComplete);

	DECLARE_MULTICAST_DELEGATE(FOnQosEvalCompleteDelegate);

	/**
	 * Get the delegate that is invoked when the current/next QoS evaluation completes.
	 */
	UE_API FOnQosEvalCompleteDelegate& OnQosEvalComplete();

	/**
	 * Returns true if Qos is in the process of being evaluated
	 */
	UE_API bool IsQosEvaluationInProgress() const;

	/**
	 * Get the region ID for this instance, checking ini and commandline overrides.
	 *
	 * Dedicated servers will have this value specified on the commandline
	 *
	 * Clients pull this value from the settings (or command line) and do a ping test to determine if the setting is viable.
	 *
	 * @return the current region identifier
	 */
	UE_API FString GetRegionId() const;

	/**
	 * Get the region ID with the current best ping time, checking ini and commandline overrides.
	 * 
	 * @return the default region identifier
	 */
	UE_API FString GetBestRegion() const;

	/** @return true if a reasonable enough number of results were returned from all known regions, false otherwise */
	UE_API bool AllRegionsFound() const;

	/**
	 * Get the list of regions that the client can choose from (returned from search and must meet min ping requirements)
	 *
	 * If this list is empty, the client cannot play.
	 */
	UE_API const TArray<FRegionQosInstance>& GetRegionOptions() const;

	/**
	 * Get a sorted list of subregions within a region
	 *
	 * @param RegionId region of interest
	 * @param OutSubregions list of subregions in sorted order
	 */
	UE_API void GetSubregionPreferences(const FString& RegionId, TArray<FString>& OutSubregions) const;
	
	/**
	 * @return a map of subregion to the computed AvgPing for that subregion
	 */
	UE_API TMap<FString, int64> GetAllSubregionPingsMap() const;

	/**
	 * @return true if this is a usable region, false otherwise
	 */
	UE_API bool IsUsableRegion(const FString& InRegionId) const;

	/**
	 * Try to set the selected region ID (must be present in GetRegionOptions)
	 */
	UE_API bool SetSelectedRegion(const FString& RegionId);

	/** Clear the region to nothing, used for logging out */
	UE_API void ClearSelectedRegion();

	/**
	 * Force the selected region creating a fake RegionOption if necessary
	 */
	UE_API void ForceSelectRegion(const FString& RegionId);

	/**
	 * Get the datacenter id for this instance, checking ini and commandline overrides
	 * This is only relevant for dedicated servers (so they can advertise). 
	 * Client does not search on this in any way
	 *
	 * @return the default datacenter identifier
	 */
	static UE_API FString GetDatacenterId();

	/**
	 * Get the subregion id for this instance, checking ini and commandline overrides
	 * This is only relevant for dedicated servers (so they can advertise). Client does
	 * not search on this (but may choose to prioritize results later)
	 */
	static UE_API FString GetAdvertisedSubregionId();

	/**
	 * Debug output for current region / datacenter information
	 */
	UE_API void DumpRegionStats();

	/**
	* Register a delegate to be called when QoS settings have changed.
	*/
	UE_API void RegisterQoSSettingsChangedDelegate(const FSimpleDelegate& OnQoSSettingsChanged);

	/**
	 * Delegate that fires whenever the current QoS region ID changes.
	 */
	 DECLARE_MULTICAST_DELEGATE_TwoParams(FOnQosRegionIdChanged, const FString& /* OldRegionId */, const FString& /* NewRegionId */);
	 UE_API FOnQosRegionIdChanged& OnQosRegionIdChanged();

protected:

	friend class FQosModule;
	UE_API FQosInterface();

	/** FGCObject interface */
	UE_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FQosInterface");
	}

private:

	/** Reference to the evaluator for making datacenter determinations */
	TObjectPtr<UQosRegionManager> RegionManager;
};

#undef UE_API
