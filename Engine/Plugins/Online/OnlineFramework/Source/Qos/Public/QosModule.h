// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleManager.h"

#define UE_API QOS_API

class FQosInterface;

/** Logging related to parties */
QOS_API DECLARE_LOG_CATEGORY_EXTERN(LogQos, Display, All);

/**
 * Module for QoS service utilities
 */
class FQosModule :
	public IModuleInterface, public FSelfRegisteringExec
{

public:

	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static inline FQosModule& Get()
	{
		return FModuleManager::LoadModuleChecked<FQosModule>("Qos");
	}

	/**
	 * Checks to see if this module is loaded and ready.  It is only valid to call Get() if IsAvailable() returns true.
	 *
	 * @return True if the module is loaded and ready to use
	 */
	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded("Qos");
	}

	/**
	 * Get the interface singleton
	 */
	UE_API TSharedRef<FQosInterface> GetQosInterface();

protected:
	// FSelfRegisteringExec
	UE_API virtual bool Exec_Runtime(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar) override;

private:
	TSharedPtr<FQosInterface> QosInterface;

private:

	// IModuleInterface

	/**
	 * Called when voice module is loaded
	 * Initialize platform specific parts of template handling
	 */
	UE_API virtual void StartupModule() override;
	
	/**
	 * Called when voice module is unloaded
	 * Shutdown platform specific parts of template handling
	 */
	UE_API virtual void ShutdownModule() override;
};

#undef UE_API
