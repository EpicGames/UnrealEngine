// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "Misc/DateTime.h"
#include "QosRegionManager.generated.h"

#define UE_API QOS_API

class IAnalyticsProvider;
class UQosEvaluator;

#define UNREACHABLE_PING 9999

#define DEBUG_SUBCOMPARE_BY_SUBSPACE 0

/** Enum for single region QoS return codes */
UENUM()
enum class EQosDatacenterResult : uint8
{
	/** Incomplete, invalid result */
	Invalid,
	/** QoS operation was successful */
	Success,
	/** QoS operation with one or more ping failures */
	Incomplete
};

inline const TCHAR* LexToString(EQosDatacenterResult Result)
{
	switch (Result)
	{
		case EQosDatacenterResult::Invalid: return TEXT("Invalid");
		case EQosDatacenterResult::Success: return TEXT("Success");
		case EQosDatacenterResult::Incomplete: return TEXT("Incomplete");
		default: return TEXT("Unknown");
	}
}

/** Enum for possible QoS return codes */
UENUM()
enum class EQosCompletionResult : uint8
{
	/** Incomplete, invalid result */
	Invalid,
	/** QoS operation was successful */
	Success,
	/** QoS operation ended in failure */
	Failure,
	/** QoS operation was canceled */
	Canceled
};

inline const TCHAR* LexToString(EQosCompletionResult Result)
{
	switch (Result)
	{
		case EQosCompletionResult::Invalid: return TEXT("Invalid");
		case EQosCompletionResult::Success: return TEXT("Success");
		case EQosCompletionResult::Failure: return TEXT("Failure");
		case EQosCompletionResult::Canceled: return TEXT("Canceled");
		default: return TEXT("Unknown");
	}
}

/**
 * Parameters to control the rules-based comparison of subspace vs non-subspace datacenter QoS results.
 * 
 * @see FDatacenterQosInstance::IsNonSubspaceRecommended(const FDatacenterQosInstance&, const FDatacenterQosInstance&, const FQosSubspaceComparisonParams&)
 */
USTRUCT()
struct FQosSubspaceComparisonParams
{
	GENERATED_USTRUCT_BODY()

	/**
	 * Maximum allowed ping of the non-subspace.
	 * If greater than this value, it is too slow, so fails to qualify.
	 * Set to zero or less to disable checks against this field.
	 */
	UPROPERTY()
	int32 MaxNonSubspacePingMs;

	/**
	 * Minimum allowed ping of the subspace.
	 * If below this value, it should not be overridden by the non-subspace.
	 * Set to zero or less to disable checks against this field.
	 */
	UPROPERTY()
	int32 MinSubspacePingMs;

	/**
	 * Maximum allowed difference between the subspace and non-subspace's ping values in milliseconds.
	 * If greater than this value, the non-subspace is too slow, so fails to qualify.
	 * Set to zero or less to disable checks against this field.
	 */
	UPROPERTY()
	int32 ConstantMaxToleranceMs;

	/**
	 * Maximum allowed difference between the subspace and non-subspace's ping values,
	 * which scales as a proportion of the non-subspace's ping, so will differ between
	 * comparisons when sorting a single list of datacenters.
	 * If greater than the scaled difference, the non-subspace is too slow, so fails to qualify. 
	 * Set to zero or less to disable checks against this field.
	 */
	UPROPERTY()
	float ScaledMaxTolerancePct;

	FQosSubspaceComparisonParams()
		: MaxNonSubspacePingMs(0)
		, MinSubspacePingMs(0)
		, ConstantMaxToleranceMs(0)
		, ScaledMaxTolerancePct(0.0f)
	{
	}

	FQosSubspaceComparisonParams(int32 InMaxNonSubspacePingMs, int32 InMinSubspacePingMs, int32 InConstantMaxToleranceMs, float InScaledMaxTolerancePct)
		: MaxNonSubspacePingMs(InMaxNonSubspacePingMs)
		, MinSubspacePingMs(InMinSubspacePingMs)
		, ConstantMaxToleranceMs(InConstantMaxToleranceMs)
		, ScaledMaxTolerancePct(InScaledMaxTolerancePct)
	{
	}

	float CalcScaledMaxToleranceMs(int32 PingMs) const
	{
		return 0.01f * ScaledMaxTolerancePct * PingMs;
	}
};

/**
 * Individual ping server details
 */
USTRUCT()
struct FQosPingServerInfo
{
	GENERATED_USTRUCT_BODY()

	/** Address of server */
	UPROPERTY()
	FString Address;
	/** Port of server */
	UPROPERTY()
	int32 Port = 0;
};

/**
 * Metadata about datacenters that can be queried
 */
USTRUCT()
struct FQosDatacenterInfo
{
	GENERATED_USTRUCT_BODY()

	/** Id for this datacenter */
	UPROPERTY()
	FString Id;
	/** Parent Region */
	UPROPERTY()
	FString RegionId;
	/** Is this region tested (only valid if region is enabled) */
	UPROPERTY()
	bool bEnabled;
	/** Addresses of ping servers */
	UPROPERTY()
	TArray<FQosPingServerInfo> Servers;

	FQosDatacenterInfo()
		: bEnabled(true)
	{
	}

	bool IsValid() const
	{
		return !Id.IsEmpty() && !RegionId.IsEmpty();
	}

	bool IsPingable() const
	{
		return bEnabled && IsValid();
	}

	bool IsSubspace(const TCHAR* const SubspaceDelimiter) const
	{
		return Id.Contains(SubspaceDelimiter, ESearchCase::IgnoreCase, ESearchDir::FromStart);
	}

	FString ToDebugString() const
	{
		return FString::Printf(TEXT("[%s][%s]"), *RegionId, *Id);
	}
};

/**
 * Metadata about regions made up of datacenters
 */
USTRUCT()
struct FQosRegionInfo
{
	GENERATED_USTRUCT_BODY()

	/** Localized name of the region */
	UPROPERTY()
	FText DisplayName;
	/** Id for the region, all datacenters must reference one of these */
	UPROPERTY()
	FString RegionId;
	/** Is this region tested at all (if false, overrides individual datacenters) */
	UPROPERTY()
	bool bEnabled;
	/** Is this region visible in the UI (can be saved by user, replaced with auto if region disappears */
	UPROPERTY()
	bool bVisible;
	/** Can this region be considered for auto detection */
	UPROPERTY()
	bool bAutoAssignable;
	/** Enable biased sorting algorithm on results for this region, which prefers non-subspaces over subspaces */
	UPROPERTY()
	bool bAllowSubspaceBias;
	/** Granular settings for biased subspace-based sorting algorithm, if enabled for this region */
	UPROPERTY()
	FQosSubspaceComparisonParams SubspaceBiasParams;

	FQosRegionInfo()
		: bEnabled(true)
		, bVisible(true)
		, bAutoAssignable(true)
		, bAllowSubspaceBias(false)
	{
	}

	bool IsValid() const
	{
		return !RegionId.IsEmpty();
	}

	/** @return true if this region is supposed to be tested */
	bool IsPingable() const
	{
		return bEnabled;
	}

	/** @return true if a user can select this region in game */
	bool IsUsable() const
	{
		return bVisible && IsPingable();
	}

	/** @return true if this region can be auto assigned */
	bool IsAutoAssignable() const
	{
		return bAutoAssignable && IsUsable();
	}
};

/** Runtime information about a given region */
USTRUCT()
struct FDatacenterQosInstance
{
	GENERATED_USTRUCT_BODY()

	/** Information about the datacenter */
	UPROPERTY(Transient)
	FQosDatacenterInfo Definition;
	/** Success of the qos evaluation */
	UPROPERTY(Transient)
	EQosDatacenterResult Result;
	/** Avg ping times across all search results */
	UPROPERTY(Transient)
	int32 AvgPingMs;
	/** Transient list of ping times obtained for this datacenter */
	UPROPERTY(Transient)
	TArray<int32> PingResults;
	/** Number of good results */
	int32 NumResponses;
	/** Last time this datacenter was checked */
	UPROPERTY(Transient)
	FDateTime LastCheckTimestamp;
	/** Is the parent region usable */
	UPROPERTY(Transient)
	bool bUsable;

	FDatacenterQosInstance()
		: Result(EQosDatacenterResult::Invalid)
		, AvgPingMs(UNREACHABLE_PING)
		, NumResponses(0)
		, LastCheckTimestamp(0)
		, bUsable(true)
	{
	}

	FDatacenterQosInstance(const FQosDatacenterInfo& InMeta, bool bInUsable)
		: Definition(InMeta)
		, Result(EQosDatacenterResult::Invalid)
		, AvgPingMs(UNREACHABLE_PING)
		, NumResponses(0)
		, LastCheckTimestamp(0)
		, bUsable(bInUsable)
	{
	}

	/** reset the data to its default state */
	void Reset()
	{
		// Only the transient values get reset
		Result = EQosDatacenterResult::Invalid;
		AvgPingMs = UNREACHABLE_PING;
		PingResults.Empty();
		NumResponses = 0;
		LastCheckTimestamp = FDateTime(0);
		bUsable = false;
	}

	/**
	 * Compares the avg ping of two datacenters, handling cases where one is a subspace
	 * and the other is not, and like-for-like.
	 * 
	 * When comparing subspace vs non-subspace, this will bias towards the non-subspace,
	 * as long as it satisfies the series of qualifying rules.
	 * When comparing like-for-like, average ping is compared, as usual.
	 * 
	 * @param A - Left-hand datacenter QoS data to compare
	 * @param B - Right-hand datacenter QoS data to compare
	 * @param ComparisonParams - Rules settings for subspace vs non-subspace comparison
	 * @param SubspaceDelimiter - Search term to look for in datacenter ID; if found, implies that it is a subspace
	 * @return True if left-hand datacenter is "better", otherwise false (right-hand datacenter is "better")
	 * 
	 * @see FDatacenterQosInstance::IsNonSubspaceRecommended(const FDatacenterQosInstance&, const FDatacenterQosInstance&, const FQosSubspaceComparisonParams&)
	 */
	static UE_API bool IsLessWhenBiasedTowardsNonSubspace(const FDatacenterQosInstance& A, const FDatacenterQosInstance& B,
		const FQosSubspaceComparisonParams& ComparisonParams, const TCHAR* const SubspaceDelimiter);

	/**
	 * Compares a subspace datacenter and a non-subspace datacenter, applying the qualifying
	 * rules to bias non-subspaces, configured via the supplied comparison parameters.
	 * 
	 * @param NonSubspace - The non-subspace datacenter QoS data (must be already known to not be a subspace)
	 * @param Subspace - The subspace datacenter QoS data (must be already known to be a subspace)
	 * @param ComparisonParams - Granulator adjustments to the comparison rules
	 * @return True if the NonSubspace is "better" (i.e. passes the qualifying rules), otherwise false.
	 */
	static UE_API bool IsNonSubspaceRecommended(const FDatacenterQosInstance& NonSubspace, const FDatacenterQosInstance& Subspace,
		const FQosSubspaceComparisonParams& ComparisonParams);

	/**
	 * Compares a subspace datacenter and a non-subspace datacenter, applying the qualifying
	 * rules to bias non-subspaces, configured via the supplied comparison parameters.
	 *
	 * @param NonSubspace - The non-subspace datacenter QoS data (must be already known to not be a subspace)
	 * @param Subspace - The subspace datacenter QoS data (must be already known to be a subspace)
	 * @param ComparisonParams - Granulator adjustments to the comparison rules
	 * @return A reference to the input datacenter that is considered "better" after rules-based comparison.
	 *
	 * @see FDatacenterQosInstance::IsNonSubspaceRecommended(const FDatacenterQosInstance&, const FDatacenterQosInstance&, const FQosSubspaceComparisonParams&)
	 */
	static UE_API const FDatacenterQosInstance& CompareBiasedTowardsNonSubspace(
		const FDatacenterQosInstance& NonSubspace, const FDatacenterQosInstance& Subspace,
		const FQosSubspaceComparisonParams& ComparisonParams);

	/**
	 * Compares a subspace datacenter and a non-subspace datacenter, applying the qualifying
	 * rules to bias non-subspaces, configured via the supplied comparison parameters.
	 *
	 * @param NonSubspace - The non-subspace datacenter QoS data (must be already known to not be a subspace)
	 * @param Subspace - The subspace datacenter QoS data (must be already known to be a subspace)
	 * @param ComparisonParams - Granulator adjustments to the comparison rules
	 * @return A pointer to the input datacenter that is considered "better" after rules-based comparison.
	 *
	 * @see FDatacenterQosInstance::IsNonSubspaceRecommended(const FDatacenterQosInstance&, const FDatacenterQosInstance&, const FQosSubspaceComparisonParams&)
	 */
	static UE_API const FDatacenterQosInstance* const CompareBiasedTowardsNonSubspace(
		const FDatacenterQosInstance* const NonSubspace, const FDatacenterQosInstance* const Subspace,
		const FQosSubspaceComparisonParams& ComparisonParams);
};

USTRUCT()
struct FRegionQosInstance
{
	GENERATED_USTRUCT_BODY()

	/** Information about the region */
	UPROPERTY(Transient)
	FQosRegionInfo Definition;

	/** Array of all known datacenters and their status */
	UPROPERTY()
	TArray<FDatacenterQosInstance> DatacenterOptions;

	FRegionQosInstance()
	{
	}

	FRegionQosInstance(const FQosRegionInfo& InMeta)
		: Definition(InMeta)
	{
	}

	/** @return the region id for this region instance */
	const FString& GetRegionId() const
	{ 
		return Definition.RegionId; 
	}

	/** @return if this region data is usable externally */
	bool IsUsable() const
	{
		return Definition.IsUsable();
	}

	/** @return true if this region can be consider for auto detection */
	bool IsAutoAssignable() const
	{
		bool bValidResults = (GetRegionResult() == EQosDatacenterResult::Success) || (GetRegionResult() == EQosDatacenterResult::Incomplete);
		return Definition.IsAutoAssignable() && IsUsable() && bValidResults;
	}

	/** @return the result of this region ping request */
	UE_API EQosDatacenterResult GetRegionResult() const;

	/** @return the ping recorded in the best sub region */
	UE_API int32 GetBestAvgPing() const;

	/** @return the subregion with the best ping */
	UE_API FString GetBestSubregion() const;

	/** @return sorted list of subregions by best ping */
	UE_API void GetSubregionPreferences(TArray<FString>& OutSubregions) const;

	/** Sort the list of datacenter options into ascending order of average ping */
	UE_API void SortDatacenterOptionsByAvgPingAsc();

	/**
	 * Sort the list of datacenter options into ascending order, using rules-based comparison
	 * for cases where a subspace is being compared to a non-subspace; non-subspace will be
	 * favoured if it passes the rules check.
	 * Like-for-like comparisons are favour lower average ping.
	 * 
	 * @param ComparisonParams - Granulator adjustments to the comparison rules
	 * @param SubspaceDelimiter - Search term to look for in datacenter ID; if found, implies that it is a subspace
	 *
	 * @see FDatacenterQosInstance::IsLessWhenBiasedTowardsNonSubspace(const FDatacenterQosInstance&, const FDatacenterQosInstance&, const FQosSubspaceComparisonParams&, const TCHAR*);
	 */
	UE_API void SortDatacenterSubspacesByRecommended(const FQosSubspaceComparisonParams& ComparisonParams, const TCHAR* const SubspaceDelimiter);

	/** Print list of datacenters results for this region to QoS log. */
	UE_API void LogDatacenterResults() const;
};

/**
 * Main Qos interface for actions related to server quality of service
 */
UCLASS(MinimalAPI, config = Engine)
class UQosRegionManager : public UObject
{

	GENERATED_UCLASS_BODY()

public:

	/**
	 * Start running the async QoS evaluation 
	 */
	UE_API void BeginQosEvaluation(UWorld* World, const TSharedPtr<IAnalyticsProvider>& AnalyticsProvider, const FSimpleDelegate& OnComplete);

	/**
	 * Returns true if Qos is in the process of being evaluated
	 */
	UE_API bool IsQosEvaluationInProgress() const;

	/**
	 * Get the region ID for this instance, checking ini and commandline overrides.
	 * 
	 * Dedicated servers will have this value specified on the commandline
	 * 
	 * Clients pull this value from the settings (or command line) and do a ping test to determine if the setting is viable.
	 *
	 * @return the current region identifier
	 */
	UE_API FString GetRegionId() const;

	/**
	 * Get the region ID with the current best ping time, checking ini and commandline overrides.
	 * 
	 * @return the default region identifier
	 */
	UE_API FString GetBestRegion() const;

	/**
	 * Get the list of regions that the client can choose from (returned from search and must meet min ping requirements)
	 *
	 * If this list is empty, the client cannot play.
	 */
	UE_API const TArray<FRegionQosInstance>& GetRegionOptions() const;

	/**
	 * Get a sorted list of subregions within a region
	 *
	 * @param RegionId region of interest
	 * @param OutSubregions list of subregions in sorted order
	 */
	UE_API void GetSubregionPreferences(const FString& RegionId, TArray<FString>& OutSubregions) const;
	
	/** @return map of all subregions to avg ping times */
	UE_API TMap<FString, int64> GetAllSubregionPingsMap() const;

	/**
	 * @return true if this is a usable region, false otherwise
	 */
	UE_API bool IsUsableRegion(const FString& InRegionId) const;

	/**
	 * Try to set the selected region ID (must be present in GetRegionOptions)
	 *
	 * @param bForce if true then use selected region even if QoS eval has not completed successfully
	 */
	UE_API bool SetSelectedRegion(const FString& RegionId, bool bForce=false);

	/** Clear the region to nothing, used for logging out */
	UE_API void ClearSelectedRegion();

	/**
	 * Force the selected region creating a fake RegionOption if necessary
	 */
	UE_API void ForceSelectRegion(const FString& RegionId);

	/**
	 * Delegate that fires whenever the current QoS region ID changes.
	 */
	 DECLARE_MULTICAST_DELEGATE_TwoParams(FOnQosRegionIdChanged, const FString& /* OldRegionId */, const FString& /* NewRegionId */);
	 FOnQosRegionIdChanged& OnQosRegionIdChanged() { return OnQosRegionIdChangedDelegate; }

	/**
	 * Get the datacenter id for this instance, checking ini and commandline overrides
	 * This is only relevant for dedicated servers (so they can advertise). 
	 * Client does not search on this in any way
	 *
	 * @return the default datacenter identifier
	 */
	static UE_API FString GetDatacenterId();

	/**
	 * Get the subregion id for this instance, checking ini and commandline overrides
	 * This is only relevant for dedicated servers (so they can advertise). Client does
	 * not search on this (but may choose to prioritize results later)
	 */
	static UE_API FString GetAdvertisedSubregionId();

	/** @return true if a reasonable enough number of results were returned from all known regions, false otherwise */
	UE_API bool AllRegionsFound() const;

	/**
	 * Debug output for current region / datacenter information
	 */
	UE_API void DumpRegionStats() const;

	UE_API void RegisterQoSSettingsChangedDelegate(const FSimpleDelegate& OnQoSSettingsChanged);

	DECLARE_MULTICAST_DELEGATE(FOnQosEvalCompleteDelegate);

	/**
	 * Get the delegate that is invoked when the current/next QoS evaluation completes.
	 */
	FOnQosEvalCompleteDelegate& OnQosEvalComplete() { return OnQosEvalCompleteDelegate; }

public:

	/** Begin UObject interface */
	UE_API virtual void PostReloadConfig(class FProperty* PropertyThatWasLoaded) override;
	/** End UObject interface */

private:

	/**
	 * Get the delimiter string used to split primary subspace ID from a subregion ID.
	 */
	UE_API const TCHAR* GetSubspaceDelimiter() const;

	/**
	 * Finds the QOS region result that matches the given region ID from an array of region results.
	 * 
	 * @return pointer the the region result if found, otherwise nullptr
	 */
	static UE_API const FRegionQosInstance* FindQosRegionById(const TArray<FRegionQosInstance>& Regions, const FString& RegionId);

	/**
	 * Finds the QOS region result that has the "best" ping result.
	 * Assumes that the datacenter results within each region result are pre-sorted,
	 * e.g. by average ping, or via a recommendation bias algorithm.
	 * 
	 * @return pointer to the "best" region result, determined by a previous sort of region datacenters, otherwise nullptr if no results exist.
	 */
	static UE_API const FRegionQosInstance* FindBestQosRegion(const TArray<FRegionQosInstance>& Regions);

#ifdef DEBUG_SUBCOMPARE_BY_SUBSPACE
	// Test methods for debugging datacenter comparisons that use special rules when dealing with subspaces.
	static UE_API bool TestCompareDatacentersBySubspace();
	static UE_API bool TestSortDatacenterSubspacesByRecommended();
	static UE_API FRegionQosInstance TestCreateExampleRegionResult();
#endif // DEBUG_SUBCOMPARE_BY_SUBSPACE

private:

	/**
	 * Double check assumptions based on current region/datacenter definitions
	 */
	UE_API void SanityCheckDefinitions() const;

	UE_API void OnQosEvaluationComplete(EQosCompletionResult Result, const TArray<FDatacenterQosInstance>& DatacenterInstances, FString* OutSelectedRegion, FString* OutSelectedSubRegion);

	/**
	 * Use the existing set value, or if it is currently invalid, set the next best region available
	 */
	UE_API void TrySetDefaultRegion();

	/**
	 * @return max allowable ping to any region and still be allowed to play
	 */
	UE_API int32 GetMaxPingMs() const;

	/**
	 * Should datacenter QoS results be sorted using rules-based comparison where subspaces are encountered?
	 * 
	 * Determined via bEnableSubspaceBiasOrder engine configuration param for QosRegionManager.
	 * Global enable/disable may be overridden by `qossubspacebias=true|false` command-line arg.
	 * 
	 * @return True if rules-based sorting (when dealing with subspaces) is enabled, otherwise false.
	 */
	UE_API bool IsSubspaceBiasOrderEnabled() const;

	/**
	 * Should datacenter QoS results be sorted using rules-based comparison where subspaces are encountered
	 * for the given region's QoS data?
	 * 
	 * Determined via bEnableSubspaceBiasOrder engine configuration param for QosRegionManager,
	 * and the specific region's RegionDefinition entry.
	 * Global enable/disable may be overridden by `qossubspacebias=true|false` command-line arg.
	 *
	 * @return True if rules-based sorting (when dealing with subspaces) is enabled, otherwise false.
	 */
	UE_API bool IsSubspaceBiasOrderEnabled(const FQosRegionInfo& RegionDefinition) const;

	/**
	 * Rebuild the list of usable subregions sorted by ping ascending.
	 */
	void RefreshUsableSubregions();

	/** Number of times to ping a given region using random sampling of available servers */
	UPROPERTY(Config)
	int32 NumTestsPerRegion;

	/** Timeout value for each ping request */
	UPROPERTY(Config)
	float PingTimeout;

	/**
	 * Global switch to enable/disable sorting of QoS datacenter results using rules-based comparison,
	 * where subspaces are encountered.
	 */
	UPROPERTY(Config)
	bool bEnableSubspaceBiasOrder;

	/**
	 * Delimiter string that identifies a subspace datacenter ID.
	 * e.g. "DE_S" would be a subspace of the "DE" subregion, using "_" as the delimiter.
	 */
	UPROPERTY(Config)
	FString SubspaceDelimiter;

	/** Granular settings for biased subspace-based sorting algorithm which applies when returning all subregions for queries */
	UPROPERTY(Config)
	FQosSubspaceComparisonParams SubspaceBiasParams;

	/** Metadata about existing regions */
	UPROPERTY(Config)
	TArray<FQosRegionInfo> RegionDefinitions;

	/** Metadata about datacenters within existing regions */
	UPROPERTY(Config)
	TArray<FQosDatacenterInfo> DatacenterDefinitions;

	UPROPERTY()
	FDateTime LastCheckTimestamp;

	/** Reference to the evaluator for making datacenter determinations (null when not active) */
	UPROPERTY()
	TObjectPtr<UQosEvaluator> Evaluator;
	/** Result of the last datacenter test */
	UPROPERTY()
	EQosCompletionResult QosEvalResult;
	/** Array of all known regions and the datacenters in them */
	UPROPERTY()
	TArray<FRegionQosInstance> RegionOptions;

	/** Value forced to be the region (development) */
	UPROPERTY()
	FString ForceRegionId;
	/** Was the region forced via commandline */
	UPROPERTY()
	bool bRegionForcedViaCommandline;
	/** Value set by the game to be the current region */
	UPROPERTY()
	FString SelectedRegionId;
	/** List of all useable subregions sorted by ping. Does not include accelerated regions. */
	TArray<FString> UsableSubregions;

	FOnQosEvalCompleteDelegate OnQosEvalCompleteDelegate;

	FSimpleDelegate OnQoSSettingsChangedDelegate;

	FOnQosRegionIdChanged OnQosRegionIdChangedDelegate;

	static UE_API const TCHAR* SubspaceDelimiterDefault;
};

#undef UE_API
