// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class Qos : ModuleRules
{
	public Qos(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDefinitions.Add("QOS_PACKAGE=1");

		PrivateDependencyModuleNames.AddRange(
			new string[] { 
				"Core", 
				"CoreUObject",
				"Engine", 
				"Json",
                "Icmp",
				"OnlineSubsystem",
				"OnlineSubsystemUtils"
			}
		);

		DynamicallyLoadedModuleNames.AddRange(
			new string[] {
				"Analytics",
				"AnalyticsET"
			}
		);

		PrivateIncludePathModuleNames.AddRange(
			new string[] {
				"Analytics"
			}
		);

		if (Target.Platform == UnrealTargetPlatform.Win64)
		{
		}
		else if (Target.Platform == UnrealTargetPlatform.Mac)
		{
		}
	}
}
