// Copyright Epic Games, Inc. All Rights Reserved.

#include "RejoinCheck.h"
#include "Engine/GameInstance.h"
#include "Interfaces/OnlineSessionInterface.h"
#include "OnlineSubsystemUtils.h"
#include "Engine/LocalPlayer.h"
#include "TimerManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RejoinCheck)

#define REJOIN_CHECK_TIMER 30.0f

static TAutoConsoleVariable<int32> CVarDebugRejoin(
	TEXT("UI.DebugRejoin"),
	-1,
	TEXT("Force switch between rejoin states (-1 is off)"));

URejoinCheck::URejoinCheck()
	: LastKnownStatus(ERejoinStatus::NeedsRecheck)
	, bRejoinAfterCheck(false)
	, bAttemptingRejoin(false)
{
}

bool URejoinCheck::IsRejoinCheckEnabled() const
{
	return true;
}

#if !UE_BUILD_SHIPPING
ERejoinStatus URejoinCheck::GetStatus() const
{
	int32 DbgVal = CVarDebugRejoin.GetValueOnGameThread();
	if (DbgVal >= 0 && DbgVal <= (int32)ERejoinStatus::NoMatchToRejoin_MatchEnded)
	{
		return (ERejoinStatus)DbgVal;
	}

	if (!IsRejoinCheckEnabled())
	{
		return ERejoinStatus::NoMatchToRejoin;
	}

	return LastKnownStatus;
}
#endif

bool URejoinCheck::HasCompletedCheck() const
{
	ERejoinStatus CurrentStatus = GetStatus();
	return (CurrentStatus != ERejoinStatus::NeedsRecheck && CurrentStatus != ERejoinStatus::UpdatingStatus);
}

bool URejoinCheck::IsRejoinAvailable() const
{
	ERejoinStatus CurrentStatus = GetStatus();
	return (CurrentStatus != ERejoinStatus::NoMatchToRejoin &&
		CurrentStatus != ERejoinStatus::NoMatchToRejoin_MatchEnded);
}

void URejoinCheck::CheckRejoinStatus(const FOnRejoinCheckComplete& InCompletionDelegate)
{
#if !UE_BUILD_SHIPPING
	if (!IsRejoinCheckEnabled())
	{
		SetStatus(ERejoinStatus::NoMatchToRejoin);
		InCompletionDelegate.ExecuteIfBound(LastKnownStatus);
		return;
	}
#endif

	if (LastKnownStatus != ERejoinStatus::UpdatingStatus)
	{
		bool bSuccess = false;

		SetStatus(ERejoinStatus::UpdatingStatus);

		UGameInstance* GameInstance = GetGameInstance<UGameInstance>();
		check(GameInstance);

		UWorld* World = GetWorld();
		check(World);

		IOnlineSessionPtr SessionInt = Online::GetSessionInterface(World);
		if (SessionInt.IsValid())
		{
			IOnlineIdentityPtr IdentityInt = Online::GetIdentityInterface(World);
			if (IdentityInt.IsValid())
			{
				FUniqueNetIdRepl PrimaryUniqueId = GameInstance->GetPrimaryPlayerUniqueIdRepl();
				if (ensure(PrimaryUniqueId.IsValid()))
				{
					ULocalPlayer* LP = GameInstance->FindLocalPlayerFromUniqueNetId(PrimaryUniqueId);
					if (ensure(LP))
					{
						FOnFindFriendSessionCompleteDelegate CompletionDelegate;
						CompletionDelegate.BindUObject(this, &ThisClass::OnCheckRejoinComplete, InCompletionDelegate);

						FindFriendSessionCompleteDelegateHandle = SessionInt->AddOnFindFriendSessionCompleteDelegate_Handle(LP->GetControllerId(), CompletionDelegate);
						bSuccess = SessionInt->FindFriendSession(*PrimaryUniqueId, *PrimaryUniqueId);
					}
					else
					{
						UE_LOGF(LogOnline, Warning, "Invalid local player during rejoin check");
					}
				}
				else
				{
					UE_LOGF(LogOnline, Warning, "Invalid user during rejoin check");
				}
			}
			else
			{
				UE_LOGF(LogOnline, Warning, "No identity interface during rejoin check");
			}
		}
		else
		{
			UE_LOGF(LogOnline, Warning, "No session interface during rejoin check");
		}

		if (!bSuccess)
		{
			TArray<FOnlineSessionSearchResult> EmptySearchResults;
			ProcessRejoinCheck(false, EmptySearchResults, InCompletionDelegate);
		}
	}
	else
	{
		UE_LOGF(LogOnline, Log, "Rejoin check in progress, ignoring call.");
		InCompletionDelegate.ExecuteIfBound(ERejoinStatus::UpdatingStatus);
	}
}

void URejoinCheck::OnCheckRejoinComplete(int32 ControllerId, bool bWasSuccessful, const TArray<FOnlineSessionSearchResult>& InSearchResults, FOnRejoinCheckComplete InCompletionDelegate)
{
	UWorld* World = GetWorld();
	check(World);

	IOnlineSessionPtr SessionInt = Online::GetSessionInterface(World);
	if (SessionInt.IsValid())
	{
		SessionInt->ClearOnFindFriendSessionCompleteDelegate_Handle(ControllerId, FindFriendSessionCompleteDelegateHandle);
	}

	ProcessRejoinCheck(bWasSuccessful, InSearchResults, InCompletionDelegate);
}

void URejoinCheck::ProcessRejoinCheck(bool bWasSuccessful, const TArray<FOnlineSessionSearchResult>& InSearchResults, const FOnRejoinCheckComplete& InCompletionDelegate)
{
	if (LastKnownStatus == ERejoinStatus::UpdatingStatus)
	{
		ERejoinStatus NewStatus = bShouldRecheckIfRejoinCheckUnsuccessful ? ERejoinStatus::NeedsRecheck : ERejoinStatus::NoMatchToRejoin;
		if (bWasSuccessful)
		{
			NewStatus = ERejoinStatus::NoMatchToRejoin;
			if (InSearchResults.Num() > 0 && InSearchResults[0].IsValid())
			{
				NewStatus = GetRejoinStateFromSearchResult(InSearchResults[0]);
			}

			if (NewStatus == ERejoinStatus::RejoinAvailable && InSearchResults.Num() > 0)
			{
				if (InSearchResults[0].GetSessionIdStr() != SearchResult.GetSessionIdStr())
				{
					// Record the analytics before the search result assignment
					// so the event is only sent once per unique session id
					Analytics_RecordRejoinDetected(InSearchResults[0]);
				}
				SearchResult = InSearchResults[0];
			}
			else
			{
				SearchResult = FOnlineSessionSearchResult();
			}
		}

		SetStatus(NewStatus);

		// Could be external delegate or internal call to OnFinalRejoinCheckComplete
		InCompletionDelegate.ExecuteIfBound(NewStatus);
		
		if (bAttemptingRejoin)
		{
			if (bRejoinAfterCheck)
			{
				// Manual call because of call to rejoin in middle of another update
				// delegate call above should not be same as this call
				bRejoinAfterCheck = false;
				OnFinalRejoinCheckComplete(LastKnownStatus);
			}
		}
		else if (IsRejoinAvailable())
		{	
			// Keep looking for the match
			StartRejoinChecks();
		}
	}
}

void URejoinCheck::RejoinCheckTimer()
{
	if (LastKnownStatus != ERejoinStatus::UpdatingStatus)
	{
		CheckRejoinStatus();
	}
}

void URejoinCheck::RejoinLastSession(const FOnRejoinLastSessionComplete& InCompletionDelegate)
{
	if (LastKnownStatus == ERejoinStatus::UpdatingStatus)
	{
		if (!bAttemptingRejoin)
		{
			bAttemptingRejoin = true;

			UE_LOGF(LogOnline, Warning, "RejoinLastSession called while check in progress, will react on completion");
			bRejoinAfterCheck = true;
			RejoinLastSessionCompleteDelegate = InCompletionDelegate;
		}
		else
		{
			UE_LOGF(LogOnline, Warning, "RejoinLastSession called already attempting a rejoin.");
			InCompletionDelegate.ExecuteIfBound(ERejoinAttemptResult::RejoinInProgress);
		}
	}
	else if (IsRejoinAvailable())
	{
		if (!bAttemptingRejoin)
		{
			bAttemptingRejoin = true;
			
			RejoinLastSessionCompleteDelegate = InCompletionDelegate;

			// Stop any recheck timer, the game will either be traveling, or reset in OnRejoinFailure
			ClearTimers();

			// Always check one last time to make sure nothing has changed
			FOnRejoinCheckComplete CompletionDelegate;
			CompletionDelegate.BindUObject(this, &ThisClass::OnFinalRejoinCheckComplete);
			CheckRejoinStatus(CompletionDelegate);
		}
		else
		{
			UE_LOGF(LogOnline, Warning, "RejoinLastSession called already attempting a rejoin.");
			InCompletionDelegate.ExecuteIfBound(ERejoinAttemptResult::RejoinInProgress);
		}
	}
	else
	{
		UE_LOGF(LogOnline, Warning, "RejoinLastSession called but no session to join");
		InCompletionDelegate.ExecuteIfBound(ERejoinAttemptResult::NothingToRejoin);
	}
}

void URejoinCheck::SetStatus(ERejoinStatus NewStatus)
{
	if (LastKnownStatus != NewStatus)
	{
		LastKnownStatus = NewStatus;
		OnRejoinCheckStatusChanged().Broadcast(NewStatus);
	}
}

void URejoinCheck::Reset()
{
	bRejoinAfterCheck = false;
	bAttemptingRejoin = false;

	SearchResult = FOnlineSessionSearchResult();

	ClearTimers();
	SetStatus(ERejoinStatus::NeedsRecheck);
}

void URejoinCheck::ClearTimers()
{
	const UWorld* const World = GetWorld();
	if (World)
	{
		if (RejoinCheckTimerHandle.IsValid())
		{
			FTimerManager& TM = World->GetTimerManager();
			TM.ClearTimer(RejoinCheckTimerHandle);
			RejoinCheckTimerHandle.Invalidate();
		}
	}
}

void URejoinCheck::OnFinalRejoinCheckComplete(ERejoinStatus Result)
{
	UE_LOGF(LogOnline, Verbose, "OnFinalRejoinCheckComplete %ls", ToString(Result));
	if (Result == ERejoinStatus::RejoinAvailable)
	{
		// Tell the game it's expected to rejoin the discovered session
		RejoinViaSession();
	}
	else
	{
		const bool bNoRejoin = (Result == ERejoinStatus::NoMatchToRejoin || Result == ERejoinStatus::NoMatchToRejoin_MatchEnded);
		OnRejoinFailure(bNoRejoin ? ERejoinAttemptResult::NothingToRejoin : ERejoinAttemptResult::RejoinFailure);
	}
}

void URejoinCheck::TravelToSession()
{
	bool bResult = false;

	// TODO: What should we do if this fails? Will need to destroy session, etc.
	UGameInstance* GameInstance = GetGameInstance<UGameInstance>();
	check(GameInstance);

	ULocalPlayer* LP = GEngine->GetFirstGamePlayer(GetWorld());
	if (ensure(LP))
	{
		bResult = GameInstance->ClientTravelToSession(LP->GetControllerId(), NAME_GameSession);
		if (bResult)
		{
			UE_LOGF(LogOnline, Log, "URejoinCheck::TravelToSession: Performing ClientTravelToSession");

			// Record the result of the attempt to rejoin
			Analytics_RecordRejoinAttempt(SearchResult, ERejoinAttemptResult::RejoinSuccess);

			// Reset the rejoin status while in game (any failure or future quit with recheck)
			Reset();
			OnRejoinLastSessionComplete().ExecuteIfBound(ERejoinAttemptResult::RejoinSuccess);
		}
		else
		{
			UE_LOGF(LogOnline, Log, "URejoinCheck::TravelToSession: Failed to travel to session");
		}
	}
	else
	{
		UE_LOGF(LogOnline, Log, "URejoinCheck::TravelToSession: Failed to find local player");
	}
	
	if (!bResult)
	{
		OnRejoinFailure(ERejoinAttemptResult::RejoinTravelFailure);
	}
}

void URejoinCheck::OnRejoinFailure(ERejoinAttemptResult Result)
{
	UE_LOGF(LogOnline, Warning, "OnRejoinFailure %ls", ToString(Result));

	// Record the result of the attempt to rejoin
	Analytics_RecordRejoinAttempt(SearchResult, Result);

	bAttemptingRejoin = false;

	if (Result == ERejoinAttemptResult::NothingToRejoin)
	{
		SetStatus(ERejoinStatus::NoMatchToRejoin);
	}
	else
	{
		SetStatus(ERejoinStatus::NeedsRecheck);
		StartRejoinChecks();
	}

	OnRejoinLastSessionComplete().ExecuteIfBound(Result);
}

void URejoinCheck::StartRejoinChecks()
{
	FTimerDelegate TimerDelegate;
	TimerDelegate.BindUObject(this, &ThisClass::RejoinCheckTimer);

	FTimerManager& TM = GetWorld()->GetTimerManager();
	TM.SetTimer(RejoinCheckTimerHandle, TimerDelegate, REJOIN_CHECK_TIMER, false);
}

UWorld* URejoinCheck::GetWorld() const
{
	UGameInstance* GameInstance = GetGameInstance<UGameInstance>();
	if (GameInstance)
	{
		return GameInstance->GetWorld();
	}

	return nullptr;
}



