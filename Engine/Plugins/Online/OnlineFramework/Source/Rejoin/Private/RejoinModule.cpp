// Copyright Epic Games, Inc. All Rights Reserved.

#include "RejoinModule.h"
#include "Modules/ModuleManager.h"

IMPLEMENT_MODULE(FRejoinModule, Rejoin);

void FRejoinModule::StartupModule()
{
}

void FRejoinModule::ShutdownModule()
{
}
