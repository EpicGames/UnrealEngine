// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Engine/TimerHandle.h"
#include "OnlineSessionSettings.h"
#include "RejoinCheck.generated.h"

#define UE_API REJOIN_API

class FTimerManager;

/**
 * Possible states that a rejoin check can be in at any given time
 */
UENUM(BlueprintType)
enum class ERejoinStatus : uint8
{
	// There is no match to rejoin.  The user is already in a match or there is no match in progress for the user. 
	NoMatchToRejoin,
	// There is a rejoin available for the user
	RejoinAvailable,
	// We are currently updating the status of rejoin
	UpdatingStatus,
	// We need to recheck the state before allowing any further progress through the UI (e.g right after login or right after leaving a match without it ending normally).
	NeedsRecheck,
	// Match ended normally, no check required (only set when returning from a match)
	NoMatchToRejoin_MatchEnded
};

/**
 * Delegate fired when a rejoin check has completed against the backend
 *
 * @param RejoinStatus status of the rejoin check attempt
 */
DECLARE_DELEGATE_OneParam(FOnRejoinCheckComplete, ERejoinStatus /** RejoinStatus */);

/**
 * Multicast delegate fired when a rejoin check state has changed
 *
 * @param NewStatus newest status of the rejoin check attempt
 */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnRejoinCheckStatusChanged, ERejoinStatus /** NewStatus */);
typedef FOnRejoinCheckStatusChanged::FDelegate FOnRejoinCheckStatusChangedDelegate;

/**
 * Possible end conditions that a rejoin attempt can be in after a user indicates intent to rejoin a session
 */
enum class ERejoinAttemptResult : uint8
{
	/** Generic failure */
	RejoinFailure,
	/** Already trying to rejoin */
	RejoinInProgress,
	/** Successfully going to travel into match */
	RejoinSuccess,
	/** Match disappeared while trying to join it */
	NothingToRejoin,
	/** Session interface failure */
	InvalidSessionFailure,
	/** Join Session failure */
	JoinSessionFailure,
	/** Failure trying to travel to session */
	RejoinTravelFailure
};

/**
 * Delegate fired at the completion of an attempt to rejoin a session
 *
 * @param RejoinResult status of the rejoin attempt
 */
DECLARE_DELEGATE_OneParam(FOnRejoinLastSessionComplete, ERejoinAttemptResult /** RejoinResult */);

/**
 * Class responsible for maintaining the status/availability of a session already in progress for a client to join
 */
UCLASS(MinimalAPI, abstract, Config=Engine)
class URejoinCheck : public UObject
{
	GENERATED_BODY()

public:

	UE_API URejoinCheck();

	/**
	 * Check the backend for the existence of game session that the local player is registered with
	 * It will continue to return a valid value until that session is complete
	 *
	 * @param CompletionDelegate delegate called after the check for possible rejoin is complete
	 */
	UE_API virtual void CheckRejoinStatus(const FOnRejoinCheckComplete& CompletionDelegate = FOnRejoinCheckComplete());

	/**
	 * Rejoin the last session if one is found.  One final call to CheckRejoinStatus is made to
	 * verify the session still exists
	 * 
	 * @param InCompletionDelegate delegate called after the rejoin attempt is complete
	 */
	UE_API void RejoinLastSession(const FOnRejoinLastSessionComplete& InCompletionDelegate);

	/**
	 * Manually set the status of rejoins.  Used when entering/leaving a map as a hint for future check requirements
	 *
	 * @param NewStatus change current status to a new value
	 */
	UE_API void SetStatus(ERejoinStatus NewStatus);

	/**
	 * Access to the multicast delegate fired when a rejoin check status update is given
	 */
	FOnRejoinCheckStatusChanged& OnRejoinCheckStatusChanged() { return RejoinCheckStatusChanged; }
	
	/**
	 * 
	 */
	UE_API virtual bool IsRejoinCheckEnabled() const;

	/**
	 * Reset the rejoin state. Sets status to NeedsRecheck, clears the cached search result, timer, and flags.
	 */
	UE_API virtual void Reset();

	/**
	 * Get the current state in the rejoin check flow.  
	 * Dev mode may return CVarDebugRejoin if it was set
	 */
#if UE_BUILD_SHIPPING
	ERejoinStatus GetStatus() const { return LastKnownStatus; }
#else
	UE_API ERejoinStatus GetStatus() const;
#endif

	/** @return True if the rejoin check has completed and does not need to be rerun. */
	UE_API bool HasCompletedCheck() const;

	/** @return True if it's possible that there's a match to rejoin */
	UE_API bool IsRejoinAvailable() const;

protected: 

	/**
	 * Called any time there is a failure to complete the attempted rejoin
	 *
	 * @param Result rejoin failure reason
	 */
	UE_API virtual void OnRejoinFailure(ERejoinAttemptResult Result);

	/**
	 * Use the search result to travel to the given server session
	 * Called after a session has been joined by the online platform
	 */
	UE_API void TravelToSession();

	/**
	 * Helpers
	 */

	/** @return the search result that is currently associated with a rejoin */
	const FOnlineSessionSearchResult& GetSearchResult() const { return SearchResult; }
	/** @return true if a rejoin attempt is in progress */
	bool IsAttemptingRejoin() const { return bAttemptingRejoin; }

	UE_API UWorld* GetWorld() const;

	template<typename T>
	T* GetGameInstance() const
	{
		return GetTypedOuter<T>();
	}

	/** Clear all timers associated with rejoin */
	UE_API void ClearTimers();

	/** Rejoin status */
	UPROPERTY()
	ERejoinStatus LastKnownStatus;

private:

	/** Flag set during a possible brief period where the user hit rejoin but the check was already in flight */
	UPROPERTY()
	bool bRejoinAfterCheck;
	/** Is a rejoin attempt in progress, prevents reentry */
	UPROPERTY()
	bool bAttemptingRejoin;

	/** Defines our default behavior if a rejoin check is unsuccessful, namely, should we recheck? */
	UPROPERTY(Config)
	bool bShouldRecheckIfRejoinCheckUnsuccessful = true;

	/** Cached value of the last session expected to rejoin */
	FOnlineSessionSearchResult SearchResult;

	/** Delegate fired when rejoin check status changes */
	FOnRejoinCheckStatusChanged RejoinCheckStatusChanged;

	/** FindFriendSession delegate handle if a call is in flight */
	FDelegateHandle FindFriendSessionCompleteDelegateHandle;
	/** Handle to the possibly active timer for another rejoin check */
	FTimerHandle RejoinCheckTimerHandle;

	/** Delegate fired when a rejoin attempt completed */
	FOnRejoinLastSessionComplete RejoinLastSessionCompleteDelegate;
	FOnRejoinLastSessionComplete& OnRejoinLastSessionComplete() { return RejoinLastSessionCompleteDelegate;  }

	/**
	 * Interpret a given search result for the possible need to rejoin an existing session
	 *
	 * @param InSearchResult search result discovered to be related to the locally logged in user
	 *
	 * @return a rejoin status code that indicates whether or not this result should be pursued
	 */
	virtual ERejoinStatus GetRejoinStateFromSearchResult(const FOnlineSessionSearchResult& InSearchResult) const PURE_VIRTUAL(URejoinCheck::GetRejoinStateFromSearchResult, return ERejoinStatus::NeedsRecheck;);
	
	/**
	 * Delegate fired when a rejoin check has completed
	 *
	 * @param ControllerId controller id of the calling user
	 * @param bWasSuccessful did the call reach the backend and return with a valid result
	 * @param InSearchSearch possibly valid search result to rejoin depending on the success of the call
	 * @param InCompletionDelegate external delegate to call after finishing internal code
	 */
	UE_API void OnCheckRejoinComplete(int32 ControllerId, bool bWasSuccessful, const TArray<FOnlineSessionSearchResult>& InSearchResults, FOnRejoinCheckComplete InCompletionDelegate);

	/**
	 * Common function for handling the result of a rejoin check
	 *
	 * @param bWasSuccessful did the call reach the backend and return with a valid result
	 * @param InSearchSearch possibly valid search result to rejoin depending on the success of the call
	 * @param InCompletionDelegate external delegate to call after finishing internal code
	 */
	UE_API void ProcessRejoinCheck(bool bWasSuccessful, const TArray<FOnlineSessionSearchResult>& InSearchResults, const FOnRejoinCheckComplete& InCompletionDelegate);

	/**
	 * Delegate fired after the last rejoin check completed with the intention of joining a search result if valid
	 *
	 * @param Result final result of the rejoin check
	 */
	UE_API void OnFinalRejoinCheckComplete(ERejoinStatus Result);

	/**
	 * Game specific method to rejoin the last session in progress
	 * Use OnRejoinFailure to communicate with base class the state of the rejoin
	 */
	virtual void RejoinViaSession() PURE_VIRTUAL(URejoinCheck::RejoinViaSession, );

	/**
	 * Set the timer for another rejoin search after a given time period
	 */
	UE_API void StartRejoinChecks();
	
	/**
	 * Called after the rejoin check timer expires to make another backend request
	 */
	UE_API void RejoinCheckTimer();

	/**
	 * Analytics
	 */

	virtual void Analytics_RecordRejoinDetected(const FOnlineSessionSearchResult& InSearchResult) const {};
	virtual void Analytics_RecordRejoinAttempt(const FOnlineSessionSearchResult& InSearchResult, ERejoinAttemptResult InAttemptResult) const {};
};

inline const TCHAR* ToString(ERejoinStatus Result)
{
	switch (Result)
	{
		case ERejoinStatus::NoMatchToRejoin:
		{
			return TEXT("RejoinNotRequired");
		}
		case ERejoinStatus::RejoinAvailable:
		{
			return TEXT("RejoinFound");
		}
		case ERejoinStatus::UpdatingStatus:
		{
			return TEXT("RejoinCheckFailure");
		}
		case ERejoinStatus::NeedsRecheck:
		{
			return TEXT("NeedsRecheck");
		}
		case ERejoinStatus::NoMatchToRejoin_MatchEnded:
		{
			return TEXT("NoMatchToRejoin_MatchEnded");
		}
	}

	return TEXT("");
}

inline const TCHAR* ToString(ERejoinAttemptResult Result)
{
	switch (Result)
	{
		case ERejoinAttemptResult::RejoinFailure:
		{
			return TEXT("RejoinFailure");
		}
		case ERejoinAttemptResult::RejoinInProgress:
		{
			return TEXT("RejoinInProgress");
		}
		case ERejoinAttemptResult::RejoinSuccess:
		{
			return TEXT("RejoinSuccess");
		}
		case ERejoinAttemptResult::NothingToRejoin:
		{
			return TEXT("RejoinNothingToRejoin");
		}
		case ERejoinAttemptResult::InvalidSessionFailure:
		{
			return TEXT("InvalidSessionFailure");
		}
		case ERejoinAttemptResult::JoinSessionFailure:
		{
			return TEXT("JoinSessionFailure");
		}
		case ERejoinAttemptResult::RejoinTravelFailure:
		{
			return TEXT("RejoinTravelFailure");
		}
	}

	return TEXT("");
}

#undef UE_API
