// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"

/**
 * Module for Rejoin base implementation
 */
class FRejoinModule : 
	public IModuleInterface
{
private:
// IModuleInterface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
};
